// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class LevelEditor : ModuleRules
{
	public LevelEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"ClassViewer",
				"MainFrame",
                "PlacementMode",
				"SlateReflector",
                "PortalServices",
				"MergeActors",
				"Layers",
				"WorldBrowser",
				"NewLevelDialog",
				"LocalizationDashboard",
				"TurnkeySupport",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"CommonMenuExtensions",
				"Settings",
				"ToolWidgets",
				"UnrealEd",
				"VREditor",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"LevelSequence",
				"Analytics",
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"LauncherPlatform",
				"InputCore",
				"Slate",
				"SlateCore",
				"Engine",
				"MessageLog",
				"SourceControl",
				"SourceControlWindows",
				"StatsViewer",
				"EditorFramework",
				"UnrealEd", 
				"DeviceProfileServices",
				"ContentBrowser",
				"SceneOutliner",
				"ActorPickerMode",
				"RHI",
				"Projects",
				"TypedElementFramework",
				"TypedElementRuntime",
				"EngineSettings",
				"PropertyEditor",
				"KismetWidgets",
				"Foliage",
				"HierarchicalLODOutliner",
				"HierarchicalLODUtilities",
				"MaterialShaderQualitySettings",
				"PixelInspectorModule",
				"CommonMenuExtensions",
				"ToolMenus",
				"StatusBar",
				"AppFramework",
				"EditorSubsystem",
				"EnvironmentLightingViewer",
				"DesktopPlatform",
				"DataLayerEditor",
				"TranslationEditor",
				"SubobjectEditor",
				"SubobjectDataInterface",
				"DerivedDataWidgets",
				"DerivedDataEditor",
				"ZenEditor",
				"EditorWidgets",
				"ToolWidgets",
				"UnsavedAssetsTracker",
				"UncontrolledChangelists",
				"RenderCore",
				"DeveloperSettings",
				"ActionableMessage",
				"Json",
				"JsonUtilities", 
				"EditorInteractiveToolsFramework",
				"InteractiveToolsFramework"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"ClassViewer",
				"DeviceManager",
				"SettingsEditor",
				"SlateReflector",
				"AutomationWindow",
				"Layers",
				"WorldBrowser",
				"WorldPartitionEditor",
				"AssetTools",
				"WorkspaceMenuStructure",
				"NewLevelDialog",
				"DeviceProfileEditor",
                "PlacementMode",
				"HeadMountedDisplay",
				"VREditor",
                "Persona",
				"MergeActors"
			}
		);

		if (Target.bBuildTargetDeveloperTools)
		{
			DynamicallyLoadedModuleNames.Add("SessionFrontend");
		}

		if (Target.bWithLiveCoding)
		{
			PrivateIncludePathModuleNames.Add("LiveCoding");
		}

		// This module needs AutoRTFM so disable the auto disable since its in a plugin.
		// The call to the delegate `OnObjectPropertyChanged` calls into here.
		bDisableAutoRTFMInstrumentation = false;
	}
}
