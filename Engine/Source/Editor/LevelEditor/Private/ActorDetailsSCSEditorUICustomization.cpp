// Copyright Epic Games, Inc. All Rights Reserved.
#include "ActorDetailsSCSEditorUICustomization.h"

#include "SubobjectData.h"

TSharedPtr<FActorDetailsSCSEditorUICustomization> FActorDetailsSCSEditorUICustomization::NewInstance()
{
	return MakeShared<FActorDetailsSCSEditorUICustomization>(FPrivateToken());
}

FActorDetailsSCSEditorUICustomization::FActorDetailsSCSEditorUICustomization(FPrivateToken)
{
}

bool FActorDetailsSCSEditorUICustomization::HideComponentsTree(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->HideComponentsTree(Context))
		{
			return true;
		}
	}

	return false;
}

bool FActorDetailsSCSEditorUICustomization::HideComponentsFilterBox(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->HideComponentsFilterBox(Context))
		{
			return true;
		}
	}

	return false;
}

bool FActorDetailsSCSEditorUICustomization::HideAddComponentButton(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->HideAddComponentButton(Context))
		{
			return true;
		}
	}

	return false;
}

bool FActorDetailsSCSEditorUICustomization::HideBlueprintButtons(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->HideBlueprintButtons(Context))
		{
			return true;
		}
	}

	return false;
}

const FSlateBrush* FActorDetailsSCSEditorUICustomization::GetIconBrush(const FSubobjectData& SubobjectData) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (const FSlateBrush* IconBrush = Customization->GetIconBrush(SubobjectData))
		{
			return IconBrush;
		}
	}
	return ISCSEditorUICustomization::GetIconBrush(SubobjectData);
}

TSharedPtr<SWidget> FActorDetailsSCSEditorUICustomization::GetControlsWidget(TSharedRef<SSubobjectEditor>& SubobjectEditor, const FSubobjectData& SubobjectData) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (TSharedPtr<SWidget> ControlsWidget = Customization->GetControlsWidget(SubobjectEditor, SubobjectData))
		{
			return ControlsWidget;
		}
	}
	return ISCSEditorUICustomization::GetControlsWidget(SubobjectEditor, SubobjectData);
}

bool FActorDetailsSCSEditorUICustomization::OverrideDesiredVisibility(const UE::Editor::FSubobjectEditorContext& Context, EVisibility& VisibilityOut) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->OverrideDesiredVisibility(Context, VisibilityOut))
		{
			return true;
		}
	}
	return ISCSEditorUICustomization::OverrideDesiredVisibility(Context, VisibilityOut);
}

EChildActorComponentTreeViewVisualizationMode FActorDetailsSCSEditorUICustomization::GetChildActorVisualizationMode() const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		EChildActorComponentTreeViewVisualizationMode VisualizationMode = Customization->GetChildActorVisualizationMode();
		if (VisualizationMode != EChildActorComponentTreeViewVisualizationMode::UseDefault)
		{
			return VisualizationMode;
		}
	}

	return EChildActorComponentTreeViewVisualizationMode::UseDefault;
}

TSubclassOf<UActorComponent> FActorDetailsSCSEditorUICustomization::GetComponentTypeFilter(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if(TSubclassOf<UActorComponent> SubClass = Customization->GetComponentTypeFilter(Context))
		{
			return SubClass;
		}
	}

	return nullptr;
}

bool FActorDetailsSCSEditorUICustomization::FilterSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData) const
{
	bool bChanged = false;
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		bChanged = Customization->FilterSubobjectData(SubobjectData) || bChanged;
	}
	return bChanged;
}

void FActorDetailsSCSEditorUICustomization::PrePotentialTargetChange(SSubobjectEditor& SubobjectEditor)
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		Customization->PrePotentialTargetChange(SubobjectEditor);
	}
}

bool FActorDetailsSCSEditorUICustomization::SelectDefaultSubobject(SSubobjectEditor& SubobjectEditor)
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->SelectDefaultSubobject(SubobjectEditor))
		{
			return true;
		}
	}
	return false;
}

void FActorDetailsSCSEditorUICustomization::AddCustomization(TSharedPtr<ISCSEditorUICustomization> Customization)
{
	Customizations.AddUnique(Customization);
}

void FActorDetailsSCSEditorUICustomization::RemoveCustomization(TSharedPtr<ISCSEditorUICustomization> Customization)
{
	Customizations.Remove(Customization);
}

bool FActorDetailsSCSEditorUICustomization::SupportsTreeViewOverlay(const SSubobjectEditor& Editor) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->SupportsTreeViewOverlay(Editor))
		{
			return true;
		}
	}
	return false;
}

TSharedRef<SWidget> FActorDetailsSCSEditorUICustomization::ConstructTreeViewOverlay(TWeakPtr<SSubobjectEditor> Editor) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		const TSharedRef<SWidget> Overlay = Customization->ConstructTreeViewOverlay(Editor);
		if (Overlay != SNullWidget::NullWidget)
		{
			return Overlay;
		}
	}

	return SNullWidget::NullWidget;
}
