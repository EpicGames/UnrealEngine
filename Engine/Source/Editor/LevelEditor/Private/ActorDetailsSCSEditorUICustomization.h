// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISCSEditorUICustomization.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class FActorDetailsSCSEditorUICustomization final : public ISCSEditorUICustomization
{
private:
	struct FPrivateToken { explicit FPrivateToken() = default; };
public:
	static TSharedPtr<FActorDetailsSCSEditorUICustomization> NewInstance();

	explicit FActorDetailsSCSEditorUICustomization(FPrivateToken);
	virtual ~FActorDetailsSCSEditorUICustomization() override {}

	virtual bool HideComponentsTree(TArrayView<UObject*> Context) const override;

	virtual bool HideComponentsFilterBox(TArrayView<UObject*> Context) const override;

	virtual bool HideAddComponentButton(TArrayView<UObject*> Context) const override;

	virtual bool HideBlueprintButtons(TArrayView<UObject*> Context) const override;

	virtual const FSlateBrush* GetIconBrush(const FSubobjectData&) const override;
	virtual TSharedPtr<SWidget> GetControlsWidget(TSharedRef<SSubobjectEditor>&, const FSubobjectData&) const override;
	virtual bool OverrideDesiredVisibility(const UE::Editor::FSubobjectEditorContext& Context, EVisibility& VisibilityOut) const;

	virtual EChildActorComponentTreeViewVisualizationMode GetChildActorVisualizationMode() const override;

	virtual TSubclassOf<UActorComponent> GetComponentTypeFilter(TArrayView<UObject*> Context) const override;
	
	virtual bool FilterSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData) const override;

	virtual void PrePotentialTargetChange(SSubobjectEditor& SubobjectEditor) override;
	virtual bool SelectDefaultSubobject(SSubobjectEditor& SubobjectEditor) override;

	virtual bool SupportsTreeViewOverlay(const SSubobjectEditor& Editor) const override;
	virtual TSharedRef<SWidget> ConstructTreeViewOverlay(TWeakPtr<SSubobjectEditor> Editor) const override;

	void AddCustomization(TSharedPtr<ISCSEditorUICustomization> Customization);
	void RemoveCustomization(TSharedPtr<ISCSEditorUICustomization> Customization);

private:
	FActorDetailsSCSEditorUICustomization() {}
	
	TArray<TSharedPtr<ISCSEditorUICustomization>> Customizations;
};