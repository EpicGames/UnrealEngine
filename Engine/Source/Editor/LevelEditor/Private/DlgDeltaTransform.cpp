// Copyright Epic Games, Inc. All Rights Reserved.

#include "DlgDeltaTransform.h"
#include "InputCoreTypes.h"
#include "Input/Reply.h"
#include "Math/MathFwd.h"
#include "Misc/AxisDisplayInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Input/SVectorInputBox.h"

#define LOCTEXT_NAMESPACE "DeltaTransform"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
//	SDlgDeltaTransform
//
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
class SDlgDeltaTransform : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SDlgDeltaTransform )
		{}
		
		/** Window in which this widget resides */
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		
	SLATE_END_ARGS()

	/** Used to construct widgets */
	void Construct( const FArguments& InArgs )
	{
		// Set this widget as focused, to allow users to hit ESC to cancel.
		ParentWindow = InArgs._ParentWindow.Get();
		ParentWindow->SetWidgetToFocusOnActivate(SharedThis(this));

		DeltaTransform.Set(FVector::ZeroVector);

		using SVectorEntryBox = SNumericVectorInputBox<FVector::FReal, FVector, 3>;

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				SNew(SVerticalBox)
			
				// Add user input block
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(3)
					[
						SNew( SHorizontalBox )
						+ SHorizontalBox::Slot()
						.FillWidth(1)
						.VAlign( VAlign_Center )
						[
							SNew( SVectorEntryBox )
							.X( this, &SDlgDeltaTransform::GetDeltaX )
							.Y( this, &SDlgDeltaTransform::GetDeltaY )
							.Z( this, &SDlgDeltaTransform::GetDeltaZ )
							.XDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Forward))
							.YDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Left))
							.ZDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Up))
							.bColorAxisLabels( true )
							.Swizzle(AxisDisplayInfo::GetTransformAxisSwizzle())
							.AllowSpin( false )
							.OnXCommitted( this, &SDlgDeltaTransform::OnSetDelta, 0 )
							.OnYCommitted( this, &SDlgDeltaTransform::OnSetDelta, 1 )
							.OnZCommitted( this, &SDlgDeltaTransform::OnSetDelta, 2 )
						]
					]
				]
					
				// Add Ok, Ok to all and Cancel buttons.
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5)
				.HAlign(HAlign_Right)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+SUniformGridPanel::Slot(0,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text( NSLOCTEXT("ModalDialogs", "SDlgDeltaTransform_OK", "OK") )
						.OnClicked( this, &SDlgDeltaTransform::OnButtonClick, FDlgDeltaTransform::OK )
					]
					+SUniformGridPanel::Slot(1,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text( NSLOCTEXT("ModalDialogs", "SDlgDeltaTransform_Cancel", "Cancel") )
						.OnClicked( this, &SDlgDeltaTransform::OnButtonClick, FDlgDeltaTransform::Cancel )
					]
				]
			]
		];
	}
	
	SDlgDeltaTransform()
	: bUserResponse(FDlgDeltaTransform::Cancel)
	{
	}
	
	/** 
	 * Returns the EResult of the button which the user pressed, if the user
	 * canceled the action using ESC it will return as if canceled. 
	 */
	FDlgDeltaTransform::EResult GetUserResponse() const 
	{
		return bUserResponse; 
	}
		
	/** Override the base method to allow for keyboard focus */
	virtual bool SupportsKeyboardFocus() const
	{
		return true;
	}

	/** Used to intercept Escape key presses, then interprets them as cancel */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
	{
		// Pressing escape returns as if the user canceled
		if ( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnButtonClick(FDlgDeltaTransform::Cancel);
		}

		return FReply::Unhandled();
	}

private:
		/** A vector where it may optionally be unset */
	struct FOptionalVector
	{
		/**
		 * Sets the value from an FVector                   
		 */
		void Set( const FVector& InVec )
		{
			X = InVec.X;
			Y = InVec.Y;
			Z = InVec.Z;
		}

		/**
		 * Sets the value from an FRotator                   
		 */
		void Set( const FRotator& InRot )
		{
			X = InRot.Roll;
			Y = InRot.Pitch;
			Z = InRot.Yaw;
		}

		/** @return Whether or not the value is set */
		bool IsSet() const
		{
			// The vector is set if all values are set
			return X.IsSet() && Y.IsSet() && Z.IsSet();
		}

		TOptional<FVector::FReal> X;
		TOptional<FVector::FReal> Y;
		TOptional<FVector::FReal> Z;
	};

	/** 
	 * Handles when a button is pressed, should be bound with appropriate EResult Key
	 * 
	 * @param ButtonID - The return type of the button which has been pressed.
	 */
	FReply OnButtonClick(FDlgDeltaTransform::EResult ButtonID)
	{
		ParentWindow->RequestDestroyWindow();
		bUserResponse = ButtonID;

		if (ButtonID == FDlgDeltaTransform::OK)
		{
			const bool bIsAxisDisplayLeftUpForward = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;

			const FVector::FReal DeltaX = DeltaTransform.X.GetValue();
			const FVector::FReal DeltaY = DeltaTransform.Y.GetValue();
			const FVector::FReal DeltaZ = DeltaTransform.Z.GetValue();		

			GUnrealEd->Exec( GEditor->GetEditorWorldContext().World(), 
				*FString::Printf(TEXT("ELEMENT DELTAMOVE X=%.5f Y=%.5f Z=%.5f"), 
				DeltaX, bIsAxisDisplayLeftUpForward ? -DeltaY : DeltaY, DeltaZ) );
		}

		return FReply::Handled();
	}

	TOptional<FVector::FReal> GetDeltaX() const
	{
		return DeltaTransform.X;
	}

	TOptional<FVector::FReal> GetDeltaY() const
	{
		return DeltaTransform.Y;
	}

	TOptional<FVector::FReal> GetDeltaZ() const
	{
		return DeltaTransform.Z;
	}

	void OnSetDelta(FVector::FReal NewValue, ETextCommit::Type CommitInfo, int32 Axis )
	{
		switch (Axis)
		{
		case 0:
			DeltaTransform.X = NewValue;
			break;
		case 1:
			DeltaTransform.Y = NewValue;
			break;
		case 2:
			DeltaTransform.Z = NewValue;
			break;
		default:
			break;
		}
	}

	FOptionalVector DeltaTransform;

	/** Used to cache the users response to the warning */
	FDlgDeltaTransform::EResult bUserResponse;

	/** Pointer to the window which holds this Widget, required for modal control */
	TSharedPtr<SWindow> ParentWindow;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
//	FDlgDeltaTransform
//
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FDlgDeltaTransform::FDlgDeltaTransform()
{
	if (FSlateApplication::IsInitialized())
	{
		DeltaTransformWindow = SNew(SWindow)
			.Title(LOCTEXT("DeltaTransformDlgTitle", "Delta Transform"))
			.SupportsMinimize(false) .SupportsMaximize(false)
			.SizingRule( ESizingRule::Autosized );

		DeltaTransformWidget = SNew(SDlgDeltaTransform)
		.ParentWindow(DeltaTransformWindow);

		DeltaTransformWindow->SetContent( DeltaTransformWidget.ToSharedRef() );
	}
}

FDlgDeltaTransform::EResult FDlgDeltaTransform::ShowModal()
{
	GEditor->EditorAddModalWindow(DeltaTransformWindow.ToSharedRef());
	return (EResult)DeltaTransformWidget->GetUserResponse();
}

#undef LOCTEXT_NAMESPACE
