// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementLevelEditorCommonActionsCustomization.h"
#include "Elements/Actor/ActorElementData.h"
#include "GameFramework/Actor.h"

#include "Elements/Framework/TypedElementSelectionSet.h"

#include "EditorModeManager.h"
#include "Toolkits/IToolkitHost.h"

bool FActorElementLevelEditorCommonActionsCustomization::IsCopyCapable() const
{
	return TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled();
}

bool FActorElementLevelEditorCommonActionsCustomization::CanCopyElements(ITypedElementWorldInterface* InWorldInterface,
	TArrayView<const FTypedElementHandle> InElementHandles)
{
	return TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled();
}

void FActorElementLevelEditorCommonActionsCustomization::CopyElements(ITypedElementWorldInterface* InWorldInterface,
                                                                      TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out)
{
	if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
	{
		FEditorDelegates::OnEditCopyActorsBegin.Broadcast();
	
		FTypedElementCommonActionsCustomization::CopyElements(InWorldInterface, InElementHandles, Out);
	
		FEditorDelegates::OnEditCopyActorsEnd.Broadcast();
	}
}

// We override this function because we want to issue OnEditCutActorsBegin/End instead of OnEditCopyActorsBegin, esp since
//  the cut delegate gets fired from within a transaction, whereas the copy does not.
void FActorElementLevelEditorCommonActionsCustomization::CutElements(ITypedElementWorldInterface* InWorldInterface, 
	TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, 
	const FTypedElementDeletionOptions& InDeletionOptions, FOutputDevice& Out)
{
	if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
	{
		FEditorDelegates::OnEditCutActorsBegin.Broadcast();

		// Note that we qualify this call to be the base class version, not ours, so that we don't call the
		//  copy callback.
		FTypedElementCommonActionsCustomization::CopyElements(InWorldInterface, InElementHandles, Out);
		
		DeleteElements(InWorldInterface, InElementHandles, InWorld, InSelectionSet, InDeletionOptions);

		FEditorDelegates::OnEditCutActorsEnd.Broadcast();
	}
}

bool FActorElementLevelEditorCommonActionsCustomization::DeleteElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
{
	if (const IToolkitHost* ToolkitHostPtr = GetToolkitHost())
	{
		// TODO: Needs to pass in the actors to delete
		if (ToolkitHostPtr->GetEditorModeManager().ProcessEditDelete())
		{
			return true;
		}
	}

	return FTypedElementCommonActionsCustomization::DeleteElements(InWorldInterface, InElementHandles, InWorld, InSelectionSet, InDeletionOptions);
}

void FActorElementLevelEditorCommonActionsCustomization::DuplicateElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements)
{
	if (const IToolkitHost* ToolkitHostPtr = GetToolkitHost())
	{
		// TODO: Needs to pass in the actors to duplicate
		if (ToolkitHostPtr->GetEditorModeManager().ProcessEditDuplicate())
		{
			return;
		}
	}

	FTypedElementCommonActionsCustomization::DuplicateElements(InWorldInterface, InElementHandles, InWorld, InLocationOffset, OutNewElements);
}

TSharedPtr<FWorldElementPasteImporter> FActorElementLevelEditorCommonActionsCustomization::GetPasteImporter(
	ITypedElementWorldInterface* InWorldInterface, const FTypedElementListConstPtr& InSelectedHandles, UWorld* InWorld)
{
	if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
	{
		return Super::GetPasteImporter(InWorldInterface, InSelectedHandles, InWorld);
	}
	return nullptr;
}
