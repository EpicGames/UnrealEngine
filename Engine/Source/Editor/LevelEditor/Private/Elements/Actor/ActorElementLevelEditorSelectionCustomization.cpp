// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementLevelEditorSelectionCustomization.h"
#include "Elements/Actor/ActorElementData.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"

#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/EngineElementsLibrary.h"

#include "Editor.h"
#include "LevelUtils.h"
#include "UnrealEdGlobals.h"
#include "EditorModeManager.h"
#include "Editor/GroupActor.h"
#include "ActorGroupingUtils.h"
#include "Editor/UnrealEdEngine.h"
#include "Toolkits/IToolkitHost.h"
#include "Kismet2/ComponentEditorUtils.h"

DEFINE_LOG_CATEGORY_STATIC(LogActorLevelEditorSelection, Log, All);

bool FActorElementLevelEditorSelectionCustomization::CanSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return CanSelectActorElement(InElementSelectionHandle, InSelectionOptions);
}

bool FActorElementLevelEditorSelectionCustomization::CanDeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return CanDeselectActorElement(InElementSelectionHandle, InSelectionOptions);
}

bool FActorElementLevelEditorSelectionCustomization::SelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return SelectActorElement(InElementSelectionHandle, InSelectionSet, InSelectionOptions);
}

bool FActorElementLevelEditorSelectionCustomization::DeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return DeselectActorElement(InElementSelectionHandle, InSelectionSet, InSelectionOptions);
}

bool FActorElementLevelEditorSelectionCustomization::AllowSelectionModifiers(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet)
{
	// Ctrl or Shift clicking an actor is the same as regular clicking when components are selected
	return !InSelectionSet->HasElementsOfType(NAME_Components);
}

FTypedElementHandle FActorElementLevelEditorSelectionCustomization::GetSelectionElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod)
{
	if (AActor* ConsideredActor = ActorElementDataUtil::GetActorFromHandle(InElementSelectionHandle))
	{
		if (!UChildActorComponent::ExperimentalAllowPerInstanceChildActorProperties())
		{
			while (ConsideredActor->IsChildActor())
			{
				ConsideredActor = ConsideredActor->GetParentActor();
			}
		}
		return UEngineElementsLibrary::AcquireEditorActorElementHandle(ConsideredActor);
	}
	return InElementSelectionHandle;
}

void FActorElementLevelEditorSelectionCustomization::GetNormalizedElements(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementSelectionHandle);
	if (!Actor)
	{
		UE_LOGF(LogActorLevelEditorSelection, Warning, "GetNormalizedElements: Failed to resolve actor from element handle");
		return;
	}

	if (InSelectionSet->HasElementsOfType(NAME_Components))
	{
		// If we have components selected then we will use those rather than the actors
		// The component may still choose to use its owner actor rather than itself
		return;
	}

	FActorElementLevelEditorSelectionCustomization::AppendNormalizedActors(Actor, InSelectionSet, InNormalizationOptions, OutNormalizedElements);
}

namespace LevelEditorSelectionHelpers
{

bool IsActorReachable(const AActor* Actor)
{
	if (!Actor)
	{
		return false;
	}

	// Ensure that neither the level nor the actor is being destroyed or is unreachable
	const EObjectFlags InvalidSelectableFlags = RF_BeginDestroyed;
	bool ReturnValue = true;
	if (!Actor->GetLevel() || Actor->GetLevel()->HasAnyFlags(InvalidSelectableFlags) || (!GIsTransacting && (!IsValidChecked(Actor->GetLevel()) || Actor->GetLevel()->IsUnreachable())))
	{
		UE_LOGF(LogActorLevelEditorSelection, Warning, "SelectActor: %ls (%ls) Flags: (0x%08X)",
			TEXT("The requested operation could not be completed because the level has invalid flags."),
			*Actor->GetActorLabel(),
			static_cast<std::underlying_type_t<EInternalObjectFlags>>(Actor->GetInternalFlags()));
		ReturnValue = false;
	}
	if (Actor->HasAnyFlags(InvalidSelectableFlags) || (!GIsTransacting && (!IsValidChecked(Actor) || Actor->IsUnreachable())))
	{
		UE_LOGF(LogActorLevelEditorSelection, Warning, "SelectActor: %ls (%ls) Flags: (0x%08X)",
			TEXT("The requested operation could not be completed because the actor has invalid flags."),
			*Actor->GetActorLabel(),
			static_cast<std::underlying_type_t<EInternalObjectFlags>>(Actor->GetInternalFlags()));
		ReturnValue = false;
	}

	return ReturnValue;
}

}

bool FActorElementLevelEditorSelectionCustomization::CanSelectActorElement(const TTypedElement<ITypedElementSelectionInterface>& InActorSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) const
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InActorSelectionHandle);
	if (!Actor)
	{
		UE_LOGF(LogActorLevelEditorSelection, Warning, "CanSelectActorElement: Failed to resolve actor from element handle");
		return false;
	}

	// Bail if global selection is locked, or this actor cannot be edited or selected
	if (GEdSelectionLock || !Actor->IsEditable() || !Actor->IsSelectable())
	{
		return false;
	}

	// Bail if the actor is hidden, and we're not allowed to select hidden elements
	if (!InSelectionOptions.AllowHidden() && (Actor->IsHiddenEd() || !FLevelUtils::IsLevelVisible(Actor->GetLevel())))
	{
		return false;
	}

	if (!LevelEditorSelectionHelpers::IsActorReachable(Actor))
	{
		return false;
	}

	// When trying to select a particular actor with a 'selection parent' we'll end 
	// up selecting the actor's selection root instead (see `SelectActorElement()`), 
	// Because of this, we need to also check how selectable the selection root is as well.
	// In the case of Level Instances, an actor in that instance does not belong to 
	// the same level as the instance... so we should be checking the selection root's level
	// instead of the actor's direct level.
	AActor* SelectionRoot = Actor->GetRootSelectionParent();
	ULevel* SelectionLevel = (SelectionRoot != nullptr) ? SelectionRoot->GetLevel() : Actor->GetLevel();

	if (!Actor->IsTemplate() && FLevelUtils::IsLevelLocked(SelectionLevel))
	{
		UE_CLOGF(InSelectionOptions.WarnIfLocked(), LogActorLevelEditorSelection, Warning, "SelectActor: %ls (%ls)", TEXT("The requested operation could not be completed because the level is locked."), *Actor->GetActorLabel());
		return false;
	}

	// If grouping operations are not currently allowed, don't select groups
	AGroupActor* SelectedGroupActor = Cast<AGroupActor>(Actor);
	if (SelectedGroupActor && !UActorGroupingUtils::IsGroupingActive())
	{
		return false;
	}

	if (const IToolkitHost* ToolkitHostPtr = GetToolkitHost())
	{
		// Allow active modes to determine whether the selection is allowed
		return ToolkitHostPtr->GetEditorModeManager().IsSelectionAllowed(Actor, /*bInSelected*/true);
	}

	return true;
}

bool FActorElementLevelEditorSelectionCustomization::CanDeselectActorElement(const TTypedElement<ITypedElementSelectionInterface>& InActorSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) const
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InActorSelectionHandle);
	if (!Actor)
	{
		UE_LOGF(LogActorLevelEditorSelection, Warning, "CanDeselectActorElement: Failed to resolve actor from element handle");
		return true;
	}

	// Bail if global selection is locked
	if (GEdSelectionLock)
	{
		return false;
	}

	if (!LevelEditorSelectionHelpers::IsActorReachable(Actor))
	{
		return false;
	}

	if (const IToolkitHost* ToolkitHostPtr = GetToolkitHost())
	{
		// Allow active modes to determine whether the deselection is allowed
		return ToolkitHostPtr->GetEditorModeManager().IsSelectionAllowed(Actor, /*bInSelected*/false);
	}

	return true;
}

bool FActorElementLevelEditorSelectionCustomization::SelectActorElement(const TTypedElement<ITypedElementSelectionInterface>& InActorSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InActorSelectionHandle);
	if (!Actor)
	{
		UE_LOGF(LogActorLevelEditorSelection, Warning, "SelectActorElement: Failed to resolve actor from element handle");
		return false;
	}

	if (const IToolkitHost* ToolkitHostPtr = GetToolkitHost())
	{
		// Allow active modes to potentially handle the selection
		// TODO: Should this pass through the selection set?
		if (ToolkitHostPtr->GetEditorModeManager().IsSelectionHandled(Actor, /*bInSelected*/true))
		{
			return true;
		}
	}

	// If trying to select an actor, use this actors root selection actor instead (if it has one), unless actor supports being selected with a root and options allow it
	if (!(InSelectionOptions.AllowSubRootSelection() && Actor->SupportsSubRootSelection()))
	{
		if (AActor* RootSelection = Actor->GetRootSelectionParent())
		{
			Actor = RootSelection;
		}
	}

	bool bSelectionChanged = false;

	if (UActorGroupingUtils::IsGroupingActive() && InSelectionOptions.AllowGroups())
	{
		// If this actor is a group, do a group select
		if (AGroupActor* SelectedGroupActor = Cast<AGroupActor>(Actor))
		{
			bSelectionChanged |= SelectActorGroup(SelectedGroupActor, InSelectionSet, InSelectionOptions, /*bForce*/true);
		}
		// Select/Deselect this actor's entire group, starting from the top locked group
		else if (AGroupActor* ActorLockedRootGroup = AGroupActor::GetRootForActor(Actor, true))
		{
			bSelectionChanged |= SelectActorGroup(ActorLockedRootGroup, InSelectionSet, InSelectionOptions, /*bForce*/false);
		}
	}

	// Select the desired actor
	{
		TTypedElement<ITypedElementSelectionInterface> ActorSelectionHandle = InSelectionSet->GetElement<ITypedElementSelectionInterface>(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor));
		if (!ActorSelectionHandle.SelectElement(InSelectionSet, InSelectionOptions))
		{
			return bSelectionChanged;
		}
	}

	UE_LOGF(LogActorLevelEditorSelection, Verbose, "Selected Actor: %ls (%ls)", *Actor->GetActorLabel(), *Actor->GetClass()->GetName());
	
	// Bind the override delegates for the components on the selected actor
	for (UActorComponent* Component : Actor->GetComponents())
	{
		if (!Component)
		{
			continue;
		}

		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			FComponentEditorUtils::BindComponentSelectionOverride(SceneComponent, /*bBind*/true);
		}
	}
		
	// Flush some cached data
	GUnrealEd->PostActorSelectionChanged();

	// A fast path to mark selection rather than reconnecting ALL components for ALL actors that have changed state
	Actor->PushSelectionToProxies();

	return true;
}

bool FActorElementLevelEditorSelectionCustomization::DeselectActorElement(const TTypedElement<ITypedElementSelectionInterface>& InActorSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InActorSelectionHandle);
	if (!Actor)
	{
		UE_LOGF(LogActorLevelEditorSelection, Warning, "DeselectActorElement: Failed to resolve actor from element handle");
		return false;
	}

	if (const IToolkitHost* ToolkitHostPtr = GetToolkitHost())
	{
		// Allow active modes to potentially handle the deselection
		// TODO: Should this pass through the selection set?
		if (ToolkitHostPtr->GetEditorModeManager().IsSelectionHandled(Actor, /*bInSelected*/false))
		{
			return true;
		}
	}

	bool bSelectionChanged = false;

	// If Selection options allows selection of sub root actors and sub actor is selected directly, avoid getting root selection parent
	if (!(InSelectionOptions.AllowSubRootSelection() && Actor->SupportsSubRootSelection() && InActorSelectionHandle.IsElementSelected(InSelectionSet, FTypedElementIsSelectedOptions().SetAllowIndirect(false))))
	{
		// If trying to deselect an actor, use this actors root selection actor instead (if it has one)
		if (AActor* RootSelection = Actor->GetRootSelectionParent())
		{
			Actor = RootSelection;
		}
	}

	if (UActorGroupingUtils::IsGroupingActive() && InSelectionOptions.AllowGroups())
	{
		// If this actor is a group, do a group select
		if (AGroupActor* SelectedGroupActor = Cast<AGroupActor>(Actor))
		{
			bSelectionChanged |= DeselectActorGroup(SelectedGroupActor, InSelectionSet, InSelectionOptions, /*bForce*/true);
		}
		// Select/Deselect this actor's entire group, starting from the top locked group
		else if (AGroupActor* ActorLockedRootGroup = AGroupActor::GetRootForActor(Actor, true))
		{
			bSelectionChanged |= DeselectActorGroup(ActorLockedRootGroup, InSelectionSet, InSelectionOptions, /*bForce*/false);
		}
	}

	// Deselect the desired actor
	{
		TTypedElement<ITypedElementSelectionInterface> ActorSelectionHandle = InSelectionSet->GetElement<ITypedElementSelectionInterface>(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor));
		if (!ActorSelectionHandle.DeselectElement(InSelectionSet, InSelectionOptions))
		{
			return bSelectionChanged;
		}
	}
	
	UE_LOGF(LogActorLevelEditorSelection, Verbose, "Deselected Actor: %ls (%ls)", *Actor->GetActorLabel(), *Actor->GetClass()->GetName());
	
	// Deselect and unbind the override delegates for the components on the selected actor
	{
		FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*InSelectionSet, InSelectionOptions.AllowLegacyNotifications());

		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (!Component)
			{
				continue;
			}

			TTypedElement<ITypedElementSelectionInterface> ComponentSelectionHandle = InSelectionSet->GetElement<ITypedElementSelectionInterface>(UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component));
			ComponentSelectionHandle.DeselectElement(InSelectionSet, InSelectionOptions);

			if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
			{
				FComponentEditorUtils::BindComponentSelectionOverride(SceneComponent, /*bBind*/false);
			}
		}
	}

	// Flush some cached data
	GUnrealEd->PostActorSelectionChanged();

	// A fast path to mark selection rather than reconnecting ALL components for ALL actors that have changed state
	Actor->PushSelectionToProxies();

	return true;
}

bool FActorElementLevelEditorSelectionCustomization::SelectActorGroup(AGroupActor* InGroupActor, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions, const bool bForce)
{
	bool bSelectionChanged = false;

	TTypedElement<ITypedElementSelectionInterface> GroupSelectionHandle = InSelectionSet->GetElement<ITypedElementSelectionInterface>(UEngineElementsLibrary::AcquireEditorActorElementHandle(InGroupActor));

	const FTypedElementSelectionOptions GroupSelectionOptions = FTypedElementSelectionOptions(InSelectionOptions)
		.SetAllowGroups(false);

	// Select all actors within the group (if locked or forced)
	// Skip if the group is already selected, since this logic will have already run
	if ((bForce || InGroupActor->IsLocked()) && !GroupSelectionHandle.IsElementSelected(InSelectionSet, FTypedElementIsSelectedOptions()) && CanSelectActorElement(GroupSelectionHandle, GroupSelectionOptions))
	{
		FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*InSelectionSet, InSelectionOptions.AllowLegacyNotifications());

		TArray<AActor*> GroupActors;
		InGroupActor->GetGroupActors(GroupActors);
		for (AActor* Actor : GroupActors)
		{
			TTypedElement<ITypedElementSelectionInterface> ActorSelectionHandle = InSelectionSet->GetElement<ITypedElementSelectionInterface>(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor));
			if (CanSelectActorElement(ActorSelectionHandle, GroupSelectionOptions))
			{
				bSelectionChanged |= SelectActorElement(ActorSelectionHandle, InSelectionSet, GroupSelectionOptions);
			}
		}
	}

	return bSelectionChanged;
}

bool FActorElementLevelEditorSelectionCustomization::DeselectActorGroup(AGroupActor* InGroupActor, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions, const bool bForce)
{
	bool bSelectionChanged = false;

	TTypedElement<ITypedElementSelectionInterface> GroupSelectionHandle = InSelectionSet->GetElement<ITypedElementSelectionInterface>(UEngineElementsLibrary::AcquireEditorActorElementHandle(InGroupActor));

	const FTypedElementSelectionOptions GroupSelectionOptions = FTypedElementSelectionOptions(InSelectionOptions)
		.SetAllowGroups(false);

	// Deselect all actors within the group (if locked or forced)
	// Skip if the group is already deselected, since this logic will have already run
	if ((bForce || InGroupActor->IsLocked()) && GroupSelectionHandle.IsElementSelected(InSelectionSet, FTypedElementIsSelectedOptions()) && CanDeselectActorElement(GroupSelectionHandle, GroupSelectionOptions))
	{
		FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*InSelectionSet, InSelectionOptions.AllowLegacyNotifications());

		bSelectionChanged |= DeselectActorElement(GroupSelectionHandle, InSelectionSet, GroupSelectionOptions);

		TArray<AActor*> GroupActors;
		InGroupActor->GetGroupActors(GroupActors);
		for (AActor* Actor : GroupActors)
		{
			TTypedElement<ITypedElementSelectionInterface> ActorSelectionHandle = InSelectionSet->GetElement<ITypedElementSelectionInterface>(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor));
			if (CanDeselectActorElement(ActorSelectionHandle, GroupSelectionOptions))
			{
				bSelectionChanged |= DeselectActorElement(ActorSelectionHandle, InSelectionSet, GroupSelectionOptions);
			}
		}
	}

	return bSelectionChanged;
}

void FActorElementLevelEditorSelectionCustomization::AppendNormalizedActors(AActor* InActor, FTypedElementListConstRef InSelectionSet, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements)
{
	if (InNormalizationOptions.FollowAttachment())
	{
		// Ensure that only parent-most actors are included if we were asked to follow attachments
		for (AActor* Parent = InActor->GetAttachParentActor(); Parent; Parent = Parent->GetAttachParentActor())
		{
			FTypedElementHandle ParentHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Parent, /*bAllowCreate*/false);
			if (ParentHandle && InSelectionSet->Contains(ParentHandle))
			{
				return;
			}
		}
	}

	if (InNormalizationOptions.ExpandGroups())
	{
		AGroupActor* ParentGroup = AGroupActor::GetRootForActor(InActor, true, true);
		if (ParentGroup && UActorGroupingUtils::IsGroupingActive())
		{
			// Skip if the group is already in the normalized list, since this logic will have already run
			FTypedElementHandle ParentGroupElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(ParentGroup);
			if (ParentGroupElementHandle && !OutNormalizedElements->Contains(ParentGroupElementHandle))
			{
				ParentGroup->ForEachActorInGroup([InSelectionSet, &InNormalizationOptions, OutNormalizedElements](AActor* InGroupedActor, AGroupActor* InGroupActor)
				{
					// Check that we've not got a parent attachment within the group/selection
					if (InNormalizationOptions.FollowAttachment() && (GroupActorHelpers::ActorHasParentInGroup(InGroupedActor, InGroupActor) || GroupActorHelpers::ActorHasParentInSelection(InGroupedActor, InSelectionSet)))
					{
						return;
					}

					if (FTypedElementHandle GroupedActorElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(InGroupedActor))
					{
						OutNormalizedElements->Add(MoveTemp(GroupedActorElementHandle));
					}
				});

				check(OutNormalizedElements->Contains(ParentGroupElementHandle));
			}
		}
	}

	if (FTypedElementHandle ActorElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(InActor))
	{
		OutNormalizedElements->Add(MoveTemp(ActorElementHandle));
	}
}
