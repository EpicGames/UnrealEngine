// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/TypedElementAssetEditorToolkitHostMixin.h"

class AGroupActor;

class FActorElementLevelEditorSelectionCustomization : public FTypedElementSelectionCustomization, public FTypedElementAssetEditorToolkitHostMixin
{
public:
	virtual bool CanSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool CanDeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool SelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool DeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool AllowSelectionModifiers(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet) override;
	virtual FTypedElementHandle GetSelectionElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod) override;
	virtual void GetNormalizedElements(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements) override;

	bool CanSelectActorElement(const TTypedElement<ITypedElementSelectionInterface>& InActorSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) const;
	bool CanDeselectActorElement(const TTypedElement<ITypedElementSelectionInterface>& InActorSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) const;

	bool SelectActorElement(const TTypedElement<ITypedElementSelectionInterface>& InActorSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);
	bool DeselectActorElement(const TTypedElement<ITypedElementSelectionInterface>& InActorSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);

	bool SelectActorGroup(AGroupActor* InGroupActor, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions, const bool bForce);
	bool DeselectActorGroup(AGroupActor* InGroupActor, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions, const bool bForce);

	static void AppendNormalizedActors(AActor* InActor, FTypedElementListConstRef InSelectionSet, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements);
};
