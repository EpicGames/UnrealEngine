// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementLevelEditorViewportInteractionCustomization.h"
#include "Elements/Actor/ActorElementData.h"
#include "GameFramework/Actor.h"

#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Component/ComponentElementLevelEditorViewportInteractionCustomization.h"

#include "Editor.h"
#include "Engine/Brush.h"
#include "Editor/GroupActor.h"
#include "ActorGroupingUtils.h"
#include "EditorInteractiveGizmoManager.h"
#include "LevelEditorViewport.h"
#include "Misc/ScopeExit.h"

void FActorElementLevelEditorViewportInteractionCustomization::GizmoManipulationStarted(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementWorldHandle);
	if (!Actor)
	{
		UE_LOGF(LogTemp, Warning, "FActorElementLevelEditorViewportInteractionCustomization::GizmoManipulationStarted: Failed to resolve actor from element handle");
		return;
	}

	// Notify that this actor is beginning to move
	GEditor->BroadcastBeginObjectMovement(*Actor);

	// Broadcast Pre Edit change notification, we can't call PreEditChange directly on Actor or ActorComponent from here since it will unregister the components until PostEditChange
	if (FProperty* TransformProperty = FComponentElementLevelEditorViewportInteractionCustomization::GetEditTransformProperty(InWidgetMode))
	{
		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(TransformProperty);
		FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(Actor, PropertyChain);
	}
}

void FActorElementLevelEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementWorldHandle);
	if (!Actor)
	{
		UE_LOGF(LogTemp, Warning, "FActorElementLevelEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate: Failed to resolve actor from element handle");
		return;
	}

	GEditor->NoteActorMovement();

	FTransform ModifiedDeltaTransform = InDeltaTransform;

	{
		FVector AdjustedScale = ModifiedDeltaTransform.GetScale3D();

		// If we are scaling, we may need to change the scaling factor a bit to properly align to the grid
		if (AdjustedScale.IsNearlyZero())
		{
			// We don't scale actors when we only have a very small scale change
			AdjustedScale = FVector::ZeroVector;
		}
		else if (!GEditor->UsePercentageBasedScaling())
		{
			bool bNeedsModifyScale = true;
			if(const FLevelEditorViewportClient* EditorViewportClient = GetLevelEditorViewportClient())
			{
				if (UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(EditorViewportClient->GetModeTools()))
				{
					// The new gizmo system handles snapping and other scale adjustments
					bNeedsModifyScale = false;
				}
			}

			if (bNeedsModifyScale)
			{
				ModifyScale(Actor, InDragAxis, AdjustedScale, Actor->IsA<ABrush>());
			}
		}

		ModifiedDeltaTransform.SetScale3D(AdjustedScale);
	}

	FActorElementEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate(InElementWorldHandle, InWidgetMode, InDragAxis, InInputState, ModifiedDeltaTransform, InPivotLocation);

	// Update the cameras from their locked actor (if any) only if the viewport is real-time enabled
	GetMutableLevelEditorViewportClient()->UpdateLockedActorViewports(Actor, true);
}

void FActorElementLevelEditorViewportInteractionCustomization::GizmoManipulationStopped(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementWorldHandle);
	if (!Actor)
	{
		UE_LOGF(LogTemp, Warning, "FActorElementLevelEditorViewportInteractionCustomization::GizmoManipulationStopped: Failed to resolve actor from element handle");
		return;
	}

	// Balance the unconditional Begin from GizmoManipulationStarted, even for click-only interactions.
	ON_SCOPE_EXIT { GEditor->BroadcastEndObjectMovement(*Actor); };

	if (InManipulationType != ETypedElementViewportInteractionGizmoManipulationType::Drag)
	{
		return;
	}

	// Broadcast Post Edit change notification, we can't call PostEditChangeProperty directly on Actor or ActorComponent from here since it wasn't pair with a proper PreEditChange
	if (FProperty* TransformProperty = FComponentElementLevelEditorViewportInteractionCustomization::GetEditTransformProperty(InWidgetMode))
	{
		FPropertyChangedEvent PropertyChangedEvent(TransformProperty, EPropertyChangeType::ValueSet);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Actor, PropertyChangedEvent);
	}

	if (const ABrush* AsBrush = Cast<ABrush>(Actor))
	{
		if (AsBrush->CanEverAffectBSP())
		{
			GEditor->RebuildAlteredBSP();
		}
	}

	Actor->PostEditMove(true);
}

void FActorElementLevelEditorViewportInteractionCustomization::PostGizmoManipulationStopped(TArrayView<const FTypedElementHandle> InElementHandles, const UE::Widget::EWidgetMode InWidgetMode)
{
	TArray<AActor*> MovedActors = ActorElementDataUtil::GetActorsFromHandles(InElementHandles);
	GEditor->BroadcastActorsMoved(MovedActors);
}

void FActorElementLevelEditorViewportInteractionCustomization::ModifyScale(AActor* InActor, const EAxisList::Type InDragAxis, FVector& ScaleDelta, bool bCheckSmallExtent)
{
	if (InActor->GetRootComponent())
	{
		const FVector CurrentScale = InActor->GetRootComponent()->GetRelativeScale3D();

		const FBox LocalBox = InActor->GetComponentsBoundingBox(true);
		const FVector ScaledExtents = LocalBox.GetExtent() * CurrentScale;
		const FTransform PreDragTransform = GetMutableLevelEditorViewportClient()->GetPreDragActorTransform(InActor);

		FComponentElementLevelEditorViewportInteractionCustomization::ValidateScale(PreDragTransform.GetScale3D(), InDragAxis, CurrentScale, ScaledExtents, ScaleDelta, bCheckSmallExtent);

		if (ScaleDelta.IsNearlyZero())
		{
			ScaleDelta = FVector::ZeroVector;
		}
	}
}
