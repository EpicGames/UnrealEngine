// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolderElementLevelEditorCommonActionsCustomization.h"

#include "HAL/IConsoleManager.h"

bool FActorFolderElementLevelEditorCommonActionsCustomization::IsCopyCapable() const
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders.TypedElements"));
	
	return CVar && CVar->GetBool();
}
