// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementAssetEditorToolkitHostMixin.h"
#include "Elements/Framework/TypedElementCommonActions.h"

class FActorFolderElementLevelEditorCommonActionsCustomization : public FTypedElementCommonActionsCustomization, public FTypedElementAssetEditorToolkitHostMixin
{
	using Super = FTypedElementCommonActionsCustomization;
public:	
	virtual bool IsCopyCapable() const override;
};