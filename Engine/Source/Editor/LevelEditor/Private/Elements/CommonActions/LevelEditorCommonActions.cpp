// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorCommonActions.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "ActorFolders/TedsActorFolderUtils.h"
#include "ActorFolders/TypedElements/ActorFolderTypedElementSupport.h"
#include "DataStorage/Features.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementUtil.h"
#include "GameFramework/Actor.h"

TArray<FTypedElementHandle> ULevelEditorCommonActions::DuplicateNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World,
                                                                                   const FVector& LocationOffset)
{
	TArray<FTypedElementHandle> NewElements = Super::DuplicateNormalizedElements(ElementListPtr, World, LocationOffset);
	
	// Actor duplicate always goes through this path regardless of Cvar, so we always do the fixup
	FixupFolderDataPostDuplicate(NewElements);
	return NewElements;
}

TArray<FTypedElementHandle> ULevelEditorCommonActions::PasteNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World,
	const FTypedElementPasteOptions& PasteOptions, const FString* OptionalInputString)
{
	TArray<FTypedElementHandle> NewElements = Super::PasteNormalizedElements(ElementListPtr, World, PasteOptions, OptionalInputString);
	
	// If TEv1 copy/paste isn't enabled for actors, the fixup happens in UEditorEngine::PasteSelectedActorsFromClipboard directly so we don't
	// need to do duplicate work here
	if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
	{
		FixupFolderDataPostDuplicate(NewElements);
	}

	return NewElements;
}

void ULevelEditorCommonActions::FixupFolderDataPostDuplicate(const TArray<FTypedElementHandle>& DuplicatedElements) const
{
	for (const FTypedElementHandle& ElementHandle : DuplicatedElements)
	{
		if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(ElementHandle))
		{
			UE::Editor::DataStorage::ActorFolders::FixupActorPostDuplicate(Actor);
		}
	}
}