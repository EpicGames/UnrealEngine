// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementCommonActions.h"
#include "Folder.h"

#include "LevelEditorCommonActions.generated.h"

// Specific override of UTypedElementCommonActions for the Level Editor, mainly to handle interop with the new TEv1 support for folders and the 
// legacy copy/paste for actors
UCLASS(Transient, MinimalAPI)
class ULevelEditorCommonActions : public UTypedElementCommonActions
{
	GENERATED_BODY()
public:
	virtual ~ULevelEditorCommonActions() override = default;
	virtual TArray<FTypedElementHandle> DuplicateNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World, const FVector& LocationOffset) override;
	virtual TArray<FTypedElementHandle> PasteNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World, const FTypedElementPasteOptions& PasteOptions, const FString* OptionalInputString = nullptr) override;
protected:
	
	// If a folder and an actor contained in it are both selected and duplicated (or pasted), the duplicated actor will end up in the original folder
	// instead of the duplicated one. So we need some post duplicate/paste fixup to handle this case.
	void FixupFolderDataPostDuplicate(const TArray<FTypedElementHandle>& DuplicatedElements) const;
	
protected:
	TSet<FFolder> DuplicateFoldersCache;
};


