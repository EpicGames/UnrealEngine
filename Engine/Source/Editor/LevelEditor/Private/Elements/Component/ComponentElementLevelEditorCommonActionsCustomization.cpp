// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementLevelEditorCommonActionsCustomization.h"
#include "Elements/Component/ComponentElementData.h"
#include "Components/ActorComponent.h"

#include "EditorModeManager.h"
#include "Toolkits/IToolkitHost.h"

bool FComponentElementLevelEditorCommonActionsCustomization::IsCopyCapable() const
{
	return TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled();
}

bool FComponentElementLevelEditorCommonActionsCustomization::CanCopyElements(ITypedElementWorldInterface* InWorldInterface,
                                                                             TArrayView<const FTypedElementHandle> InElementHandles)
{
	return TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled();
}

void FComponentElementLevelEditorCommonActionsCustomization::CopyElements(ITypedElementWorldInterface* InWorldInterface,
	TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out)
{
	if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
	{
		FTypedElementCommonActionsCustomization::CopyElements(InWorldInterface, InElementHandles, Out);
	}
}

bool FComponentElementLevelEditorCommonActionsCustomization::DeleteElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
{
	if (const IToolkitHost* ToolkitHostPtr = GetToolkitHost())
	{
		// TODO: Needs to pass in the components to delete
		if (ToolkitHostPtr->GetEditorModeManager().ProcessEditDelete())
		{
			return true;
		}
	}

	return FTypedElementCommonActionsCustomization::DeleteElements(InWorldInterface, InElementHandles, InWorld, InSelectionSet, InDeletionOptions);
}

void FComponentElementLevelEditorCommonActionsCustomization::DuplicateElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements)
{
	if (const IToolkitHost* ToolkitHostPtr = GetToolkitHost())
	{
		// TODO: Needs to pass in the components to duplicate
		if (ToolkitHostPtr->GetEditorModeManager().ProcessEditDuplicate())
		{
			return;
		}
	}

	FTypedElementCommonActionsCustomization::DuplicateElements(InWorldInterface, InElementHandles, InWorld, InLocationOffset, OutNewElements);
}

TSharedPtr<FWorldElementPasteImporter> FComponentElementLevelEditorCommonActionsCustomization::GetPasteImporter(
	ITypedElementWorldInterface* InWorldInterface, const FTypedElementListConstPtr& InSelectedHandles, UWorld* InWorld)
{
	if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
	{
		return Super::GetPasteImporter(InWorldInterface,InSelectedHandles,InWorld );
	}
	return nullptr;
}
