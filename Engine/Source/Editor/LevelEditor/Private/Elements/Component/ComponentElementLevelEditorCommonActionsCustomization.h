// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementCommonActions.h"
#include "Elements/Framework/TypedElementAssetEditorToolkitHostMixin.h"

class FComponentElementLevelEditorCommonActionsCustomization : public FTypedElementCommonActionsCustomization, public FTypedElementAssetEditorToolkitHostMixin
{
	using Super = FTypedElementCommonActionsCustomization;
public:
	virtual bool IsCopyCapable() const override;
	virtual bool CanCopyElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles) override;
	virtual void CopyElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out) override;
	virtual bool DeleteElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions) override;
	virtual void DuplicateElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements) override;
	virtual TSharedPtr<FWorldElementPasteImporter> GetPasteImporter(ITypedElementWorldInterface* InWorldInterface,
		const FTypedElementListConstPtr& InSelectedHandles, UWorld* InWorld) override;
	
};
