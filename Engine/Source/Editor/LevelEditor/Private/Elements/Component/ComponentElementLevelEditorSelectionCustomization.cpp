// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementLevelEditorSelectionCustomization.h"
#include "Components/PrimitiveComponent.h"
#include "Elements/Component/ComponentElementData.h"
#include "Components/ActorComponent.h"

#include "Elements/Actor/ActorElementLevelEditorSelectionCustomization.h"
#include "GameFramework/Actor.h"

#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/EngineElementsLibrary.h"

#include "EditorViewportClient.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"

DEFINE_LOG_CATEGORY_STATIC(LogComponentLevelEditorSelection, Log, All);

bool FComponentElementLevelEditorSelectionCustomization::CanSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return CanSelectComponentElement(InElementSelectionHandle, InSelectionOptions);
}

bool FComponentElementLevelEditorSelectionCustomization::CanDeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return CanDeselectComponentElement(InElementSelectionHandle, InSelectionOptions);
}

bool FComponentElementLevelEditorSelectionCustomization::SelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return SelectComponentElement(InElementSelectionHandle, InSelectionSet, InSelectionOptions);
}

bool FComponentElementLevelEditorSelectionCustomization::DeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return DeselectComponentElement(InElementSelectionHandle, InSelectionSet, InSelectionOptions);
}

FTypedElementHandle FComponentElementLevelEditorSelectionCustomization::GetSelectionElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod)
{
	return GetSelectionElementStatic(InElementSelectionHandle, InCurrentSelection, InSelectionMethod);
}

FTypedElementHandle FComponentElementLevelEditorSelectionCustomization::GetSelectionElementStatic(const FTypedElementHandle& InComponentHandle, FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod)
{
	if (const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InComponentHandle))
	{
		const AActor* ConsideredActor = Component->GetOwner();
		const USceneComponent* ConsideredComponent = Cast<USceneComponent>(Component);
		if (ConsideredActor)
		{
			while (ConsideredActor->IsChildActor())
			{
				ConsideredComponent = ConsideredActor->GetParentComponent();
				ConsideredActor = ConsideredActor->GetParentActor();
			}

			// If the component selected is a visualization component, we want to select the non-visualization component it's attached to
			while (ConsideredComponent && ConsideredComponent->IsVisualizationComponent())
			{
				ConsideredComponent = ConsideredComponent->GetAttachParent();
			}

			const FTypedElementHandle ConsideredActorHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(ConsideredActor);

			// We want to process the click on the component only if:
			// 1. The actor clicked is already selected and is the only actor selected
			// 2. The actor selected is blueprintable
			// 3. No components are already selected and the click was a double click
			// 4. OR, a component is already selected and the click was NOT a double click
			const bool bActorAlreadySelectedExclusively = InCurrentSelection->Contains(ConsideredActorHandle) && InCurrentSelection->CountElementsOfType(NAME_Actor) == 1;
			const bool bActorIsBlueprintable = FKismetEditorUtilities::CanCreateBlueprintOfClass(ConsideredActor->GetClass());
			const bool bComponentAlreadySelected = InCurrentSelection->HasElementsOfType(NAME_Components);
			const bool bWasDoubleClick = InSelectionMethod == ETypedElementSelectionMethod::Secondary;
			const bool bWasFromSecondary = InSelectionMethod == ETypedElementSelectionMethod::FromSecondary;

			const bool bSelectComponent = bActorIsBlueprintable && (bWasFromSecondary || (bActorAlreadySelectedExclusively && (bComponentAlreadySelected != bWasDoubleClick)));

			if (bSelectComponent && ConsideredComponent)
			{
				return UEngineElementsLibrary::AcquireEditorComponentElementHandle(ConsideredComponent);
			}
			else
			{
				return ConsideredActorHandle;
			}
		}
	}
	return InComponentHandle;
}

void FComponentElementLevelEditorSelectionCustomization::GetNormalizedElements(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementSelectionHandle);
	if (!Component)
	{
		UE_LOGF(LogComponentLevelEditorSelection, Warning, "GetNormalizedElements: Failed to resolve component from element handle");
		return;
	}

	if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
	{
		if (AActor* ComponentOwner = SceneComponent->GetOwner())
		{
#if DO_CHECK
			{
				FTypedElementHandle ComponentOwnerElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(ComponentOwner, /*bAllowCreate*/false);
				const bool bIsOwnerSelected = ComponentOwnerElementHandle && InSelectionSet->Contains(ComponentOwnerElementHandle);
				ensureMsgf(bIsOwnerSelected, TEXT("Owner(%s) of %s is not selected"), *ComponentOwner->GetFullName(), *Component->GetFullName());
			}
#endif	// DO_CHECK

			if (InNormalizationOptions.FollowAttachment())
			{
				if (ComponentOwner->GetRootComponent() == SceneComponent)
				{
					// If it is a root component, use the parent actor instead
					FActorElementLevelEditorSelectionCustomization::AppendNormalizedActors(ComponentOwner, InSelectionSet, InNormalizationOptions, OutNormalizedElements);
					return;
				}
				else
				{
					// Check to see if any parent is selected
					for (USceneComponent* Parent = SceneComponent->GetAttachParent(); Parent; Parent = Parent->GetAttachParent())
					{
						FTypedElementHandle ParentElementHandle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(Parent, /*bAllowCreate*/false);
						if (ParentElementHandle && InSelectionSet->Contains(ParentElementHandle))
						{
							return;
						}
					}
				}
			}
		}
	}

	OutNormalizedElements->Add(InElementSelectionHandle);
}

bool FComponentElementLevelEditorSelectionCustomization::CanSelectComponentElement(const TTypedElement<ITypedElementSelectionInterface>& InComponentSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) const
{
	const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InComponentSelectionHandle);
	if (!Component)
	{
		UE_LOGF(LogComponentLevelEditorSelection, Warning, "CanSelectComponentElement: Failed to resolve component from element handle");
		return false;
	}

	// Bail if global selection is locked
	return !GEdSelectionLock;
}

bool FComponentElementLevelEditorSelectionCustomization::CanDeselectComponentElement(const TTypedElement<ITypedElementSelectionInterface>& InComponentSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) const
{
	const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InComponentSelectionHandle);
	if (!Component)
	{
		UE_LOGF(LogComponentLevelEditorSelection, Warning, "CanDeselectComponentElement: Failed to resolve component from element handle");
		return true;
	}

	// Bail if global selection is locked
	return !GEdSelectionLock;
}

bool FComponentElementLevelEditorSelectionCustomization::SelectComponentElement(const TTypedElement<ITypedElementSelectionInterface>& InComponentSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InComponentSelectionHandle);
	if (!Component)
	{
		UE_LOGF(LogComponentLevelEditorSelection, Warning, "SelectComponentElement: Failed to resolve component from element handle");
		return false;
	}

	if (!InComponentSelectionHandle.SelectElement(InSelectionSet, InSelectionOptions))
	{
		return false;
	}

	UE_LOGF(LogComponentLevelEditorSelection, Verbose, "Selected Component: %ls (%ls)", *Component->GetPathName(), *Component->GetClass()->GetName());
	
	// Make sure the override delegate is bound properly
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
	{
		FComponentEditorUtils::BindComponentSelectionOverride(SceneComponent, /*bBind*/true);
	}

	if (AActor* ComponentOwner = Component->GetOwner())
	{
		// Selecting a component requires that its owner actor be selected too
		TTypedElement<ITypedElementSelectionInterface> ActorSelectionHandle = InSelectionSet->GetElement<ITypedElementSelectionInterface>(UEngineElementsLibrary::AcquireEditorActorElementHandle(ComponentOwner));
		ActorSelectionHandle.SelectElement(InSelectionSet, InSelectionOptions);

		// Update the selection visualization
		TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents;
		ComponentOwner->GetComponents(PrimitiveComponents);
	
		for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
		{
			PrimitiveComponent->PushSelectionToProxy();
		}
	}

	return true;
}

bool FComponentElementLevelEditorSelectionCustomization::DeselectComponentElement(const TTypedElement<ITypedElementSelectionInterface>& InComponentSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InComponentSelectionHandle);
	if (!Component)
	{
		UE_LOGF(LogComponentLevelEditorSelection, Warning, "DeselectComponentElement: Failed to resolve component from element handle");
		return false;
	}

	if (!InComponentSelectionHandle.DeselectElement(InSelectionSet, InSelectionOptions))
	{
		return false;
	}

	UE_LOGF(LogComponentLevelEditorSelection, Verbose, "Deselected Component: %ls (%ls)", *Component->GetPathName(), *Component->GetClass()->GetName());
	
	// Update the selection visualization
	if (AActor* ComponentOwner = Component->GetOwner())
	{
		TTypedElement<ITypedElementSelectionInterface> ActorSelectionHandle = InSelectionSet->GetElement<ITypedElementSelectionInterface>(UEngineElementsLibrary::AcquireEditorActorElementHandle(ComponentOwner));
		if (!ActorSelectionHandle.IsElementSelected(InSelectionSet, FTypedElementIsSelectedOptions().SetAllowIndirect(true)))
		{
			// Make sure the override delegate is unbound properly
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
			{
				FComponentEditorUtils::BindComponentSelectionOverride(SceneComponent, /*bBind*/false);
			}
		}

		TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents;
		ComponentOwner->GetComponents(PrimitiveComponents);

		for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
		{
			PrimitiveComponent->PushSelectionToProxy();
		}
	}

	return true;
}
