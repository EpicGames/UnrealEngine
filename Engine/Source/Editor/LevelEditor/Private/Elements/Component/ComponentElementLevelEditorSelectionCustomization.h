// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/TypedElementAssetEditorToolkitHostMixin.h"

class FComponentElementLevelEditorSelectionCustomization : public FTypedElementSelectionCustomization, public FTypedElementAssetEditorToolkitHostMixin
{
public:
	virtual bool CanSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool CanDeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool SelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool DeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual FTypedElementHandle GetSelectionElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod) override;
	virtual void GetNormalizedElements(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements) override;

	bool CanSelectComponentElement(const TTypedElement<ITypedElementSelectionInterface>& InComponentSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) const;
	bool CanDeselectComponentElement(const TTypedElement<ITypedElementSelectionInterface>& InComponentSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) const;

	bool SelectComponentElement(const TTypedElement<ITypedElementSelectionInterface>& InComponentSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);
	bool DeselectComponentElement(const TTypedElement<ITypedElementSelectionInterface>& InComponentSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);

	static FTypedElementHandle GetSelectionElementStatic(const FTypedElementHandle& InComponentHandle, FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod);
};
