// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementLevelEditorViewportInteractionCustomization.h"
#include "Elements/Component/ComponentElementData.h"
#include "Components/SceneComponent.h"

#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Actor/ActorElementLevelEditorViewportInteractionCustomization.h"

#include "Editor.h"
#include "EditorInteractiveGizmoManager.h"
#include "LevelEditorViewport.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Misc/ScopeExit.h"

void FComponentElementLevelEditorViewportInteractionCustomization::GizmoManipulationStarted(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementWorldHandle);
	if (!Component)
	{
		UE_LOGF(LogTemp, Warning, "FComponentElementLevelEditorViewportInteractionCustomization::GizmoManipulationStarted: Failed to resolve component from element handle");
		return;
	}

	if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
	{
		// Notify that this component is beginning to move
		GEditor->BroadcastBeginObjectMovement(*SceneComponent);

		// Broadcast Pre Edit change notification, we can't call PreEditChange directly on Actor or ActorComponent from here since it will unregister the components until PostEditChange
		if (FProperty* TransformProperty = FComponentElementLevelEditorViewportInteractionCustomization::GetEditTransformProperty(InWidgetMode))
		{
			FEditPropertyChain PropertyChain;
			PropertyChain.AddHead(TransformProperty);
			FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(SceneComponent, PropertyChain);
		}
	}
}

void FComponentElementLevelEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementWorldHandle);
	if (!Component)
	{
		UE_LOGF(LogTemp, Warning, "FComponentElementLevelEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate: Failed to resolve component from element handle");
		return;
	}

	if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
	{
		FTransform ModifiedDeltaTransform = InDeltaTransform;

		{
			FVector AdjustedDrag = ModifiedDeltaTransform.GetTranslation();
			FRotator AdjustedRot = ModifiedDeltaTransform.Rotator();
			FVector AdjustedScale = ModifiedDeltaTransform.GetScale3D();

			FComponentEditorUtils::AdjustComponentDelta(SceneComponent, AdjustedDrag, AdjustedRot);

			// If we are scaling, we need to change the scaling factor a bit to properly align to grid
			if (AdjustedScale.IsNearlyZero())
			{
				// We don't scale components when we only have a very small scale change
				AdjustedScale = FVector::ZeroVector;
			}
			else if (!GEditor->UsePercentageBasedScaling())
			{
				bool bNeedsModifyScale = true;
				if(const FLevelEditorViewportClient* EditorViewportClient = GetLevelEditorViewportClient())
				{
					if (UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(EditorViewportClient->GetModeTools()))
					{
						// The new gizmo system handles snapping and other scale adjustments
						bNeedsModifyScale = false;
					}
				}

				if (bNeedsModifyScale)
				{
					ModifyScale(SceneComponent, InDragAxis, AdjustedScale);
				}
			}

			ModifiedDeltaTransform.SetTranslation(AdjustedDrag);
			ModifiedDeltaTransform.SetRotation(AdjustedRot.Quaternion());
			ModifiedDeltaTransform.SetScale3D(AdjustedScale);
		}

		FComponentElementEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate(InElementWorldHandle, InWidgetMode, InDragAxis, InInputState, ModifiedDeltaTransform, InPivotLocation);
	}
}

void FComponentElementLevelEditorViewportInteractionCustomization::GizmoManipulationStopped(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementWorldHandle);
	if (!Component)
	{
		UE_LOGF(LogTemp, Warning, "FComponentElementLevelEditorViewportInteractionCustomization::GizmoManipulationStopped: Failed to resolve component from element handle");
		return;
	}

	USceneComponent* SceneComponent = Cast<USceneComponent>(Component);
	if (!SceneComponent)
	{
		return;
	}

	// Balance the unconditional Begin from GizmoManipulationStarted, even for click-only interactions.
	ON_SCOPE_EXIT { GEditor->BroadcastEndObjectMovement(*SceneComponent); };

	if (InManipulationType != ETypedElementViewportInteractionGizmoManipulationType::Drag)
	{
		return;
	}

	// Broadcast Post Edit change notification, we can't call PostEditChangeProperty directly on Actor or ActorComponent from here since it wasn't pair with a proper PreEditChange
	if (FProperty* TransformProperty = FComponentElementLevelEditorViewportInteractionCustomization::GetEditTransformProperty(InWidgetMode))
	{
		FPropertyChangedEvent PropertyChangedEvent(TransformProperty, EPropertyChangeType::ValueSet);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(SceneComponent, PropertyChangedEvent);
	}

	SceneComponent->PostEditComponentMove(true);
}

void FComponentElementLevelEditorViewportInteractionCustomization::ModifyScale(USceneComponent* InComponent, const EAxisList::Type InDragAxis, FVector& ScaleDelta)
{
	AActor* Actor = InComponent->GetOwner();
	const FTransform PreDragTransform = GetMutableLevelEditorViewportClient()->GetPreDragActorTransform(Actor);
	const FBox LocalBox = Actor->GetComponentsBoundingBox(true);
	const FVector ScaledExtents = LocalBox.GetExtent() * InComponent->GetRelativeScale3D();

	FComponentElementLevelEditorViewportInteractionCustomization::ValidateScale(PreDragTransform.GetScale3D(), InDragAxis, InComponent->GetRelativeScale3D(), ScaledExtents, ScaleDelta, /*bCheckSmallExtent*/false);

	if (ScaleDelta.IsNearlyZero())
	{
		ScaleDelta = FVector::ZeroVector;
	}
}

void FComponentElementLevelEditorViewportInteractionCustomization::ValidateScale(const FVector& InOriginalPreDragScale, const EAxisList::Type InDragAxis, const FVector& InCurrentScale, const FVector& InBoxExtent, FVector& InOutScaleDelta, bool bInCheckSmallExtent)
{
	// If original scale has a zeroed axis, we apply the scale as-is
	if (FMath::IsNearlyZero(InOriginalPreDragScale.X) || FMath::IsNearlyZero(InOriginalPreDragScale.Y) || FMath::IsNearlyZero(InOriginalPreDragScale.Z))
	{
		return;
	}

	static const float MIN_ACTOR_BOUNDS_EXTENT = 1.0f;

	/** Convert the active Dragging Axis to per-axis flags */
	auto CheckActiveAxes = [InDragAxis](bool bActiveAxes[3])
	{
		bActiveAxes[0] = bActiveAxes[1] = bActiveAxes[2] = false;
		switch (InDragAxis)
		{
		default:
		case EAxisList::None:
			break;
		case EAxisList::X:
			bActiveAxes[0] = true;
			break;
		case EAxisList::Y:
			bActiveAxes[1] = true;
			break;
		case EAxisList::Z:
			bActiveAxes[2] = true;
			break;
		case EAxisList::XYZ:
		case EAxisList::All:
		case EAxisList::Screen:
			bActiveAxes[0] = bActiveAxes[1] = bActiveAxes[2] = true;
			break;
		case EAxisList::XY:
			bActiveAxes[0] = bActiveAxes[1] = true;
			break;
		case EAxisList::XZ:
			bActiveAxes[0] = bActiveAxes[2] = true;
			break;
		case EAxisList::YZ:
			bActiveAxes[1] = bActiveAxes[2] = true;
			break;
		}
	};

	/** Check scale criteria to see if this is allowed, returns modified absolute scale*/
	auto CheckScaleValue = [](FVector::FReal ScaleDeltaToCheck, FVector::FReal CurrentScaleFactor, FVector::FReal CurrentExtent, bool bCheckSmallExtent, bool bSnap) -> FVector::FReal
	{
		FVector::FReal AbsoluteScaleValue = ScaleDeltaToCheck + CurrentScaleFactor;
		if (bSnap)
		{
			AbsoluteScaleValue = FMath::GridSnap(AbsoluteScaleValue, GEditor->GetScaleGridSize());
		}
		// In some situations CurrentExtent can be 0 (eg: when scaling a plane in Z), this causes a divide by 0 that we need to avoid.
		if (FMath::Abs(CurrentExtent) < KINDA_SMALL_NUMBER) {
			return AbsoluteScaleValue;
		}
		FVector::FReal UnscaledExtent = CurrentExtent / CurrentScaleFactor;
		FVector::FReal ScaledExtent = UnscaledExtent * AbsoluteScaleValue;

		if ((FMath::Square(ScaledExtent)) > BIG_NUMBER)	// cant get too big...
		{
			return CurrentScaleFactor;
		}
		else if (bCheckSmallExtent &&
			(FMath::Abs(ScaledExtent) < MIN_ACTOR_BOUNDS_EXTENT * 0.5f ||		// ...or too small (apply sign in this case)...
				(CurrentScaleFactor < 0.0f) != (AbsoluteScaleValue < 0.0f)))	// ...also cant cross the zero boundary
		{
			return ((MIN_ACTOR_BOUNDS_EXTENT * 0.5f) / UnscaledExtent) * (CurrentScaleFactor < 0.0f ? -1.0f : 1.0f);
		}

		return AbsoluteScaleValue;
	};

	/**
	* If the "PreserveNonUniformScale" setting is enabled, this function will appropriately re-scale the scale delta so that
	* proportions are preserved also when snapping.
	* This function will modify the scale delta sign so that scaling is apply in the good direction when using multiple axis in same time.
	* The function will not transform the scale delta in the case the scale delta is not uniform
	* @param    InOriginalPreDragScale		The object's original scale
	* @param	bActiveAxes					The axes that are active when scaling interactively.
	* @param	InOutScaleDelta				The scale delta we are potentially transforming.
	* @return true if the axes should be snapped individually, according to the snap setting (i.e. this function had no effect)
	*/
	auto ApplyScalingOptions = [](const FVector& InOriginalPreDragScale, const FVector& InCurrentScale, const bool bActiveAxes[3], FVector& InOutScaleDelta)
	{
		int ActiveAxisCount = 0;
		bool CurrentValueSameSign = true;
		bool FirstSignPositive = true;
		FVector::FReal MaxComponentSum = -1.0f;
		int32 MaxAxisIndex = -1;
		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
		bool SnapScaleAfter = ViewportSettings->SnapScaleEnabled;

		//Found the number of active axis
		//Found if we have to swap some sign
		for (int Axis = 0; Axis < 3; ++Axis)
		{
			if (bActiveAxes[Axis])
			{
				bool CurrentValueIsZero = FMath::IsNearlyZero(InOriginalPreDragScale[Axis], (FVector::FReal)SMALL_NUMBER);
				//when the current value is zero we assume it is positive
				bool IsCurrentValueSignPositive = CurrentValueIsZero ? true : InOriginalPreDragScale[Axis] > 0.0f;
				if (ActiveAxisCount == 0)
				{
					//Set the first value when we find the first active axis
					FirstSignPositive = IsCurrentValueSignPositive;
				}
				else
				{
					if (FirstSignPositive != IsCurrentValueSignPositive)
					{
						CurrentValueSameSign = false;
					}
				}
				ActiveAxisCount++;
			}
		}

		//If we scale more then one axis and
		//we have to swap some sign
		if (ActiveAxisCount > 1 && !CurrentValueSameSign)
		{
			//Change the scale delta to reflect the sign of the value
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				if (bActiveAxes[Axis])
				{
					bool CurrentValueIsZero = FMath::IsNearlyZero(InOriginalPreDragScale[Axis], (FVector::FReal)SMALL_NUMBER);
					//when the current value is zero we assume it is positive
					bool IsCurrentValueSignPositive = CurrentValueIsZero ? true : InOriginalPreDragScale[Axis] > 0.0f;
					InOutScaleDelta[Axis] = IsCurrentValueSignPositive ? InOutScaleDelta[Axis] : -(InOutScaleDelta[Axis]);
				}
			}
		}

		if (ViewportSettings->PreserveNonUniformScale)
		{
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				if (bActiveAxes[Axis])
				{
					const FVector::FReal AbsScale = FMath::Abs(InOutScaleDelta[Axis] + InOriginalPreDragScale[Axis]);
					if (AbsScale > MaxComponentSum)
					{
						MaxAxisIndex = Axis;
						MaxComponentSum = AbsScale;
					}
				}
			}

			check(MaxAxisIndex != -1);

			FVector::FReal AbsoluteScaleValue = InCurrentScale[MaxAxisIndex] + InOutScaleDelta[MaxAxisIndex];
			if (ViewportSettings->SnapScaleEnabled)
			{
				AbsoluteScaleValue = FMath::GridSnap(AbsoluteScaleValue, (FVector::FReal)GEditor->GetScaleGridSize());
				SnapScaleAfter = false;
			}

			FVector::FReal ScaleRatioMax = AbsoluteScaleValue / InOriginalPreDragScale[MaxAxisIndex];
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				if (bActiveAxes[Axis])
				{
					InOutScaleDelta[Axis] = (InOriginalPreDragScale[Axis] * ScaleRatioMax) - InCurrentScale[Axis];
				}
			}
		}

		return SnapScaleAfter;
	};

	/** Helper function for ModifyScale - Check scale criteria to see if this is allowed */

	// get the axes that are active in this operation
	bool bActiveAxes[3];
	CheckActiveAxes(bActiveAxes);

	//When scaling with more then one active axis, We must make sure we apply the correct delta sign to each delta scale axis
	//We also want to support the PreserveNonUniformScale option
	bool bSnapAxes = ApplyScalingOptions(InOriginalPreDragScale, InCurrentScale, bActiveAxes, InOutScaleDelta);

	// check each axis
	for (int Axis = 0; Axis < 3; ++Axis)
	{
		if (bActiveAxes[Axis])
		{
			FVector::FReal ModifiedScaleAbsolute = CheckScaleValue(InOutScaleDelta[Axis], InCurrentScale[Axis], InBoxExtent[Axis], bInCheckSmallExtent, bSnapAxes);
			InOutScaleDelta[Axis] = ModifiedScaleAbsolute - InCurrentScale[Axis];
		}
		else
		{
			InOutScaleDelta[Axis] = 0.0f;
		}
	}
}

FProperty* FComponentElementLevelEditorViewportInteractionCustomization::GetEditTransformProperty(const UE::Widget::EWidgetMode InWidgetMode)
{
	switch (InWidgetMode)
	{
	case UE::Widget::WM_Translate:
	case UE::Widget::WM_TranslateRotateZ:
	case UE::Widget::WM_2D:
		return FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName());

	case UE::Widget::WM_Rotate:
		return FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName());

	case UE::Widget::WM_Scale:
		return FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName());

	default:
		break;
	}

	return nullptr;
}
