// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementLevelEditorSelectionCustomization.h"
#include "Elements/Component/ComponentElementLevelEditorSelectionCustomization.h"
#include "Elements/SMInstance/SMInstanceElementData.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementInterfaceCustomization.h"
#include "Elements/Interfaces/TypedElementHierarchyInterface.h"

#include "LevelUtils.h"

DEFINE_LOG_CATEGORY_STATIC(LogSMInstanceLevelEditorSelection, Log, All);

bool FSMInstanceElementLevelEditorSelectionCustomization::CanSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandleChecked(InElementSelectionHandle);
	if (!SMInstance)
	{
		return false;
	}

	AActor* Owner = SMInstance.GetISMComponent()->GetOwner();
	AActor* SelectionRoot = Owner->GetRootSelectionParent();
	ULevel* SelectionLevel = (SelectionRoot != nullptr) ? SelectionRoot->GetLevel() : Owner->GetLevel();
	if (!Owner->IsTemplate() && FLevelUtils::IsLevelLocked(SelectionLevel))
	{
		return false;
	}
	
	// Bail if global selection is locked
	return !GEdSelectionLock;
}

bool FSMInstanceElementLevelEditorSelectionCustomization::CanDeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandleChecked(InElementSelectionHandle);
	if (!SMInstance)
	{
		return false;
	}

	// Bail if global selection is locked
	return !GEdSelectionLock;
}

bool FSMInstanceElementLevelEditorSelectionCustomization::SelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandleChecked(InElementSelectionHandle);
	if (!SMInstance)
	{
		return false;
	}

	if (!InElementSelectionHandle.SelectElement(InSelectionSet, InSelectionOptions))
	{
		return false;
	}

	UE_LOGF(LogSMInstanceLevelEditorSelection, Verbose, "Selected SMInstance: %ls (%ls), Index %d", *SMInstance.GetISMComponent()->GetPathName(), *SMInstance.GetISMComponent()->GetClass()->GetName(), SMInstance.GetISMInstanceIndex());
	
	return true;
}

bool FSMInstanceElementLevelEditorSelectionCustomization::DeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandleChecked(InElementSelectionHandle);
	if (!SMInstance)
	{
		return false;
	}

	if (!InElementSelectionHandle.DeselectElement(InSelectionSet, InSelectionOptions))
	{
		return false;
	}

	UE_LOGF(LogSMInstanceLevelEditorSelection, Verbose, "Deselected SMInstance: %ls (%ls), Index %d", *SMInstance.GetISMComponent()->GetPathName(), *SMInstance.GetISMComponent()->GetClass()->GetName(), SMInstance.GetISMInstanceIndex());

	return true;
}

FTypedElementHandle FSMInstanceElementLevelEditorSelectionCustomization::GetSelectionElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandleChecked(InElementSelectionHandle);
	if (!SMInstance)
	{
		return InElementSelectionHandle;
	}

	if (const UInstancedStaticMeshComponent* Component = SMInstance.GetISMComponent())
	{
		const FTypedElementHandle OwningComponentHandle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component);

		const bool bWasDoubleClick = InSelectionMethod == ETypedElementSelectionMethod::Secondary;
		const bool bComponentAlreadySelected = InCurrentSelection->Contains(OwningComponentHandle);
		const bool bIsISMAlreadySelected = InCurrentSelection->Contains(InElementSelectionHandle);

		// Do we need to go down or up the hierarchy from a double click?
		if (bWasDoubleClick && !bCameFromSecondary)
		{
			// down
			if (bComponentAlreadySelected)
			{
				return InElementSelectionHandle;
			}
			// up
			else if (InCurrentSelection->HasElementsOfType(InElementSelectionHandle.GetId().GetTypeId()))
			{
				bCameFromSecondary = true;
				return FComponentElementLevelEditorSelectionCustomization::GetSelectionElementStatic(OwningComponentHandle, InCurrentSelection, ETypedElementSelectionMethod::FromSecondary);
			}
		}

		// If the active selection is within the same component
		else if (HasSMInstanceSelectionInSameComponent(InElementSelectionHandle, InCurrentSelection))
		{
			return InElementSelectionHandle;
		}

		// Otherwise forward selection to the component to component handle
		if (InSelectionMethod == ETypedElementSelectionMethod::Secondary)
		{
			bCameFromSecondary = false;
		}
		return FComponentElementLevelEditorSelectionCustomization::GetSelectionElementStatic(OwningComponentHandle, InCurrentSelection, InSelectionMethod);
	}

	return InElementSelectionHandle;
}

bool FSMInstanceElementLevelEditorSelectionCustomization::AllowSelectionModifiers(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet)
{
	return HasSMInstanceSelectionInSameComponent(InElementSelectionHandle, InSelectionSet);
}

bool FSMInstanceElementLevelEditorSelectionCustomization::HasSMInstanceSelectionInSameComponent(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementSelectionHandle);
	if (!SMInstance)
	{
		return false;
	}

	bool bHasSMInstanceSelectedWithSameComponent = false;
	if (InSelectionSet->HasElementsOfType(FSMInstanceElementData::StaticTypeId()))
	{
		InSelectionSet->ForEachElementHandle([InElementSelectionHandle, SMInstance, &bHasSMInstanceSelectedWithSameComponent](const FTypedElementHandle& InHandle)
		{
			if (InHandle.GetId().GetTypeId() == FSMInstanceElementData::StaticTypeId())
			{
				const FSMInstanceManager SelectedSMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InHandle);
				if (SelectedSMInstance && (SelectedSMInstance.GetISMComponent() == SMInstance.GetISMComponent()))
				{
					bHasSMInstanceSelectedWithSameComponent = true;
				}
				return false;
			}

			return true;
		});
	}

	return bHasSMInstanceSelectedWithSameComponent;
}
