// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/TypedElementAssetEditorToolkitHostMixin.h"

class FSMInstanceElementLevelEditorSelectionCustomization : public FTypedElementSelectionCustomization, public FTypedElementAssetEditorToolkitHostMixin
{
public:
	virtual bool CanSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool CanDeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool SelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool DeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual FTypedElementHandle GetSelectionElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod) override;
	virtual bool AllowSelectionModifiers(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet) override;
private:
	static bool HasSMInstanceSelectionInSameComponent(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet);
	bool bCameFromSecondary = false;
};
