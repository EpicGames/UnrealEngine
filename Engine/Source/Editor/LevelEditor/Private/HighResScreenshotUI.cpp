// Copyright Epic Games, Inc. All Rights Reserved.

#include "HighresScreenshotUI.h"

#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/PlatformFileManager.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "Engine/World.h"
#include "Kismet/KismetSystemLibrary.h"
#endif

TWeakPtr<class SWindow> SHighResScreenshotDialog::CurrentWindow = NULL;
TWeakPtr<class SHighResScreenshotDialog> SHighResScreenshotDialog::CurrentDialog = NULL;
bool SHighResScreenshotDialog::bMaskVisualizationWasEnabled = false;

SHighResScreenshotDialog::SHighResScreenshotDialog()
: Config(GetHighResScreenshotConfig())
{
}

void SHighResScreenshotDialog::Construct( const FArguments& InArgs )
{
	const FMargin GridPadding(6.f, 3.f);
	constexpr float WindowBorderPadding = 16.f;

	this->ChildSlot
	.Padding(0.f)
	[
		SNew(SBorder)
		.Padding(0.f)
		.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		[
		    SNew(SVerticalBox)
		    + SVerticalBox::Slot()
		    .FillHeight(1.0f)
			.Padding(0.f)
		    [
		    	SNew(SScrollBox)
		    	+ SScrollBox::Slot()
		    	[
		    		SNew(SVerticalBox)
		    	    +SVerticalBox::Slot()
		    	    .AutoHeight()
		    	    [
		    	    	SNew(SHorizontalBox)
		    	    	+SHorizontalBox::Slot()
		    	    	.AutoWidth()
		    	    	.Padding(6.f)
		    	    	[
		    	    		SAssignNew(CaptureRegionButton, SButton)
		    	    		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		    	    		.IsEnabled(this, &SHighResScreenshotDialog::IsCaptureRegionEditingAvailable)
		    	    		.Visibility(this, &SHighResScreenshotDialog::GetSpecifyCaptureRegionVisibility)
		    	    		.ToolTipText(NSLOCTEXT("HighResScreenshot", "ScreenshotSpecifyCaptureRectangleTooltip", "Specify the region which will be captured by the screenshot"))
		    	    		.OnClicked(this, &SHighResScreenshotDialog::OnSelectCaptureRegionClicked)
		    	    		.ContentPadding(4.f)
		    	    		[
		    	    			SNew(SImage)
		    	    			.ColorAndOpacity(FSlateColor::UseForeground())
		    	    			.Image(FAppStyle::Get().GetBrush("Icons.Crop"))
		    	    		]
		    	    	]
		    	    	+SHorizontalBox::Slot()
		    	    	.AutoWidth()
		    	    	.Padding(6.f)
		    	    	[
		    	    		SNew( SButton )
		    	    		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		    	    		.Visibility(this, &SHighResScreenshotDialog::GetCaptureRegionControlsVisibility)
		    	    		.ToolTipText( NSLOCTEXT("HighResScreenshot", "ScreenshotAcceptCaptureRegionTooltip", "Accept any changes made to the capture region") )
		    	    		.OnClicked( this, &SHighResScreenshotDialog::OnSelectCaptureAcceptRegionClicked )
		    	    		.ContentPadding(4.f)
		    	    		[
		    	    			SNew(SImage)
		    	    			.ColorAndOpacity(FSlateColor::UseForeground())
		    	    			.Image(FAppStyle::Get().GetBrush("Icons.Check"))
		    	    		]
		    	    	]
		    	    	+SHorizontalBox::Slot()
		    	    	.AutoWidth()
		    	    	.Padding(6.f)
		    	    	[
		    	    		SNew( SButton )
		    	    		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		    	    		.ToolTipText( NSLOCTEXT("HighResScreenshot", "ScreenshotDiscardCaptureRegionTooltip", "Discard any changes made to the capture region") )
		    	    		.Visibility(this, &SHighResScreenshotDialog::GetCaptureRegionControlsVisibility)
		    	    		.OnClicked( this, &SHighResScreenshotDialog::OnSelectCaptureCancelRegionClicked )
		    	    		.ContentPadding(4.f)
		    	    		[
		    	    			SNew(SImage)
		    	    			.ColorAndOpacity(FSlateColor::UseForeground())
		    	    			.Image(FAppStyle::Get().GetBrush("Icons.X"))
		    	    		]
		    	    	]
		    	    	+SHorizontalBox::Slot()
		    	    	.AutoWidth()
		    	    	.Padding(6.f)
		    	    	[
		    	    		SNew( SButton )
		    	    		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		    	    		.ToolTipText( NSLOCTEXT("HighResScreenshot", "ScreenshotFullViewportCaptureRegionTooltip", "Set the capture rectangle to the whole viewport") )
		    	    		.Visibility(this, &SHighResScreenshotDialog::GetCaptureRegionControlsVisibility)
		    	    		.OnClicked( this, &SHighResScreenshotDialog::OnSetFullViewportCaptureRegionClicked )
		    	    		.ContentPadding(4.f)
		    	    		[
		    	    			SNew(SImage)
		    	    			.ColorAndOpacity(FSlateColor::UseForeground())
		    	    			.Image(FAppStyle::Get().GetBrush("Icons.Fullscreen"))
		    	    		]
		    	    	]
		    	    ]
		    	    
		    	    +SVerticalBox::Slot()
		    	    .AutoHeight()
		    	    [
		    	    	SNew(SSeparator)
		    	    	.Thickness(1.f)
		    	    	.SeparatorImage(FAppStyle::Get().GetBrush("Brushes.Background"))
		    	    ]
		    	    
		    	    +SVerticalBox::Slot()
		    		.Padding(6.0)
		    	    .AutoHeight()
		    	    [
		    	    	SNew(SWarningOrErrorBox)
		    	    	.Padding(FMargin(16.f, 13.f, 16.f, 13.f))
		    	    	.Message(NSLOCTEXT("HighResScreenshot", "CaptureWarningText", "Large multipliers may cause the graphics driver to crash.  Please try using a lower multiplier."))
		    	    	.Visibility(this, &SHighResScreenshotDialog::GetCaptureWarningVisibility)
		    	    ]
		    	    
		    	    +SVerticalBox::Slot()
		    	    .Padding(6.0)
		    	    .HAlign(HAlign_Fill)
		    	    [
		    	    	// Row/Column
		    	    	SNew(SGridPanel)
		    	    	.FillColumn(1, 1.0f)
		    	    
		    	    	+SGridPanel::Slot(0, 0)
		    	    	.Padding(GridPadding)
		    	    	.VAlign(VAlign_Center)
		    	    	[
		    	    		SNew( STextBlock )
		    	    		.Text( NSLOCTEXT("HighResScreenshot", "ScreenshotSizeMultiplier", "Screenshot Size Multiplier") )
		    	    	]
		    	    	+SGridPanel::Slot(0, 1)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SNew( STextBlock )
		    	    		.Text( NSLOCTEXT("HighResScreenshot", "UseDateTimeAsImageName", "Use Date & Timestamp as Image name") )
		    	    	]
		    	    	+ SGridPanel::Slot(0, 2)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SNew(STextBlock)
		    	    		.Text(NSLOCTEXT("HighResScreenshot", "IncludeBufferVisTargets", "Include Buffer Visualization Targets"))
		    	    	]
		    	    	+ SGridPanel::Slot(0, 3)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SAssignNew(HDRLabel, STextBlock)
		    	    		.Text(NSLOCTEXT("HighResScreenshot", "CaptureHDR", "Write HDR format visualization targets"))
		    	    	]
		    	    	+SGridPanel::Slot(0, 4)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SAssignNew(Force128BitRenderingLabel, STextBlock)
		    	    		.Text(NSLOCTEXT("HighResScreenshot", "Force128BitPipeline", "Force 128-bit buffers for rendering pipeline"))
		    	    	]
		    	    	+SGridPanel::Slot(0, 5)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SNew( STextBlock )
		    	    		.Text( NSLOCTEXT("HighResScreenshot", "UseCustomDepth", "Use custom depth as mask") )
		    	    	]
		    	    
		    	    	+SGridPanel::Slot(1, 0)
		    	    	.Padding(GridPadding)
		    	    	.HAlign(HAlign_Fill)
		    	    	[
		    	    		SNew( SSpinBox<float> )
		    	    		.MinValue(FHighResScreenshotConfig::MinResolutionMultipler)
		    	    		.MaxValue(FHighResScreenshotConfig::MaxResolutionMultipler)
		    	    		.Delta(1.0f)
		    	    		.Value(this, &SHighResScreenshotDialog::GetResolutionMultiplierSlider)
		    	    		.OnValueChanged(this, &SHighResScreenshotDialog::OnResolutionMultiplierSliderChanged)
		    	    	]
		    	    	+ SGridPanel::Slot(1, 1)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SNew(SCheckBox)
		    	    		.OnCheckStateChanged(this, &SHighResScreenshotDialog::OnDateTimeBasedNamingEnabledChanged)
		    	    		.IsChecked(this, &SHighResScreenshotDialog::GetDateTimeBasedNamingEnabled)
		    	    	]
		    	    	+SGridPanel::Slot(1, 2)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SNew( SCheckBox )
		    	    		.OnCheckStateChanged(this, &SHighResScreenshotDialog::OnBufferVisualizationDumpEnabledChanged)
		    	    		.IsChecked(this, &SHighResScreenshotDialog::GetBufferVisualizationDumpEnabled)
		    	    	]
		    	    	+ SGridPanel::Slot(1, 3)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SAssignNew(HDRCheckBox, SCheckBox)
		    	    		.OnCheckStateChanged(this, &SHighResScreenshotDialog::OnHDREnabledChanged)
		    	    		.IsChecked(this, &SHighResScreenshotDialog::GetHDRCheckboxUIState)
		    	    	]
		    	    	+ SGridPanel::Slot(1, 4)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SAssignNew(Force128BitRenderingCheckBox, SCheckBox)
		    	    		.OnCheckStateChanged(this, &SHighResScreenshotDialog::OnForce128BitRenderingChanged)
		    	    		.IsChecked(this, &SHighResScreenshotDialog::GetForce128BitRenderingCheckboxUIState)
		    	    	]
		    	    	+SGridPanel::Slot(1, 5)
		    	    	.Padding(GridPadding)
		    	    	[
		    	    		SNew( SCheckBox )
		    	    		.OnCheckStateChanged(this, &SHighResScreenshotDialog::OnMaskEnabledChanged)
		    	    		.IsChecked(this, &SHighResScreenshotDialog::GetMaskEnabled)
		    	    	]
		    	    ]
		    	]
		    ]
		    + SVerticalBox::Slot()
		    .AutoHeight()
			.Padding(WindowBorderPadding)
		    [
		    	SNew(SHorizontalBox)
		    	+SHorizontalBox::Slot()
		    	.VAlign(VAlign_Bottom)
		    	.HAlign(HAlign_Left)
		    	.AutoWidth()
		    	.Padding(FMargin(0.f, 0.f, 24.f, 0.f))
		    	[
		    		SNew(SButton)
		    		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
		    		.ToolTipText(NSLOCTEXT("HighResScreenshot", "OpenScreenshotFolderToolTip", "Open the screenshot folder in the file explorer") )
		    		.OnClicked(this, &SHighResScreenshotDialog::OnOpenScreenshotFolder)
		    		.Text(NSLOCTEXT("HighResScreenshot", "OpenScreenshotFolder", "Open Screenshot Folder"))
		    	]
		    	
		    	+ SHorizontalBox::Slot()
		    	.FillWidth(1.f)
		    	[
		    		SNew(SSpacer)
		    	]
		    	
		    	+SHorizontalBox::Slot()
		    	.VAlign(VAlign_Bottom)
		    	.HAlign(HAlign_Right)
		    	.AutoWidth()
		    	.Padding(FMargin(24.f, 0.f, 0.f, 0.f))
		    	[
		    		SNew(SButton)
		    		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
		    		.ToolTipText(NSLOCTEXT("HighResScreenshot", "ScreenshotCaptureToolTip", "Take a screenshot"))
		    		.OnClicked(this, &SHighResScreenshotDialog::OnCaptureClicked)
		    		.Text(NSLOCTEXT("HighResScreenshot", "CaptureCommit", "Capture"))
		    	]
		    ]			
		]
	];

	SetHDRUIEnableState(Config.bDumpBufferVisualizationTargets);
	SetForce128BitRenderingState(Config.bDumpBufferVisualizationTargets);
	bCaptureRegionControlsVisible = false;
}

void SHighResScreenshotDialog::WindowClosedHandler(const TSharedRef<SWindow>& InWindow)
{
	FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();

	ResetViewport();
	ResetFrameBuffer();

	// Cleanup the config after each usage as it is a static and we don't want it to keep pointers or settings around between runs.
	Config.bDisplayCaptureRegion = false;
	Config.ChangeViewport(TWeakPtr<FSceneViewport>());
	CurrentWindow.Reset();
	CurrentDialog.Reset();
}

void SHighResScreenshotDialog::ResetViewport()
{
	FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();
	auto CurrentDialogPinned = CurrentDialog.Pin();
	auto ConfigViewport = Config.TargetViewport.Pin();

	if (CurrentDialogPinned.IsValid())
	{
		// Deactivate capture region widget from old viewport
		TSharedPtr<SCaptureRegionWidget> CaptureRegionWidget = CurrentDialogPinned->GetCaptureRegionWidget();
		if (CaptureRegionWidget.IsValid())
		{
			CaptureRegionWidget->Deactivate(false);
		}

		if (ConfigViewport.IsValid())
		{
			// Restore mask visualization state from before window was opened
			if (ConfigViewport->GetClient() &&
				ConfigViewport->GetClient()->GetEngineShowFlags())
			{
				ConfigViewport->GetClient()->GetEngineShowFlags()->SetHighResScreenshotMask(bMaskVisualizationWasEnabled);
			}
		}
	}
}

void SHighResScreenshotDialog::ResetFrameBuffer()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();
	if (Config.TargetViewport.IsValid())
	{
		FViewportClient* Client = Config.TargetViewport.Pin()->GetClient();
		if (Client)
		{
			UKismetSystemLibrary::ExecuteConsoleCommand(Client->GetWorld(), TEXT("r.ResetRenderTargetsExtent"), nullptr);
		}
	}
#endif
}

TWeakPtr<class SWindow> SHighResScreenshotDialog::OpenDialog(const TSharedPtr<FSceneViewport>& InViewport, TSharedPtr<SCaptureRegionWidget> InCaptureRegionWidget)
{
	FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();
	auto ConfigViewport = Config.TargetViewport.Pin();
	auto CurrentWindowPinned = CurrentWindow.Pin();

	bool bInitializeDialog = false;

	if (CurrentWindowPinned.IsValid())
	{
		// Dialog window is already open - if it is being pointed at a new viewport, reset the old one
		if (InViewport != ConfigViewport)
		{
			ResetViewport();
			bInitializeDialog = true;
		}
	}
	else
	{
		// No dialog window currently open - need to create one
		TSharedRef<SHighResScreenshotDialog> Dialog = SNew(SHighResScreenshotDialog);
		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title( NSLOCTEXT("HighResScreenshot", "HighResolutionScreenshot", "High Resolution Screenshot") )
			.ClientSize(FVector2D(480, 350))
			.MinHeight(250.f)
			.MinWidth(400.f)
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			.FocusWhenFirstShown(true)
			[
				Dialog
			];

		Window->SetOnWindowClosed(FOnWindowClosed::CreateStatic(&WindowClosedHandler));
		Dialog->SetWindow(Window);


		TSharedPtr<SWindow> ParentWindow = FGlobalTabmanager::Get()->GetRootWindow();

		if (ParentWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(Window, ParentWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(Window);
		}

		CurrentWindow = TWeakPtr<SWindow>(Window);
		CurrentDialog = TWeakPtr<SHighResScreenshotDialog>(Dialog);

		Config.bDisplayCaptureRegion = true;

		bInitializeDialog = true;
	}

	if (bInitializeDialog)
	{
		auto CurrentDialogPinned = CurrentDialog.Pin();
		if (CurrentDialogPinned.IsValid())
		{
			CurrentDialogPinned->SetCaptureRegionWidget(InCaptureRegionWidget);
			CurrentDialogPinned->SetCaptureRegionControlsVisibility(false);
		}

		CurrentWindow.Pin()->BringToFront();
		Config.ChangeViewport(InViewport);

		// Enable mask visualization if the mask is enabled
		if (InViewport.IsValid())
		{
			bMaskVisualizationWasEnabled = InViewport->GetClient()->GetEngineShowFlags()->HighResScreenshotMask;
			InViewport->GetClient()->GetEngineShowFlags()->SetHighResScreenshotMask(Config.bMaskEnabled);
		}
	}

	return CurrentWindow;
}

FReply SHighResScreenshotDialog::OnSelectCaptureRegionClicked()
{
	// Only enable the capture region widget if the owning viewport gave us one
	if (CaptureRegionWidget.IsValid())
	{
		CaptureRegionWidget->Activate(Config.UnscaledCaptureRegion.Area() > 0);
		bCaptureRegionControlsVisible = true;
	}
	return FReply::Handled();
}

EVisibility SHighResScreenshotDialog::GetCaptureWarningVisibility() const
{
	return Config.ResolutionMultiplier >= 3. ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SHighResScreenshotDialog::OnOpenScreenshotFolder()
{
	const FString PathToScreenshotFolder = FPaths::ConvertRelativePathToFull(FPaths::ScreenShotDir());

	// Before navigating to folder, check if it exists and create it if not
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*PathToScreenshotFolder))
	{
		PlatformFile.CreateDirectoryTree(*PathToScreenshotFolder);
	}

	// Navigate to folder in explorer
	FPlatformProcess::ExploreFolder(*PathToScreenshotFolder);

	return FReply::Handled();
}

FReply SHighResScreenshotDialog::OnCaptureClicked()
{
	if (!GIsHighResScreenshot)
	{
		auto ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			GScreenshotResolutionX = uint32(static_cast<float>(ConfigViewport->GetRenderTargetTextureSizeXY().X) * Config.ResolutionMultiplier);
			GScreenshotResolutionY = uint32(static_cast<float>(ConfigViewport->GetRenderTargetTextureSizeXY().Y) * Config.ResolutionMultiplier);
			FIntRect ScaledCaptureRegion = Config.UnscaledCaptureRegion;

			if (ScaledCaptureRegion.Area() > 0)
			{
				ScaledCaptureRegion.Clip(FIntRect(FIntPoint::ZeroValue, ConfigViewport->GetRenderTargetTextureSizeXY()));
				ScaledCaptureRegion *= FMath::TruncToInt32(Config.ResolutionMultiplier);
			}

			Config.CaptureRegion = ScaledCaptureRegion;

			// Trigger the screenshot on the owning viewport
			ConfigViewport->TakeHighResScreenShot();
		}
	}

	return FReply::Handled();
}

FReply SHighResScreenshotDialog::OnSelectCaptureCancelRegionClicked()
{
	if (CaptureRegionWidget.IsValid())
	{
		auto ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			ConfigViewport->Invalidate();
		}

		CaptureRegionWidget->Deactivate(false);
	}

	// Hide the Cancel/Accept buttons, show the Edit button
	bCaptureRegionControlsVisible = false;
	return FReply::Handled();
}

FReply SHighResScreenshotDialog::OnSelectCaptureAcceptRegionClicked()
{
	if (CaptureRegionWidget.IsValid())
	{
		CaptureRegionWidget->Deactivate(true);
	}

	// Hide the Cancel/Accept buttons, show the Edit button
	bCaptureRegionControlsVisible = false;
	return FReply::Handled();
}

FReply SHighResScreenshotDialog::OnSetFullViewportCaptureRegionClicked()
{
	auto ConfigViewport = Config.TargetViewport.Pin();
	if (ConfigViewport.IsValid())
	{
		ConfigViewport->Invalidate();
	}

	Config.UnscaledCaptureRegion = FIntRect(0, 0, 0, 0);
	CaptureRegionWidget->Reset();
	return FReply::Handled();
}

FReply SHighResScreenshotDialog::OnSetCameraSafeAreaCaptureRegionClicked()
{
	FIntRect NewCaptureRegion;

	if (Config.TargetViewport.IsValid())
	{
		auto ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport->GetClient()->OverrideHighResScreenshotCaptureRegion(NewCaptureRegion))
		{
			Config.UnscaledCaptureRegion = NewCaptureRegion;
			ConfigViewport->Invalidate();
		}
	}

	return FReply::Handled();
}

bool SHighResScreenshotDialog::IsSetCameraSafeAreaCaptureRegionEnabled() const
{
	if (Config.TargetViewport.IsValid())
	{
		FViewportClient* Client = Config.TargetViewport.Pin()->GetClient();
		if (Client)
		{
			FIntRect Rect;
			if (Client->OverrideHighResScreenshotCaptureRegion(Rect))
			{
				return true;
			}
		}
	}

	return false;
}
