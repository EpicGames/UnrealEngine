// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "ViewportClient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "ShowFlags.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Slate/SceneViewport.h"
#include "SCaptureRegionWidget.h"
#include "Widgets/Input/SCheckBox.h"
#include "HighResScreenshot.h"

class SButton;

class SHighResScreenshotDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SHighResScreenshotDialog ){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	SHighResScreenshotDialog();

	virtual ~SHighResScreenshotDialog()
	{
		Window.Reset();
	}

	void SetWindow( TSharedPtr<SWindow> InWindow )
	{
		Window = InWindow;
	}

	void SetCaptureRegionWidget(TSharedPtr<class SCaptureRegionWidget> InCaptureRegionWidget)
	{
		CaptureRegionWidget = InCaptureRegionWidget;
	}

	void SetCaptureRegion(const FIntRect& InCaptureRegion)
	{
		Config.UnscaledCaptureRegion = InCaptureRegion;
		auto ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			ConfigViewport->Invalidate();
		}
	}

	FHighResScreenshotConfig& GetConfig()
	{
		return Config;
	}

	TSharedPtr<class SCaptureRegionWidget> GetCaptureRegionWidget()
	{
		return CaptureRegionWidget;
	}

	static TWeakPtr<class SWindow> OpenDialog(const TSharedPtr<FSceneViewport>& InViewport, TSharedPtr<SCaptureRegionWidget> InCaptureRegionWidget = TSharedPtr<class SCaptureRegionWidget>());

private:
	EVisibility GetCaptureWarningVisibility() const;

	FReply OnOpenScreenshotFolder();
	FReply OnCaptureClicked();
	FReply OnSelectCaptureRegionClicked();
	FReply OnSelectCaptureCancelRegionClicked();
	FReply OnSelectCaptureAcceptRegionClicked();
	FReply OnSetFullViewportCaptureRegionClicked();
	FReply OnSetCameraSafeAreaCaptureRegionClicked();

	bool IsSetCameraSafeAreaCaptureRegionEnabled() const;

	void OnResolutionMultiplierSliderChanged( float NewValue )
	{
		Config.ResolutionMultiplier = NewValue;

		// scale needs to be [0, 1.0]
		Config.ResolutionMultiplierScale = (NewValue - FHighResScreenshotConfig::MinResolutionMultipler) / (FHighResScreenshotConfig::MaxResolutionMultipler - FHighResScreenshotConfig::MinResolutionMultipler);
	}

	void OnMaskEnabledChanged( ECheckBoxState NewValue )
	{
		Config.bMaskEnabled = (NewValue == ECheckBoxState::Checked);
		auto ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			ConfigViewport->GetClient()->GetEngineShowFlags()->SetHighResScreenshotMask(Config.bMaskEnabled);
			ConfigViewport->Invalidate();
		}
	}

	void OnHDREnabledChanged(ECheckBoxState NewValue)
	{
		Config.SetHDRCapture(NewValue == ECheckBoxState::Checked);
		auto ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			ConfigViewport->Invalidate();
		}
	}

	void OnForce128BitRenderingChanged(ECheckBoxState NewValue)
	{
		Config.SetForce128BitRendering(NewValue == ECheckBoxState::Checked);
		auto ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			ConfigViewport->Invalidate();
		}
	}

	void OnDateTimeBasedNamingEnabledChanged(ECheckBoxState NewValue)
	{
		bool bEnabled = (NewValue == ECheckBoxState::Checked);
		Config.bDateTimeBasedNaming = bEnabled;
	}

	void OnBufferVisualizationDumpEnabledChanged(ECheckBoxState NewValue)
	{
		bool bEnabled = (NewValue == ECheckBoxState::Checked);
		Config.bDumpBufferVisualizationTargets = bEnabled;
		SetHDRUIEnableState(bEnabled);
		SetForce128BitRenderingState(bEnabled);
	}

	EVisibility GetSpecifyCaptureRegionVisibility() const
	{
		return bCaptureRegionControlsVisible ? EVisibility::Collapsed: EVisibility::Visible;
	}

	EVisibility GetCaptureRegionControlsVisibility() const
	{
		return bCaptureRegionControlsVisible ? EVisibility::Visible : EVisibility::Collapsed;
	}

	void SetCaptureRegionControlsVisibility(bool bVisible)
	{
		bCaptureRegionControlsVisible = bVisible;
	}

	float GetResolutionMultiplierSlider() const
	{
		return Config.ResolutionMultiplier;
	}

	ECheckBoxState GetMaskEnabled() const
	{
		return Config.bMaskEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	ECheckBoxState GetHDRCheckboxUIState() const
	{
		return Config.bCaptureHDR ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	ECheckBoxState GetForce128BitRenderingCheckboxUIState() const
	{
		return Config.bForce128BitRendering ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	ECheckBoxState GetBufferVisualizationDumpEnabled() const
	{
		return Config.bDumpBufferVisualizationTargets ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	ECheckBoxState GetDateTimeBasedNamingEnabled() const
	{
		return Config.bDateTimeBasedNaming ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	bool IsCaptureRegionEditingAvailable() const
	{
		return CaptureRegionWidget.IsValid();
	}
	
	void SetHDRUIEnableState(bool bEnable)
	{
		HDRCheckBox->SetEnabled(bEnable);
		HDRLabel->SetEnabled(bEnable);
	}

	void SetForce128BitRenderingState(bool bEnable)
	{
		Force128BitRenderingCheckBox->SetEnabled(bEnable);
		Force128BitRenderingLabel->SetEnabled(bEnable);
	}

	static void WindowClosedHandler(const TSharedRef<SWindow>& InWindow);

	static void ResetViewport();
	static void ResetFrameBuffer();

	TSharedPtr<SWindow> Window;
	TSharedPtr<class SCaptureRegionWidget> CaptureRegionWidget;
	TSharedPtr<SButton> CaptureRegionButton;
	TSharedPtr<SHorizontalBox> RegionCaptureActiveControlRoot;
	TSharedPtr<SCheckBox> HDRCheckBox;
	TSharedPtr<STextBlock> HDRLabel;
	TSharedPtr<SCheckBox> Force128BitRenderingCheckBox;
	TSharedPtr<STextBlock> Force128BitRenderingLabel;

	FHighResScreenshotConfig& Config;
	bool bCaptureRegionControlsVisible;

	static TWeakPtr<SWindow> CurrentWindow;
	static TWeakPtr<SHighResScreenshotDialog> CurrentDialog;
	static bool bMaskVisualizationWasEnabled;
};
