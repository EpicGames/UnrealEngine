// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditor.h"

#include "CommonMenuExtensionsModule.h"
#include "ContentBrowserModule.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor.h"
#include "EditorModes.h"
#include "EditorViewportCommands.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IContentBrowserSingleton.h"
#include "Interfaces/IProjectManager.h"
#include "ISlateReflectorModule.h"
#include "Layout/WidgetPath.h"
#include "LevelEditorViewport.h"
#include "LevelViewportActions.h"
#include "LevelViewportLayout.h"
#include "LevelViewportLayoutEntity.h"
#include "LevelEditorActions.h"
#include "LevelEditorModesActions.h"
#include "LightmapResRatioAdjust.h"
#include "MessageLogModule.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Model.h"
#include "Modules/ModuleManager.h"
#include "PixelInspectorModule.h"
#include "PlatformInfo.h"
#include "ProjectDescriptor.h"
#include "SLevelEditor.h"
#include "SLevelViewport.h"
#include "Styling/AppStyle.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

// @todo Editor: remove this circular dependency
#include "Interfaces/IMainFrameModule.h"
#include "Framework/Commands/GenericCommands.h"
#include "Misc/EngineBuildSettings.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "LevelEditorOutlinerSettings.h"

#define LOCTEXT_NAMESPACE "LevelEditor"

IMPLEMENT_MODULE( FLevelEditorModule, LevelEditor );
LLM_DEFINE_TAG(LevelEditor);

const FName LevelEditorApp = FName(TEXT("LevelEditorApp"));
const FName MainFrame("MainFrame");
const FName CommonMenuExtensionsName(TEXT("CommonMenuExtensions"));

const FName LevelEditorTabIds::LevelEditorViewport(TEXT("LevelEditorViewport"));
const FName LevelEditorTabIds::LevelEditorViewport_Clone1(TEXT("LevelEditorViewport_Clone1"));
const FName LevelEditorTabIds::LevelEditorViewport_Clone2(TEXT("LevelEditorViewport_Clone2"));
const FName LevelEditorTabIds::LevelEditorViewport_Clone3(TEXT("LevelEditorViewport_Clone3"));
const FName LevelEditorTabIds::LevelEditorViewport_Clone4(TEXT("LevelEditorViewport_Clone4"));
const FName LevelEditorTabIds::LevelEditorToolBox(TEXT("LevelEditorToolBox"));
const FName LevelEditorTabIds::LevelEditorSelectionDetails(TEXT("LevelEditorSelectionDetails"));
const FName LevelEditorTabIds::LevelEditorSelectionDetails2(TEXT("LevelEditorSelectionDetails2"));
const FName LevelEditorTabIds::LevelEditorSelectionDetails3(TEXT("LevelEditorSelectionDetails3"));
const FName LevelEditorTabIds::LevelEditorSelectionDetails4(TEXT("LevelEditorSelectionDetails4"));
const FName LevelEditorTabIds::PlacementBrowser(TEXT("PlacementBrowser"));
const FName LevelEditorTabIds::LevelEditorBuildAndSubmit(TEXT("LevelEditorBuildAndSubmit"));
const FName LevelEditorTabIds::LevelEditorSceneOutliner(TEXT("LevelEditorSceneOutliner"));
const FName LevelEditorTabIds::LevelEditorSceneOutliner2(TEXT("LevelEditorSceneOutliner2"));
const FName LevelEditorTabIds::LevelEditorSceneOutliner3(TEXT("LevelEditorSceneOutliner3"));
const FName LevelEditorTabIds::LevelEditorSceneOutliner4(TEXT("LevelEditorSceneOutliner4"));
const FName LevelEditorTabIds::LevelEditorStatsViewer(TEXT("LevelEditorStatsViewer"));
const FName LevelEditorTabIds::LevelEditorLayerBrowser(TEXT("LevelEditorLayerBrowser"));
const FName LevelEditorTabIds::LevelEditorDataLayerBrowser(TEXT("LevelEditorDataLayerBrowser"));
const FName LevelEditorTabIds::Sequencer(TEXT("Sequencer"));
const FName LevelEditorTabIds::SequencerGraphEditor(TEXT("SequencerGraphEditor"));
const FName LevelEditorTabIds::WorldSettings(TEXT("WorldSettingsTab"));
const FName LevelEditorTabIds::WorldBrowserComposition(TEXT("WorldBrowserComposition"));
const FName LevelEditorTabIds::WorldBrowserPartitionEditor(TEXT("WorldBrowserPartitionEditor"));
const FName LevelEditorTabIds::WorldBrowserHierarchy(TEXT("WorldBrowserHierarchy"));
const FName LevelEditorTabIds::WorldBrowserDetails(TEXT("WorldBrowserDetails"));
const FName LevelEditorTabIds::LevelEditorHierarchicalLODOutliner(TEXT("LevelEditorHierarchicalLODOutliner"));
const FName LevelEditorTabIds::OutputLog(TEXT("OutputLog"));
const FName LevelEditorTabIds::LevelEditorEnvironmentLightingViewer(TEXT("LevelEditorEnvironmentLightingViewer"));

namespace UE::Editor::LevelEditor::Private
{
	void OnPostEngineInit()
	{
		if (GEditor)
		{
			if (UPanelExtensionSubsystem* PanelExtensionSubsystem = GEditor->GetEditorSubsystem<UPanelExtensionSubsystem>())
			{
				FText DeprecationMessage = NSLOCTEXT("UnrealEditor", "LevelViewportToolbarPanelExtensionDeprecationMessage", "Please extend the \"LevelEditor.ViewportToolbar\" Tool Menus menu instead.");
			
				PanelExtensionSubsystem->MarkDeprecatedExtension("LevelViewportToolBar.LeftExtension", DeprecationMessage);
				PanelExtensionSubsystem->MarkDeprecatedExtension("LevelViewportToolBar.MiddleExtension", DeprecationMessage);
				PanelExtensionSubsystem->MarkDeprecatedExtension("LevelViewportToolBar.RightExtension", DeprecationMessage);
				PanelExtensionSubsystem->MarkDeprecatedExtension("LevelViewportToolBar.RightmostExtension", DeprecationMessage);
			}
		}
	}
}

FLevelEditorModule::FLevelEditorModule()
	: ToggleImmersiveConsoleCommand(
	TEXT( "LevelEditor.ToggleImmersive" ),
	TEXT( "Toggle 'Immersive Mode' for the active level editing viewport" ),
	FConsoleCommandDelegate::CreateRaw( this, &FLevelEditorModule::ToggleImmersiveOnActiveLevelViewport ) )
{
}


class SProjectBadge : public SBox
{
public:
	SLATE_BEGIN_ARGS(SProjectBadge) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		FString ProjectNameWatermarkPrefix;
		GConfig->GetString(TEXT("LevelEditor"), TEXT("ProjectNameWatermarkPrefix"), /*out*/ ProjectNameWatermarkPrefix, GEditorPerProjectIni);

		FSlateColor BadgeBackgroundColor = FAppStyle::Get().GetSlateColor("Colors.Title");
		{
			FColor ConfigColor;
			if (GConfig->GetColor(
					TEXT("LevelEditor"), TEXT("ProjectBadgeBackgroundColor"), /*out*/ ConfigColor, GEditorPerProjectIni
				))
			{
				BadgeBackgroundColor = FLinearColor(ConfigColor);
			}
		}

		FSlateColor BadgeTextColor = FStyleColors::Foreground;
		{
			FColor ConfigColor;
			if (GConfig->GetColor(TEXT("LevelEditor"), TEXT("ProjectBadgeTextColor"), /*out*/ ConfigColor, GEditorPerProjectIni))
			{
				BadgeTextColor = FLinearColor(ConfigColor);
			}
		}

		const FString EngineVersionString = FEngineVersion::Current().ToString(FEngineVersion::Current().HasChangelist() ? EVersionComponent::Changelist : EVersionComponent::Patch);
		const EBuildConfiguration BuildConfig = FApp::GetBuildConfiguration();
		const bool bAppendBuildConfig = (BuildConfig != EBuildConfiguration::Shipping && BuildConfig != EBuildConfiguration::Development && BuildConfig != EBuildConfiguration::Unknown);

		FFormatNamedArguments LabelArgs;
		LabelArgs.Add(TEXT("ProjectNameWatermarkPrefix"), FText::FromString(ProjectNameWatermarkPrefix));
		LabelArgs.Add(TEXT("Branch"), FEngineBuildSettings::IsPerforceBuild() ? FText::FromString(TEXT(" ") + FEngineVersion::Current().GetBranch()) : FText::GetEmpty());
		LabelArgs.Add(TEXT("GameName"), FText::FromString(TEXT(" ") + FString(FApp::GetProjectName())));
		LabelArgs.Add(TEXT("EngineVersion"), (GetDefault<UEditorPerProjectUserSettings>()->bDisplayEngineVersionInBadge) ? FText::FromString(TEXT(" (") + EngineVersionString + TEXT(")")) : FText::GetEmpty());
		LabelArgs.Add(TEXT("Config"), bAppendBuildConfig ? FText::Format(NSLOCTEXT("UnrealEditor", "TitleBarConfig", " [{0}]"), EBuildConfigurations::ToText(BuildConfig)) : FText::GetEmpty());

		const FText RightContentText = FText::Format(NSLOCTEXT("UnrealEditor", "TitleBarRightContentAndConfig", "{ProjectNameWatermarkPrefix}{GameName}{Config}{Branch}{EngineVersion}"), LabelArgs);
	
		// Create the tooltip showing more detailed information
		{
			const FText RightContentTooltip = FText::FormatOrdered(NSLOCTEXT("UnrealEditor", "TitleBarRightContentInfoTooltip", "{0}\n\nRight Click to open context menu with various clipboard options\nDouble Click to quickly copy changelist to clipboard"), GetAllInfoText());
			SetToolTipText(RightContentTooltip);
		}

		TSharedRef<SWidget> DefaultNamePlate = SNew(STextBlock)
			.Text(RightContentText)
			.TextStyle(FAppStyle::Get(), "SProjectBadge.Text")
			.Margin(FAppStyle::Get().GetMargin("SProjectBadge.BadgePadding"))
			.ColorAndOpacity(BadgeTextColor);

		// clang-format off
		SBox::Construct(SBox::FArguments()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Top)
			.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
			[
				SNew(SBorder)
				.OnMouseDoubleClick(this, &SProjectBadge::OnBadgeDoubleClick)
				.OnMouseButtonDown(this, &SProjectBadge::OnOpenContextMenu)
				.BorderBackgroundColor(BadgeBackgroundColor)
				.BorderImage(FAppStyle::GetBrush("SProjectBadge.BadgeShape"))
				.Padding(FMargin(0))
				.VAlign(VAlign_Top)
				[
					SNew(SExtensionPanel)
					.ExtensionPanelID("LevelEditorProjectNamePlate")
					.DefaultWidget(DefaultNamePlate)
					.WindowZoneOverride(EWindowZone::TitleBar)
				]
			]);
		// clang-format on
	}

	FVector2D GetSizeLastFrame() const
	{
		return GetDesiredSize();
	}

	// SWidget interface
	virtual EWindowZone::Type GetWindowZoneOverride() const override
	{
		return EWindowZone::TitleBar;
	}
	// End of SWidget interface

private:
	static void CopyToClipboard(const FText& InKey, const FText& InValue)
	{
		FPlatformApplicationMisc::ClipboardCopy(*InValue.ToString());
		
		// Notify of the action
		FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ProjectBadgeCopyNotification", "{0} value copied to clipboard\n{1}"), InKey, InValue));
		NotificationInfo.ExpireDuration = 1.f;
		NotificationInfo.bFireAndForget = true;
		NotificationInfo.Image = FAppStyle::GetBrush("Icons.SuccessWithColor");
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}
	
	static FText GetProjectNameText()
	{
		return FText::FromString(FApp::GetProjectName());
	}

	static FText GetVersionText()
	{
		const FString EngineVersionString = FEngineVersion::Current().ToString(FEngineVersion::Current().HasChangelist() ? EVersionComponent::Changelist : EVersionComponent::Patch);
		return FText::FromString(EngineVersionString);
	}
	
	static FText GetChangelistText()
	{
		return FText::FromString(LexToString(FEngineVersion::Current().GetChangelist()));
	}
	
	static FText GetBranchText()
	{
		return FText::FromString(FEngineVersion::Current().GetBranch());
	}
	
	static FText GetBuildConfigurationText()
	{
		return EBuildConfigurations::ToText(FApp::GetBuildConfiguration());
	}

	static FText GetBuildDateText()
	{
		return FText::FromString(FApp::GetBuildDate());
	}
	
	static FText GetGraphicsRHIText()
	{
		return FText::FromString(FApp::GetGraphicsRHI());
	}
	
	static FText GetAllInfoText()
	{
		FFormatNamedArguments TooltipArgs;
		TooltipArgs.Add(TEXT("Version"), GetVersionText());
		TooltipArgs.Add(TEXT("Changelist"), GetChangelistText());
		TooltipArgs.Add(TEXT("Branch"), GetBranchText());
		TooltipArgs.Add(TEXT("BuildConfiguration"), GetBuildConfigurationText());
		TooltipArgs.Add(TEXT("BuildDate"), GetBuildDateText());
		TooltipArgs.Add(TEXT("GraphicsRHI"), GetGraphicsRHIText());
		return FText::Format(NSLOCTEXT("UnrealEditor", "TitleBarRightContentTooltip", "Version: {Version}\nBranch: {Branch}\nBuild Configuration: {BuildConfiguration}\nBuild Date: {BuildDate}\nGraphics RHI: {GraphicsRHI}"), TooltipArgs);
	}

	static TSharedRef<SWidget> CreateContextMenu()
	{
		FMenuBuilder MenuBuilder(/** CloseAfterSelection */true, /** CommandList */nullptr);
		
		const FText LabelFormat = LOCTEXT("ProjectBadgeContextLabel", "Copy {0} ({1})");
		const FText TooltipFormat = LOCTEXT("ProjectBadgeContextTooltip", "Copy {0} value to the clipboard:\n{1}");

		// ProjectName
		{
			const FText ProjectNameLabel = LOCTEXT("ProjectNameLabel", "Project Name");
			const FText ProjectNameValue = GetProjectNameText();

			MenuBuilder.AddMenuEntry(
				FText::Format(LabelFormat, ProjectNameLabel, ProjectNameValue),
				FText::Format(TooltipFormat, ProjectNameLabel, ProjectNameValue),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda(&SProjectBadge::CopyToClipboard, ProjectNameLabel, ProjectNameValue))
			);
		}
		
		// Version
		{
			const FText VersionLabel = LOCTEXT("VersionLabel", "Version");
			const FText VersionValue = GetVersionText();

			MenuBuilder.AddMenuEntry(
				FText::Format(LabelFormat, VersionLabel, VersionValue),
				FText::Format(TooltipFormat, VersionLabel, VersionValue),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda(&SProjectBadge::CopyToClipboard, VersionLabel, VersionValue))
			);
		}

		// Changelist
		{
			const FText ChangelistLabel = LOCTEXT("ChangelistLabel", "Changelist");
			const FText ChangelistValue = GetChangelistText();

			MenuBuilder.AddMenuEntry(
				FText::Format(LabelFormat, ChangelistLabel, ChangelistValue),
				FText::Format(TooltipFormat, ChangelistLabel, ChangelistValue),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda(&SProjectBadge::CopyToClipboard, ChangelistLabel, ChangelistValue))
			);
		}
		
		// Branch
		{
			const FText BranchLabel = LOCTEXT("BranchLabel", "Branch");
			const FText BranchValue = GetBranchText();

			MenuBuilder.AddMenuEntry(
				FText::Format(LabelFormat, BranchLabel, BranchValue),
				FText::Format(TooltipFormat, BranchLabel, BranchValue),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda(&SProjectBadge::CopyToClipboard, BranchLabel, BranchValue))
			);
		}
		
		// BuildConfiguration
		{
			const FText BuildConfigurationLabel = LOCTEXT("BuildConfigurationLabel", "Build Configuration");
			const FText BuildConfigurationValue = GetBuildConfigurationText();

			MenuBuilder.AddMenuEntry(
				FText::Format(LabelFormat, BuildConfigurationLabel, BuildConfigurationValue),
				FText::Format(TooltipFormat, BuildConfigurationLabel, BuildConfigurationValue),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda(&SProjectBadge::CopyToClipboard, BuildConfigurationLabel, BuildConfigurationValue))
			);
		}
		
		// BuildDate
		{
			const FText BuildDateLabel = LOCTEXT("BuildDateLabel", "Build Date");
			const FText BuildDateValue = GetBuildDateText();

			MenuBuilder.AddMenuEntry(
				FText::Format(LabelFormat, BuildDateLabel, BuildDateValue),
				FText::Format(TooltipFormat, BuildDateLabel, BuildDateValue),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda(&SProjectBadge::CopyToClipboard, BuildDateLabel, BuildDateValue))
			);
		}
		
		// GraphicsRHI
		{
			const FText GraphicsRHILabel = LOCTEXT("GraphicsRHILabel", "Graphics RHI");
			const FText GraphicsRHIValue = GetGraphicsRHIText();

			MenuBuilder.AddMenuEntry(
				FText::Format(LabelFormat, GraphicsRHILabel, GraphicsRHIValue),
				FText::Format(TooltipFormat, GraphicsRHILabel, GraphicsRHIValue),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda(&SProjectBadge::CopyToClipboard, GraphicsRHILabel, GraphicsRHIValue))
			);
		}
		
		// All info
		{
			const FText AllInfoLabel = LOCTEXT("AllLabel", "Copy All Info");
			const FText AllInfoValue = GetAllInfoText();

			MenuBuilder.AddMenuEntry(
				AllInfoLabel,
				FText::Format(TooltipFormat, AllInfoLabel, AllInfoValue),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda(&SProjectBadge::CopyToClipboard, AllInfoLabel, AllInfoValue))
			);
		}

		return MenuBuilder.MakeWidget();
	}

	FReply OnBadgeDoubleClick(const FGeometry& InGeometry, const FPointerEvent& InEvent)
	{
		const FText ChangelistText = GetChangelistText();
		if (!ChangelistText.IsEmpty())
		{
			CopyToClipboard(LOCTEXT("ChangelistLabel", "Changelist"), ChangelistText);
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	FReply OnOpenContextMenu(const FGeometry& InGeometry, const FPointerEvent& InEvent)
	{
		if (InEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			FSlateApplication::Get().PushMenu(
				AsShared(),
				FWidgetPath(),
				CreateContextMenu(),
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	FGeometry CachedGeometry;
};

template <>
struct TWidgetTypeTraits<SProjectBadge>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

TSharedRef<SDockTab> FLevelEditorModule::SpawnLevelEditor( const FSpawnTabArgs& InArgs )
{
	LLM_SCOPE_BYTAG(LevelEditor);
	
	TSharedRef<SDockTab> LevelEditorTab = SNew(SDockTab)
		.OnExtendContextMenu(SDockTab::FExtendContextMenu::CreateRaw(this, &FLevelEditorModule::OnExtendTabContextMenu))
		.TabRole(ETabRole::MajorTab)
		.ContentPadding(FMargin(0))
		.IconColor(FAppStyle::Get().GetColor("LevelEditor.AssetColor")); // Same color as FAssetTypeActions_World

	LevelEditorTab->SetTabIcon(FAppStyle::Get().GetBrush("LevelEditor.Tab"));

	SetLevelEditorInstanceTab(LevelEditorTab);
	TSharedPtr< SWindow > OwnerWindow = InArgs.GetOwnerWindow();
	
	if ( !OwnerWindow.IsValid() )
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( MainFrame );
		OwnerWindow = MainFrameModule.GetParentWindow();
	}

	OutlinerSettings->CreateDefaultFilters();

	TSharedPtr<SLevelEditor> LevelEditorTmp;
	if (OwnerWindow.IsValid())
	{
		// Make sure that any mode that could hold on to the old SLevelEditor releases it before setting the new one below
		GLevelEditorModeTools().DeactivateAllModes();

		LevelEditorTab->SetContent(SAssignNew(LevelEditorTmp, SLevelEditor));
		SetLevelEditorInstance(LevelEditorTmp);
		LevelEditorTmp->Initialize(LevelEditorTab, OwnerWindow.ToSharedRef());

		LevelEditorCreatedEvent.Broadcast(LevelEditorTmp);

		TSharedRef<SProjectBadge> ProjectBadge = SNew(SProjectBadge);

		TSharedPtr< SWidget > RightContent =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				LevelEditorTmp->GetTitleBarMessageWidget()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 8.0f, 0.0f)
			[
				ProjectBadge
			];


		LevelEditorTab->SetTitleBarRightContent(RightContent.ToSharedRef());
	}
	
	return LevelEditorTab;
}

void FLevelEditorModule::OnExtendTabContextMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("DockingTabLevelOptions", LOCTEXT("LevelTabSection", "Level"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("BrowseLevelInContentBrowser", "Browse to Level"),
			LOCTEXT("BrowseLevelInContentBrowserTooltip", "Browse to level in content browser"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BrowseContent"),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FLevelEditorModule::OnBrowseLevelInContentBrowser),
				FCanExecuteAction::CreateRaw(this, &FLevelEditorModule::OnCanBrowseLevelInContentBrowser)
			)
		);
	}
	InMenuBuilder.EndSection();
}

bool FLevelEditorModule::OnCanBrowseLevelInContentBrowser()
{
	if (const TSharedPtr<ILevelEditor> LevelEditor = GetLevelEditorInstance().Pin())
	{
		const UWorld* World = LevelEditor->GetWorld();

		return !!World
			&& !!World->PersistentLevel
			&& !!World->PersistentLevel->GetPackage()
			&& !World->PersistentLevel->GetPackage()->GetName().StartsWith("/Temp/");
	}

	return false;
}

void FLevelEditorModule::OnBrowseLevelInContentBrowser()
{
	if (const TSharedPtr<ILevelEditor> LevelEditor = GetLevelEditorInstance().Pin())
	{
		UWorld* World = LevelEditor->GetWorld();

		if (!!World
			&& !!World->PersistentLevel
			&& !!World->PersistentLevel->GetPackage()
			&& !World->PersistentLevel->GetPackage()->GetName().StartsWith("/Temp/"))
		{
			if (const FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
			{
				ContentBrowserModule->Get().SyncBrowserToAssets({World->PersistentLevel});
			}
		}
	}
}


/**
 * Called right after the module's DLL has been loaded and the module object has been created
 */
void FLevelEditorModule::StartupModule()
{
	// Our command context bindings depend on having the mainframe loaded
	FModuleManager::LoadModuleChecked<IMainFrameModule>(MainFrame);

	FModuleManager::LoadModuleChecked<FCommonMenuExtensionsModule>(CommonMenuExtensionsName);

	MenuExtensibilityManager = MakeShared<FExtensibilityManager>();
	ToolBarExtensibilityManager = MakeShared<FExtensibilityManager>();
	ModeBarExtensibilityManager = MakeShared<FExtensibilityManager>();
	NotificationBarExtensibilityManager = MakeShared<FExtensibilityManager>();

	// Note this must come before any tab spawning because that can create the SLevelEditor and attempt to map commands
	FGenericCommands::Register();
	FGlobalEditorCommonCommands::Register();
	FEditorViewportCommands::Register();
	FLevelViewportCommands::Register();
	FLevelEditorCommands::Register();

	// Bind level editor commands shared across an instance
	BindGlobalLevelEditorCommands();

	// Exposes the global level editor command list to subscribers from other systems
	FInputBindingManager::Get().RegisterCommandList(FLevelEditorCommands::Get().GetContextName(), GetGlobalLevelEditorActions());

	FViewportTypeDefinition ViewportType = FViewportTypeDefinition([](const FAssetEditorViewportConstructionArgs& ConstructionArgs, TSharedPtr<ILevelEditor> InLevelEditor)
		{
			TSharedPtr<SLevelViewport> EditorViewport = SNew(SLevelViewport, ConstructionArgs)
				.ParentLevelEditor(InLevelEditor);

			return MakeShareable(new FLevelViewportLayoutEntity(EditorViewport));
		},
		FLevelViewportCommands::Get().SetDefaultViewportType);
	RegisterViewportType("Default", ViewportType);

	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();

	FGlobalTabmanager::Get()->RegisterTabSpawner("LevelEditor", FOnSpawnTab::CreateRaw(this, &FLevelEditorModule::SpawnLevelEditor))
		.SetDisplayName(NSLOCTEXT("LevelEditor", "LevelEditorTab", "Level Editor"))
		.SetAutoGenerateMenuEntry(false);

	FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector").RegisterTabSpawner(MenuStructure.GetDeveloperToolsDebugCategory());

	FModuleManager::LoadModuleChecked<FPixelInspectorModule>("PixelInspectorModule").RegisterTabSpawner(MenuStructure.GetDeveloperToolsDebugCategory());

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing("BuildAndSubmitErrors", LOCTEXT("BuildAndSubmitErrors", "Build and Submit Errors"));

	OutlinerSettings = MakeShared<FLevelEditorOutlinerSettings>();
	OutlinerSettings->Initialize();
	OutlinerSettings->SetupBuiltInCategories();
	
	FCoreDelegates::GetOnPostEngineInit().AddStatic(&UE::Editor::LevelEditor::Private::OnPostEngineInit);
}

/**
 * Called before the module is unloaded, right before the module object is destroyed.
 */
void FLevelEditorModule::ShutdownModule()
{
	IProjectManager::Get().OnTargetPlatformsForCurrentProjectChanged().RemoveAll(this);

	if(FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing("BuildAndSubmitErrors");
	}

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();

	// Stop PIE/SIE before unloading the level editor module
	// Otherwise, when the module is reloaded, it's likely to be in a bad state
	if (GUnrealEd && GUnrealEd->PlayWorld)
	{
		GUnrealEd->EndPlayMap();
	}

	// If the level editor tab is currently open, close it
	{
		if (!IsEngineExitRequested())
		{
			TSharedPtr<SDockTab> LevelEditorTab = LevelEditorInstanceTabPtr.Pin();
			if (LevelEditorTab.IsValid())
			{
				LevelEditorTab->RemoveTabFromParent();
			}
			LevelEditorInstanceTabPtr.Reset();
		}
	}

	// Clear out some globals that may be referencing this module
	SetLevelEditorTabManager(nullptr);
	WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory()->ClearItems();

	if (FSlateApplication::IsInitialized() && FModuleManager::Get().IsModuleLoaded("SlateReflector"))
	{
		FGlobalTabmanager::Get()->UnregisterTabSpawner("LevelEditor");
		FModuleManager::GetModuleChecked<ISlateReflectorModule>("SlateReflector").UnregisterTabSpawner();
	}

	FLevelEditorCommands::Unregister();
	FLevelEditorModesCommands::Unregister();
	FEditorViewportCommands::Unregister();
	FLevelViewportCommands::Unregister();
}

/**
 * Spawns a new property viewer
 * @todo This only works with the first level editor. Fix it.
 */
void FLevelEditorModule::SummonSelectionDetails()
{
	TSharedPtr<SLevelEditor> LevelEditorInstance = LevelEditorInstancePtr.Pin();
	LevelEditorInstance->SyncDetailsToSelection();
}

void FLevelEditorModule::SummonBuildAndSubmit()
{
	TSharedPtr<SLevelEditor> LevelEditorInstance = LevelEditorInstancePtr.Pin();
	LevelEditorInstance->TryInvokeTab(LevelEditorTabIds::LevelEditorBuildAndSubmit);
}


void FLevelEditorModule::SummonWorldBrowserHierarchy()
{
	TSharedPtr<SLevelEditor> LevelEditorInstance = LevelEditorInstancePtr.Pin();
	LevelEditorInstance->TryInvokeTab(LevelEditorTabIds::WorldBrowserHierarchy);
}

void FLevelEditorModule::SummonWorldBrowserDetails()
{
	TSharedPtr<SLevelEditor> LevelEditorInstance = LevelEditorInstancePtr.Pin();
	LevelEditorInstance->TryInvokeTab(LevelEditorTabIds::WorldBrowserDetails);
}

void FLevelEditorModule::SummonWorldBrowserComposition()
{
	TSharedPtr<SLevelEditor> LevelEditorInstance = LevelEditorInstancePtr.Pin();
	LevelEditorInstance->TryInvokeTab(LevelEditorTabIds::WorldBrowserComposition);
}

// @todo remove when world-centric mode is added
TSharedPtr<SDockTab> FLevelEditorModule::AttachSequencer(TSharedPtr<SWidget> SequencerWidget, TSharedPtr<IAssetEditorInstance> SequencerAssetEditor)
{
	TSharedPtr<SLevelEditor> LevelEditorInstance = LevelEditorInstancePtr.Pin();
	if (LevelEditorInstance)
	{
		return LevelEditorInstance->AttachSequencer(SequencerWidget, SequencerAssetEditor);
	}

	return nullptr;
}

TSharedPtr<IAssetViewport> FLevelEditorModule::GetFirstActiveViewport()
{
	TSharedPtr<SLevelEditor> LevelEditorInstance = LevelEditorInstancePtr.Pin();
	return (LevelEditorInstance.IsValid()) ? LevelEditorInstance->GetActiveViewport() : nullptr;
}

TSharedPtr<SLevelViewport> FLevelEditorModule::GetFirstActiveLevelViewport()
{
	TSharedPtr<SLevelEditor> LevelEditorInstance = LevelEditorInstancePtr.Pin();
	return (LevelEditorInstance.IsValid()) ? LevelEditorInstance->GetActiveViewport() : nullptr;
}

void FLevelEditorModule::FocusPIEViewport()
{
	TSharedPtr<SLevelEditor> LevelEditorInstance = LevelEditorInstancePtr.Pin();
	if( LevelEditorInstance.IsValid() && LevelEditorTabManager.IsValid() && LevelEditorInstance->HasActivePlayInEditorViewport() )
	{
		FGlobalTabmanager::Get()->DrawAttentionToTabManager( LevelEditorTabManager.ToSharedRef() );
	}
}

void FLevelEditorModule::FocusViewport()
{
	TSharedPtr<IAssetViewport> ActiveLevelViewport = GetFirstActiveViewport();
	if( ActiveLevelViewport.IsValid() )
	{
		TSharedRef< const SWidget > ViewportAsWidget = ActiveLevelViewport->AsWidget();
		FWidgetPath FocusWidgetPath;

		if( FSlateApplication::Get().GeneratePathToWidgetUnchecked( ViewportAsWidget, FocusWidgetPath ) )
		{
			FSlateApplication::Get().SetKeyboardFocus( FocusWidgetPath, EFocusCause::SetDirectly );
		}
	}
}

void FLevelEditorModule::BroadcastActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	ActorSelectionChangedEvent.Broadcast(NewSelection, bForceRefresh);
}

void FLevelEditorModule::BroadcastElementSelectionChanged(const UTypedElementSelectionSet* SelectionSet, bool bForceRefresh)
{
	ElementSelectionChangedEvent.Broadcast(SelectionSet, bForceRefresh);
}

void FLevelEditorModule::BroadcastOverridePropertyEditorSelection(const TArray<AActor*>& NewSelection, bool bForceRefresh)
{
	OverridePropertyEditorSelectionEvent.Broadcast(NewSelection, bForceRefresh);
}

void FLevelEditorModule::BroadcastRedrawViewports( bool bInvalidateHitProxies )
{
	RedrawLevelEditingViewportsEvent.Broadcast( bInvalidateHitProxies );
}

void FLevelEditorModule::BroadcastTakeHighResScreenShots( )
{
	TakeHighResScreenShotsEvent.Broadcast();
}

void FLevelEditorModule::BroadcastMapChanged( UWorld* World, EMapChangeType MapChangeType )
{
	MapChangedEvent.Broadcast( World, MapChangeType );
}

void FLevelEditorModule::BroadcastComponentsEdited()
{
	ComponentsEditedEvent.Broadcast();
}

const FLevelEditorCommands& FLevelEditorModule::GetLevelEditorCommands() const
{
	return FLevelEditorCommands::Get();
}

const FLevelEditorModesCommands& FLevelEditorModule::GetLevelEditorModesCommands() const
{
	return FLevelEditorModesCommands::Get();
}

const FLevelViewportCommands& FLevelEditorModule::GetLevelViewportCommands() const
{
	return FLevelViewportCommands::Get();
}

TWeakPtr<class ILevelEditor> FLevelEditorModule::GetLevelEditorInstance() const
{
	return LevelEditorInstancePtr;
}

TWeakPtr<class SDockTab> FLevelEditorModule::GetLevelEditorInstanceTab() const
{
	return LevelEditorInstanceTabPtr;
}

TSharedPtr<FTabManager> FLevelEditorModule::GetLevelEditorTabManager() const
{
	return LevelEditorTabManager;
}

void FLevelEditorModule::SetLevelEditorInstance( TWeakPtr<SLevelEditor> LevelEditor )
{
	LevelEditorInstancePtr = LevelEditor;
	GLevelEditorModeTools().SetToolkitHost(LevelEditorInstancePtr.Pin().ToSharedRef());
}

void FLevelEditorModule::SetLevelEditorInstanceTab( TWeakPtr<SDockTab> LevelEditorTab )
{
	LevelEditorInstanceTabPtr = LevelEditorTab;
}

void FLevelEditorModule::SetLevelEditorTabManager( const TSharedPtr<SDockTab>& OwnerTab )
{
	if (LevelEditorTabManager.IsValid())
	{
		LevelEditorTabManager->UnregisterAllTabSpawners();
		LevelEditorTabManager.Reset();
	}

	if (OwnerTab.IsValid())
	{
		LevelEditorTabManager = FGlobalTabmanager::Get()->NewTabManager(OwnerTab.ToSharedRef());
		LevelEditorTabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateRaw(this, &FLevelEditorModule::HandleTabManagerPersistLayout));
		LevelEditorTabManager->SetAllowWindowMenuBar(true);

		TabManagerChangedEvent.Broadcast();
	}
}

void FLevelEditorModule::StartPlayInEditorSession()
{
	TSharedPtr<IAssetViewport> ActiveLevelViewport = GetFirstActiveViewport();
	FRequestPlaySessionParams SessionParams;

	if( ActiveLevelViewport.IsValid() )
	{
		// We never want to play from the camera's location at startup, because the camera could have
		// been abandoned in a strange location in the map
		#if 0	// @todo immersive
		{
			// If this is a perspective viewport, then we'll Play From Here
			const FEditorViewportClient& LevelViewportClient = ActiveLevelViewport->GetAssetViewportClient();
			if( LevelViewportClient.IsPerspective() )
			{
				// Start PIE from the camera's location and orientation!
				SessionParams.StartLocation = LevelViewportClient.GetViewLocation();
				SessionParams.StartRotation = LevelViewportClient.GetViewRotation();
			}
		}
		#endif

		SessionParams.DestinationSlateViewport = ActiveLevelViewport;

		GUnrealEd->RequestPlaySession(SessionParams);

		// Kick off the queued PIE session immediately.  This is so that at startup, we don't need to
		// wait for the next engine tick.  We want to see PIE gameplay when the editor first appears!
		GUnrealEd->StartQueuedPlaySessionRequest();
	}
}

void FLevelEditorModule::GoImmersiveWithActiveLevelViewport( const bool bForceGameView )
{
	TSharedPtr<IAssetViewport> ActiveLevelViewport = GetFirstActiveViewport();

	if( ActiveLevelViewport.IsValid() )
	{
		// Make sure we can find a path to the viewport.  This will fail in cases where the viewport widget
		// is in a backgrounded tab, etc.  We can't currently support starting PIE in a backgrounded tab
		// due to how PIE manages focus and requires event forwarding from the application.
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow( ActiveLevelViewport->AsWidget() );
		if(Window.IsValid() )
		{
			if( bForceGameView && !ActiveLevelViewport->IsInGameView() )
			{
				ActiveLevelViewport->ToggleGameView();
			}

			{
				const bool bWantImmersive = true;
				const bool bAllowAnimation = false;
				ActiveLevelViewport->MakeImmersive( bWantImmersive, bAllowAnimation );
				FVector2f WindowSize = Window->GetSizeInScreen();
				// Set the initial size of the viewport to be the size of the window. This must be done because Slate has not ticked yet so the viewport will have no initial size
				ActiveLevelViewport->GetActiveViewport()->SetInitialSize( FIntPoint( FMath::TruncToInt32( WindowSize.X ), FMath::TruncToInt32( WindowSize.Y ) ) );
			}
		}
	}
}

void FLevelEditorModule::ToggleImmersiveOnActiveLevelViewport()
{
	TSharedPtr< IAssetViewport > ActiveLevelViewport = GetFirstActiveViewport();
	if( ActiveLevelViewport.IsValid() )
	{
		// Toggle immersive mode (with animation!)
		const bool bAllowAnimation = true;
		ActiveLevelViewport->MakeImmersive( !ActiveLevelViewport->IsImmersive(), bAllowAnimation );
	}
}

/** @return Returns the first Level Editor that we currently know about */
TSharedPtr<ILevelEditor> FLevelEditorModule::GetFirstLevelEditor() const
{
	return LevelEditorInstancePtr.Pin();
}

TSharedPtr<SDockTab> FLevelEditorModule::GetLevelEditorTab() const
{
	return LevelEditorInstanceTabPtr.Pin();
}

void FLevelEditorModule::AddTitleBarItem(FName InTitleBarIdentifier, const FTitleBarItem& InTitleBarItem)
{
	TitleBarItems.FindOrAdd(InTitleBarIdentifier) = InTitleBarItem;
	BroadcastTitleBarMessagesChanged();
}

void FLevelEditorModule::RemoveTitleBarItem(FName InTitleBarIdentifier)
{
	TitleBarItems.Remove(InTitleBarIdentifier);
	BroadcastTitleBarMessagesChanged();
}

TSharedRef<ILevelViewportLayoutEntity> FLevelEditorModule::FactoryViewport(FName InTypeName, const FAssetEditorViewportConstructionArgs& ConstructionArgs) const
{
	const FViewportTypeDefinition* Definition = CustomViewports.Find(InTypeName);
	if (Definition)
	{
		return Definition->FactoryFunction(ConstructionArgs, GetFirstLevelEditor());
	}

	check(CustomViewports.Find("Default"));
	return CustomViewports["Default"].FactoryFunction(ConstructionArgs, GetFirstLevelEditor());
}

TSharedPtr<FExtender> FLevelEditorModule::AssembleExtenders(TSharedRef<FUICommandList>& InCommandList, TArray<FLevelEditorMenuExtender>& MenuExtenderDelegates) const
{
	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute(InCommandList));
		}
	}
	return FExtender::Combine(Extenders);
}

void FLevelEditorModule::BindGlobalLevelEditorCommands()
{
	check( !GlobalLevelEditorActions.IsValid() );

	GlobalLevelEditorActions = MakeShareable( new FUICommandList );

	const FLevelEditorCommands& Commands = FLevelEditorCommands::Get();
	FUICommandList& ActionList = *GlobalLevelEditorActions;

	// Make a default can execute action that disables input when in debug mode
	FCanExecuteAction DefaultExecuteAction = FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::DefaultCanExecuteAction );

	ActionList.MapAction( Commands.BrowseDocumentation, FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BrowseDocumentation ) );
	ActionList.MapAction( Commands.BrowseViewportControls, FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BrowseViewportControls ) );
	ActionList.MapAction( Commands.NewLevel, FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::NewLevel ), FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::NewLevel_CanExecute ) );
	ActionList.MapAction( Commands.Save, FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Save ), FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanSaveWorld ) );
	ActionList.MapAction( Commands.SaveAs, FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SaveCurrentAs ), FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanSaveCurrentAs), FGetActionCheckState(), FIsActionButtonVisible::CreateStatic( &FLevelEditorActionCallbacks::CanSaveCurrentAs) );
	ActionList.MapAction( Commands.SaveAllLevels, FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SaveAllLevels ), FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanSaveUnpartitionedWorld), FGetActionCheckState(), FIsActionButtonVisible::CreateStatic(&FLevelEditorActionCallbacks::CanSaveUnpartitionedWorld) );
	ActionList.MapAction( Commands.BrowseLevel, FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Browse ), FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanBrowse ) );
	ActionList.MapAction( Commands.ToggleFavorite, FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ToggleFavorite ), FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ToggleFavorite_CanExecute ), FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::ToggleFavorite_IsChecked ) );

	for( int32 CurRecentIndex = 0; CurRecentIndex < FLevelEditorCommands::MaxRecentFiles; ++CurRecentIndex )
	{
		ActionList.MapAction( Commands.OpenRecentFileCommands[ CurRecentIndex ], FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OpenRecentFile, CurRecentIndex ), DefaultExecuteAction );
	}

	ActionList.MapAction( Commands.ClearRecentFiles, FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ClearRecentFiles) );

	for (int32 CurFavoriteIndex = 0; CurFavoriteIndex < FLevelEditorCommands::MaxFavoriteFiles; ++CurFavoriteIndex)
	{
		ActionList.MapAction(Commands.OpenFavoriteFileCommands[CurFavoriteIndex], FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::OpenFavoriteFile, CurFavoriteIndex), DefaultExecuteAction);
	}

	ActionList.MapAction( 
		Commands.ToggleVR, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ToggleVR ), 
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ToggleVR_IsButtonActive ),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::ToggleVR_IsChecked ) );

	ActionList.MapAction(Commands.ImportScene,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ImportScene_Clicked));

	ActionList.MapAction( Commands.ExportAll,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExportAll_Clicked ) );

	ActionList.MapAction( Commands.ExportSelected,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExportSelected_Clicked ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExportSelected_CanExecute ) );

	ActionList.MapAction( Commands.Build,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Build_Execute ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Build_CanExecute ) );

	
	ActionList.MapAction(Commands.RecompileGameCode,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::RecompileGameCode_Clicked ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Recompile_CanExecute )
		);

#if WITH_LIVE_CODING
	ActionList.MapAction( 
		Commands.LiveCoding_Enable, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LiveCoding_ToggleEnabled ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::LiveCoding_IsEnabled ) );

	ActionList.MapAction( 
		Commands.LiveCoding_StartSession, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LiveCoding_StartSession_Clicked ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LiveCoding_CanStartSession ) );

	ActionList.MapAction( 
		Commands.LiveCoding_ShowConsole, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LiveCoding_ShowConsole_Clicked ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LiveCoding_CanShowConsole ) );

	ActionList.MapAction( 
		Commands.LiveCoding_Settings, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LiveCoding_Settings_Clicked ),
		FCanExecuteAction());

#endif

	ActionList.MapAction( 
		FGlobalEditorCommonCommands::Get().FindInContentBrowser, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::FindInContentBrowser_Clicked ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::FindInContentBrowser_CanExecute )
		);

	ActionList.MapAction(
		Commands.GoHere,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::GoHere_Clicked, (const FVector*)nullptr )
		);

	ActionList.MapAction(
		Commands.PlayFromHere,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::PlayFromHere)
	);

	ActionList.MapAction( 
		Commands.SnapCameraToObject,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("CAMERA SNAP") ) )
		);

	ActionList.MapAction(
		Commands.SnapObjectToCamera,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapObjectToView_Clicked)
		);

	ActionList.MapAction(
		Commands.CopyActorFilePathtoClipboard,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::CopyActorFilePathtoClipboard_Clicked)
		);

	ActionList.MapAction(
		Commands.OpenActorInReferenceViewer,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ViewActorReferences_Clicked),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ViewActorReferences_CanExecute)
	);

	ActionList.MapAction(
		Commands.SaveActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SaveActor_Clicked),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SaveActor_CanExecute)
		);

	ActionList.MapAction(
		Commands.ShowActorHistory,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ShowActorHistory_Clicked),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ShowActorHistory_CanExecute)
	);

	ActionList.MapAction(
		Commands.GoToCodeForActor, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::GoToCodeForActor_Clicked ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::GoToCodeForActor_CanExecute ),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FLevelEditorActionCallbacks::GoToCodeForActor_IsVisible)
		);

	ActionList.MapAction( 
		Commands.GoToDocsForActor, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::GoToDocsForActor_Clicked )
		);

	ActionList.MapAction( 
		FGenericCommands::Get().Duplicate, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("DUPLICATE") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Duplicate_CanExecute )
		);

	ActionList.MapAction( 
		FGenericCommands::Get().Delete, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("DELETE") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Delete_CanExecute )
		);

	ActionList.MapAction( 
		FGenericCommands::Get().Rename, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Rename_Execute ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Rename_CanExecute )
		);

	ActionList.MapAction( 
		FGenericCommands::Get().Cut, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("EDIT CUT") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Cut_CanExecute )
		);

	ActionList.MapAction( 
		FGenericCommands::Get().Copy, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("EDIT COPY") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Copy_CanExecute )
		);

	ActionList.MapAction( 
		FGenericCommands::Get().Paste, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("EDIT PASTE") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Paste_CanExecute )
		);

	ActionList.MapAction( 
		Commands.PasteHere, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("EDIT PASTE TO=HERE") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::PasteHere_CanExecute )
		);

	ActionList.MapAction(
		Commands.SnapOriginToGrid,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::MoveElementsToGrid_Clicked, /*bAlign*/false, /*bPerElement*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.SnapOriginToGridPerActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::MoveElementsToGrid_Clicked, /*bAlign*/false, /*bPerElement*/true),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove)
		);
	
	ActionList.MapAction(
		Commands.AlignOriginToGrid,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::MoveElementsToGrid_Clicked, /*bAlign*/true, /*bPerElement*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.SnapOriginToActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::MoveElementsToElement_Clicked, /*bAlign*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementsSelected_CanExecuteMove)
		);
	
	ActionList.MapAction(
		Commands.AlignOriginToActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::MoveElementsToElement_Clicked, /*bAlign*/true),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementsSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.SnapTo2DLayer,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapTo2DLayer_Clicked),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::CanSnapTo2DLayer)
		);

	ActionList.MapAction(
		Commands.MoveSelectionUpIn2DLayers,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::MoveSelectionToDifferent2DLayer_Clicked, /*bGoingUp=*/ true, /*bForceToTopOrBottom=*/ false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::CanMoveSelectionToDifferent2DLayer, /*bGoingUp=*/ true)
		);
	ActionList.MapAction(
		Commands.MoveSelectionDownIn2DLayers,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::MoveSelectionToDifferent2DLayer_Clicked, /*bGoingUp=*/ false, /*bForceToTopOrBottom=*/ false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::CanMoveSelectionToDifferent2DLayer, /*bGoingUp=*/ false)
		);
	ActionList.MapAction(
		Commands.MoveSelectionToTop2DLayer,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::MoveSelectionToDifferent2DLayer_Clicked, /*bGoingUp=*/ true, /*bForceToTopOrBottom=*/ true),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::CanMoveSelectionToDifferent2DLayer, /*bGoingUp=*/ true)
		);
	ActionList.MapAction(
		Commands.MoveSelectionToBottom2DLayer,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::MoveSelectionToDifferent2DLayer_Clicked, /*bGoingUp=*/ false, /*bForceToTopOrBottom=*/ true),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::CanMoveSelectionToDifferent2DLayer, /*bGoingUp=*/ false)
		);


	ActionList.MapAction(
		Commands.Select2DLayerAbove,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Select2DLayerDeltaAway_Clicked, -1)
		);
	ActionList.MapAction(
		Commands.Select2DLayerBelow,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Select2DLayerDeltaAway_Clicked, 1)
		);
	ActionList.MapAction(
		Commands.AlignBrushVerticesToGrid,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::AlignBrushVerticesToGrid_Execute)
		);

	ActionList.MapAction(
		Commands.SnapToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, /*bAlign*/false, /*bUseLineTrace*/false, /*bUseBounds*/false, /*bUsePivot*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.AlignToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, /*bAlign*/true, /*bUseLineTrace*/false, /*bUseBounds*/false, /*bUsePivot*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.SnapPivotToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, /*bAlign*/false, /*bUseLineTrace*/true, /*bUseBounds*/false, /*bUsePivot*/true),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.AlignPivotToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, /*bAlign*/true, /*bUseLineTrace*/true, /*bUseBounds*/false, /*bUsePivot*/true),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.SnapBottomCenterBoundsToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, /*bAlign*/false, /*bUseLineTrace*/true, /*bUseBounds*/true, /*bUsePivot*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.AlignBottomCenterBoundsToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, /*bAlign*/true, /*bUseLineTrace*/true, /*bUseBounds*/true, /*bUsePivot*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.SnapToActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapElementsToElement_Clicked, /*bAlign*/false, /*bUseLineTrace*/false, /*bUseBounds*/false, /*bUsePivot*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementsSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.AlignToActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapElementsToElement_Clicked, /*bAlign*/true, /*bUseLineTrace*/false, /*bUseBounds*/false, /*bUsePivot*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementsSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.SnapPivotToActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapElementsToElement_Clicked, /*bAlign*/false, /*bUseLineTrace*/true, /*bUseBounds*/false, /*bUsePivot*/true),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementsSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.AlignPivotToActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapElementsToElement_Clicked, /*bAlign*/true, /*bUseLineTrace*/true, /*bUseBounds*/false, /*bUsePivot*/true),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementsSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.SnapBottomCenterBoundsToActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapElementsToElement_Clicked, /*bAlign*/false, /*bUseLineTrace*/true, /*bUseBounds*/true, /*bUsePivot*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementsSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.AlignBottomCenterBoundsToActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapElementsToElement_Clicked, /*bAlign*/true, /*bUseLineTrace*/true, /*bUseBounds*/true, /*bUsePivot*/false),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementsSelected_CanExecuteMove)
		);

	ActionList.MapAction(
		Commands.DeltaTransformToActors, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::DeltaTransform ), 
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove) );

	ActionList.MapAction(
		Commands.MirrorActorX,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ELEMENT MIRROR X=-1") ) ),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteScale)
		);

	ActionList.MapAction(
		Commands.MirrorActorY,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ELEMENT MIRROR Y=-1") ) ),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteScale)
		);

	ActionList.MapAction(
		Commands.MirrorActorZ,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ELEMENT MIRROR Z=-1") ) ),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ElementSelected_CanExecuteScale)
		);

	ActionList.MapAction(
		Commands.DetachFromParent,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::DetachActor_Clicked ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::DetachActor_CanExecute )
		);

	ActionList.MapAction(
		Commands.AttachSelectedActors,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::AttachSelectedActors )
		);

	ActionList.MapAction(
		Commands.AttachActorIteractive,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::AttachActorIteractive )
		);

	ActionList.MapAction(
		Commands.CreateNewOutlinerFolder,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CreateNewOutlinerFolder_Clicked )
		);

	ActionList.MapAction(
		Commands.LockActorMovement,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LockActorMovement_Clicked ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::LockActorMovement_IsChecked )
		);

	ActionList.MapAction(
		Commands.RegroupActors,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::RegroupActor_Clicked ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::GroupActors_CanExecute )
		);

	ActionList.MapAction(
		Commands.UngroupActors,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::UngroupActor_Clicked )
		);

	ActionList.MapAction(
		Commands.LockGroup,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LockGroup_Clicked )
		);

	ActionList.MapAction(
		Commands.UnlockGroup,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::UnlockGroup_Clicked )
		);

	ActionList.MapAction(
		Commands.AddActorsToGroup,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::AddActorsToGroup_Clicked ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::GroupActors_CanExecute )
		);

	ActionList.MapAction(
		Commands.RemoveActorsFromGroup,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::RemoveActorsFromGroup_Clicked )
		);
	
	ActionList.MapAction(
		Commands.ShowAll,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR UNHIDE ALL") ) ) 
		);

	ActionList.MapAction(
		Commands.ShowSelectedOnly,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnShowOnlySelectedActors )
		);

	ActionList.MapAction(
		Commands.ShowSelected,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR UNHIDE SELECTED") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanShowSelectedActors )
		);

	ActionList.MapAction(
		Commands.HideSelected,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR HIDE SELECTED") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanHideSelectedActors )
		);
		
	ActionList.MapAction(
		Commands.ToggleSelectedVisibility,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR TOGGLE SELECTED VISIBILITY") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanToggleSelectedActorsVisibility )
		);

	ActionList.MapAction(
		Commands.ShowSelectedHierarchy,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR UNHIDE SELECTED HIERARCHY") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanShowSelectedActors )
		);

	ActionList.MapAction(
		Commands.HideSelectedHierarchy,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR HIDE SELECTED HIERARCHY") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanHideSelectedActors )
		);

	ActionList.MapAction(
		Commands.ToggleSelectedHierarchyVisibility,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR TOGGLE SELECTED VISIBILITY HIERARCHY") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanToggleSelectedActorsVisibility )
		);

	ActionList.MapAction(
		Commands.ShowAllStartup,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR UNHIDE ALL STARTUP") ) )
		);

	ActionList.MapAction(
		Commands.ShowSelectedStartup,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR UNHIDE SELECTED STARTUP") ) )
		);

	ActionList.MapAction(
		Commands.HideSelectedStartup,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR HIDE SELECTED STARTUP") ) )
		);

	ActionList.MapAction(
		Commands.ShowSelectedHierarchyStartup,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR UNHIDE SELECTED STARTUP HIERARCHY") ) )
		);
		
	ActionList.MapAction(
        Commands.HideSelectedHierarchyStartup,
        FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR HIDE SELECTED STARTUP HIERARCHY") ) )
        );

	ActionList.MapAction(
		Commands.CycleNavigationDataDrawn,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("CYCLENAVDRAWN") ) )
		);

	ActionList.MapAction(
		FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT ALL") ) )
		);

	ActionList.MapAction(
		Commands.SelectNone,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("SELECT NONE") ) )
		);

	ActionList.MapAction(
		Commands.InvertSelection,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT INVERT") ) )
		);

	ActionList.MapAction(
		Commands.SelectImmediateChildren,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT ALL CHILDREN") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorSelected_CanExecute )
	);

	ActionList.MapAction(
		Commands.SelectAllDescendants,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT ALL DESCENDANTS") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorSelected_CanExecute )
	);

	ActionList.MapAction(
		Commands.SelectAllActorsOfSameClass,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSelectAllActorsOfClass, false ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanSelectAllActorsOfClass )
		);

	ActionList.MapAction(
		Commands.SelectAllActorsOfSameClassWithArchetype,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSelectAllActorsOfClass, true ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanSelectAllActorsOfClass )
		);

	ActionList.MapAction(
		Commands.SelectComponentOwnerActor,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSelectComponentOwnerActor ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanSelectComponentOwnerActor )
		);

	ActionList.MapAction(
		Commands.SelectRelevantLights,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT RELEVANTLIGHTS") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorSelected_CanExecute )
		);

	ActionList.MapAction(
		Commands.SelectStaticMeshesOfSameClass,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT MATCHINGSTATICMESH") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorSelected_CanExecute )
		);

	ActionList.MapAction(
		Commands.SelectStaticMeshesAllClasses,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT MATCHINGSTATICMESH ALLCLASSES") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorSelected_CanExecute )
		);

	ActionList.MapAction(
		Commands.SelectOwningHierarchicalLODCluster,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::OnSelectOwningHLODCluster),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorTypesSelected_CanExecute, FLevelEditorActionCallbacks::IncludeStaticMeshes, /*bSingleOnly*/ true )
		);

	FLevelEditorActionCallbacks::EActorTypeFlags IncludePawnsAndSkeletalMeshes = static_cast<FLevelEditorActionCallbacks::EActorTypeFlags>(FLevelEditorActionCallbacks::IncludePawns | FLevelEditorActionCallbacks::IncludeSkeletalMeshes);
	ActionList.MapAction(
		Commands.SelectSkeletalMeshesOfSameClass,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT MATCHINGSKELETALMESH") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorTypesSelected_CanExecute, IncludePawnsAndSkeletalMeshes, /*bSingleOnly*/ false )
		);

	ActionList.MapAction(
		Commands.SelectSkeletalMeshesAllClasses,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT MATCHINGSKELETALMESH ALLCLASSES") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorTypesSelected_CanExecute, IncludePawnsAndSkeletalMeshes, /*bSingleOnly*/ false )
		);

	ActionList.MapAction(
		Commands.SelectAllWithSameMaterial,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT MATCHINGMATERIAL") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorSelected_CanExecute )
		);

	ActionList.MapAction(
		Commands.SelectMatchingEmitter,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR SELECT MATCHINGEMITTER") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ActorTypesSelected_CanExecute, FLevelEditorActionCallbacks::IncludeEmitters, /*bSingleOnly*/ false )
		);

	ActionList.MapAction(
		Commands.SelectAllLights,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSelectAllLights )
		);

	ActionList.MapAction(
		Commands.SelectStationaryLightsExceedingOverlap,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSelectStationaryLightsExceedingOverlap )
		);

	ActionList.MapAction(
		Commands.SelectAllAddditiveBrushes,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("MAP SELECT ADDS") ) )
		);

	ActionList.MapAction(
		Commands.SelectAllSubtractiveBrushes,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("MAP SELECT SUBTRACTS") ) )
		);

	ActionList.MapAction(
		Commands.SelectAllSurfaces,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT ALL") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectAllMatchingBrush,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT MATCHING BRUSH") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectAllMatchingTexture,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT MATCHING TEXTURE") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectAllAdjacents,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT ADJACENT ALL") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectAllAdjacentCoplanars,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT ADJACENT COPLANARS") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectAllAdjacentWalls,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT ADJACENT WALLS") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectAllAdjacentFloors,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT ADJACENT FLOORS") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectAllAdjacentSlants,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT ADJACENT SLANTS") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectReverse,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT REVERSE") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectMemorize,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT MEMORY SET") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectRecall,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT MEMORY RECALL") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectOr,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT MEMORY INTERSECTION") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectAnd,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT MEMORY UNION") ) )
		);

	ActionList.MapAction( 
		Commands.SurfSelectXor,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("POLY SELECT MEMORY XOR") ) )
		);

	ActionList.MapAction( 
		Commands.SurfUnalign,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSurfaceAlignment, TEXALIGN_Default )
		);

	ActionList.MapAction( 
		Commands.SurfAlignPlanarAuto,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSurfaceAlignment, TEXALIGN_PlanarAuto )
		);

	ActionList.MapAction( 
		Commands.SurfAlignPlanarWall,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSurfaceAlignment, TEXALIGN_PlanarWall )
		);

	ActionList.MapAction( 
		Commands.SurfAlignPlanarFloor,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSurfaceAlignment, TEXALIGN_PlanarFloor )
		);

	ActionList.MapAction( 
		Commands.SurfAlignBox,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSurfaceAlignment, TEXALIGN_Box )
		);

	ActionList.MapAction(
		Commands.SurfAlignFit,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSurfaceAlignment, TEXALIGN_Fit )
		);


	ActionList.MapAction(
		Commands.ApplyMaterialToSurface,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnApplyMaterialToSurface )
		);

	ActionList.MapAction(
		Commands.SavePivotToPrePivot,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR BAKEPREPIVOT") ) )
		);

	ActionList.MapAction(
		Commands.ResetPivot,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR RESET PIVOT") ) )
		);

	ActionList.MapAction(
		Commands.ResetPrePivot,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR UNBAKEPREPIVOT") ) )
		);

	ActionList.MapAction(
		Commands.MovePivotHere,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("PIVOT HERE") ) )
		);

	ActionList.MapAction(
		Commands.MovePivotHereSnapped,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("PIVOT SNAPPED") ) )
		);

	ActionList.MapAction(
		Commands.MovePivotToCenter,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("PIVOT CENTERSELECTION") ) )
		);

	ActionList.MapAction(
		Commands.ConvertToAdditive,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString::Printf(TEXT("MAP SETBRUSH BRUSHTYPE=%d"), (int32)Brush_Add) )
		);

	ActionList.MapAction(
		Commands.ConvertToSubtractive,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString::Printf(TEXT("MAP SETBRUSH BRUSHTYPE=%d"), (int32)Brush_Subtract) )
		);

	ActionList.MapAction(
		Commands.OrderFirst,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("MAP SENDTO FIRST") ) )
		);

	ActionList.MapAction(
		Commands.OrderLast,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("MAP SENDTO LAST") ) )
		);

	ActionList.MapAction(
		Commands.MakeSolid,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString::Printf( TEXT("MAP SETBRUSH CLEARFLAGS=%d SETFLAGS=%d"), PF_Semisolid + PF_NotSolid, 0 ) )
		);

	ActionList.MapAction(
		Commands.MakeSemiSolid,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString::Printf( TEXT("MAP SETBRUSH CLEARFLAGS=%d SETFLAGS=%d"), (int32)(PF_Semisolid + PF_NotSolid), (int32)PF_Semisolid ) )
		);

	ActionList.MapAction(
		Commands.MakeNonSolid,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString::Printf( TEXT("MAP SETBRUSH CLEARFLAGS=%d SETFLAGS=%d"), (int32)(PF_Semisolid + PF_NotSolid), (int32)PF_NotSolid ) )
		);

	ActionList.MapAction(
		Commands.MergePolys,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("BRUSH MERGEPOLYS") ) )
		);

	ActionList.MapAction(
		Commands.SeparatePolys,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("BRUSH SEPARATEPOLYS") ) )
		);


	ActionList.MapAction(
		Commands.CreateBoundingBoxVolume,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR CREATE_BV_BOUNDINGBOX SnapToGrid=1") ) )
		);


	ActionList.MapAction(
		Commands.CreateHeavyConvexVolume,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR CREATE_BV_CONVEXVOLUME NORMALTOLERANCE=0.01 SnapToGrid=1") ) )
		);

	ActionList.MapAction(
		Commands.CreateNormalConvexVolume,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR CREATE_BV_CONVEXVOLUME NORMALTOLERANCE=0.15 SnapToGrid=1") ) )
		);

	ActionList.MapAction(
		Commands.CreateLightConvexVolume,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR CREATE_BV_CONVEXVOLUME NORMALTOLERANCE=.5 SnapToGrid=1") ) )
		);

	ActionList.MapAction(
		Commands.CreateRoughConvexVolume,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("ACTOR CREATE_BV_CONVEXVOLUME NORMALTOLERANCE=0.75 SnapToGrid=1") ) )
		);

	ActionList.MapAction(
		Commands.KeepSimulationChanges,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnKeepSimulationChanges ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanExecuteKeepSimulationChanges )
		);


	ActionList.MapAction( 
		Commands.MakeActorLevelCurrent,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnMakeSelectedActorLevelCurrent )
		);

	ActionList.MapAction(
		Commands.MoveSelectedToCurrentLevel,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnMoveSelectedToCurrentLevel )
		);

	ActionList.MapAction(
		Commands.FindActorLevelInContentBrowser,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::OnFindActorLevelInContentBrowser),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::CanExecuteFindActorLevelInContentBrowser)
		);

	ActionList.MapAction(
		Commands.FindLevelsInLevelBrowser,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnFindLevelsInLevelBrowser )
		);

	ActionList.MapAction(
		Commands.AddLevelsToSelection,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnSelectLevelInLevelBrowser )
		);

	ActionList.MapAction(
		Commands.RemoveLevelsFromSelection,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnDeselectLevelInLevelBrowser )
		);

	ActionList.MapAction(
		Commands.FindActorInLevelScript,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnFindActorInLevelScript )
		);

	ActionList.MapAction( Commands.BuildAndSubmitToSourceControl,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BuildAndSubmitToSourceControl_Execute ) );

	ActionList.MapAction( Commands.BuildLightingOnly,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BuildLightingOnly_Execute ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BuildLighting_CanExecute ) );

	ActionList.MapAction( Commands.BuildReflectionCapturesOnly,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BuildReflectionCapturesOnly_Execute ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BuildReflectionCapturesOnly_CanExecute )  );

	ActionList.MapAction( Commands.BuildLightingOnly_VisibilityOnly,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BuildLightingOnly_VisibilityOnly_Execute ) );

	ActionList.MapAction( Commands.LightingBuildOptions_UseErrorColoring,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LightingBuildOptions_UseErrorColoring_Toggled ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::LightingBuildOptions_UseErrorColoring_IsChecked ) );

	ActionList.MapAction( Commands.LightingBuildOptions_ShowLightingStats,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::LightingBuildOptions_ShowLightingStats_Toggled ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::LightingBuildOptions_ShowLightingStats_IsChecked ) );

	ActionList.MapAction( Commands.BuildGeometryOnly,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BuildGeometryOnly_Execute ) );

	ActionList.MapAction( Commands.BuildGeometryOnly_OnlyCurrentLevel,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BuildGeometryOnly_OnlyCurrentLevel_Execute ) );

	ActionList.MapAction( Commands.BuildPathsOnly,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::BuildPathsOnly_Execute ) );

	ActionList.MapAction(Commands.BuildHLODs,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::BuildHLODs_Execute),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::IsWorldPartitionStreamingEnabled));

	ActionList.MapAction(Commands.BuildHLODs_SelectedActors,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::BuildHLODs_SelectedActors_Execute),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::IsWorldPartitionStreamingEnabled));
	
	ActionList.MapAction(Commands.BuildMinimap,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::BuildMinimap_Execute),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::IsWorldPartitionEnabled));

	ActionList.MapAction(Commands.BuildLandscapeSplineMeshes,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::BuildLandscapeSplineMeshes_Execute),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::IsWorldPartitionStreamingEnabled));

	ActionList.MapAction(Commands.BuildTextureStreamingOnly,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::BuildTextureStreamingOnly_Execute));

	ActionList.MapAction(Commands.BuildVirtualTextureOnly,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::BuildVirtualTextureOnly_Execute));

	ActionList.MapAction(Commands.BuildAllLandscape,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::BuildAllLandscape_Execute));

	for (int32 Index = 0; Index < FLevelEditorCommands::MaxExternalBuildTypes; ++Index)
	{
		ActionList.MapAction(
			Commands.ExternalBuildTypeCommands[Index],
			FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::BuildExternalType_Execute, Index),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::BuildExternalType_CanExecute, Index));
	}

	ActionList.MapAction( 
		Commands.LightingQuality_Production, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetLightingQuality, (ELightingBuildQuality)Quality_Production ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsLightingQualityChecked, (ELightingBuildQuality)Quality_Production ) );
	ActionList.MapAction( 
		Commands.LightingQuality_High, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetLightingQuality, (ELightingBuildQuality)Quality_High ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsLightingQualityChecked, (ELightingBuildQuality)Quality_High ) );
	ActionList.MapAction( 
		Commands.LightingQuality_Medium, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetLightingQuality, (ELightingBuildQuality)Quality_Medium ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsLightingQualityChecked, (ELightingBuildQuality)Quality_Medium ) );
	ActionList.MapAction( 
		Commands.LightingQuality_Preview, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetLightingQuality, (ELightingBuildQuality)Quality_Preview ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsLightingQualityChecked, (ELightingBuildQuality)Quality_Preview) );

	ActionList.MapAction( 
		Commands.LightingDensity_RenderGrayscale, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetLightingDensityRenderGrayscale ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsLightingDensityRenderGrayscaleChecked ) );

	ActionList.MapAction( 
		Commands.LightingResolution_CurrentLevel, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetLightingResolutionLevel, (FLightmapResRatioAdjustSettings::AdjustLevels)FLightmapResRatioAdjustSettings::Current ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsLightingResolutionLevelChecked, (FLightmapResRatioAdjustSettings::AdjustLevels)FLightmapResRatioAdjustSettings::Current ) );
	ActionList.MapAction( 
		Commands.LightingResolution_SelectedLevels, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetLightingResolutionLevel, (FLightmapResRatioAdjustSettings::AdjustLevels)FLightmapResRatioAdjustSettings::Selected ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsLightingResolutionLevelChecked, (FLightmapResRatioAdjustSettings::AdjustLevels)FLightmapResRatioAdjustSettings::Selected ) );
	ActionList.MapAction( 
		Commands.LightingResolution_AllLoadedLevels, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetLightingResolutionLevel, (FLightmapResRatioAdjustSettings::AdjustLevels)FLightmapResRatioAdjustSettings::AllLoaded ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsLightingResolutionLevelChecked, (FLightmapResRatioAdjustSettings::AdjustLevels)FLightmapResRatioAdjustSettings::AllLoaded ) );
	ActionList.MapAction( 
		Commands.LightingResolution_SelectedObjectsOnly, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetLightingResolutionSelectedObjectsOnly ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsLightingResolutionSelectedObjectsOnlyChecked ) );

	ActionList.MapAction( 
		Commands.LightingStaticMeshInfo, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ShowLightingStaticMeshInfo ) );

	ActionList.MapAction( 
		Commands.SceneStats,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ShowSceneStats ) );

	ActionList.MapAction( 
		Commands.TextureStats,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ShowTextureStats ) );

	ActionList.MapAction( 
		Commands.MapCheck,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::MapCheck_Execute ) );

	ActionList.MapAction(
		Commands.ShowTransformWidget,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleTransformWidgetVisibility ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnGetTransformWidgetVisibility )
		);

	ActionList.MapAction(
		Commands.ShowSelectionSubcomponents,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::OnToggleShowSelectionSubcomponents),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::OnGetShowSelectionSubcomponents)
	);

	ActionList.MapAction(
		Commands.AllowTranslucentSelection,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnAllowTranslucentSelection ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsAllowTranslucentSelectionEnabled ) 
		);


	ActionList.MapAction(
		Commands.AllowGroupSelection,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnAllowGroupSelection ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsAllowGroupSelectionEnabled ) 
		);

	ActionList.MapAction(
		Commands.StrictBoxSelect,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleStrictBoxSelect ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsStrictBoxSelectEnabled ) 
		);

	ActionList.MapAction(
		Commands.TransparentBoxSelect,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleTransparentBoxSelect ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsTransparentBoxSelectEnabled ) 
		);

	ActionList.MapAction(
		Commands.DrawBrushMarkerPolys,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnDrawBrushMarkerPolys ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsDrawBrushMarkerPolysEnabled ) 
		);

	ActionList.MapAction(
		Commands.OnlyLoadVisibleInPIE,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleOnlyLoadVisibleInPIE ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsOnlyLoadVisibleInPIEEnabled ) 
		);

	ActionList.MapAction(
		Commands.ToggleSocketSnapping,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleSocketSnapping ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsSocketSnappingEnabled ) 
		);

	ActionList.MapAction(
		Commands.ToggleParticleSystemLOD,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleParticleSystemLOD ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsParticleSystemLODEnabled ) 
		);

	ActionList.MapAction(
		Commands.ToggleFreezeParticleSimulation,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleFreezeParticleSimulation ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsParticleSimulationFrozen )
		);

	ActionList.MapAction(
		Commands.ToggleParticleSystemHelpers,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleParticleSystemHelpers ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsParticleSystemHelpersEnabled ) 
		);

	ActionList.MapAction(
		Commands.ToggleLODViewLocking,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleLODViewLocking ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsLODViewLockingEnabled ) 
		);

	ActionList.MapAction(
		Commands.LevelStreamingVolumePrevis,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleLevelStreamingVolumePrevis ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsLevelStreamingVolumePrevisEnabled ) 
		);

	ActionList.MapAction(
		Commands.EnableActorSnap,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnEnableActorSnap ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsActorSnapEnabled ) 
		);

	ActionList.MapAction(
		Commands.EnableVertexSnap,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnEnableVertexSnap ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::OnIsVertexSnapEnabled ) 
		);

	ActionList.MapAction(
		Commands.ShowSelectedDetails,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("EDCALLBACK SELECTEDPROPS" ) ) ) 
		);

	//if (FParse::Param( FCommandLine::Get(), TEXT( "editortoolbox" ) ))
	//{
	//	ActionList.MapAction(
	//		Commands.BspMode,
	//		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("MODE BSP") ) ),
	//		FCanExecuteAction(),
	//		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsEditorModeActive, FBuiltinEditorModes::EM_Bsp ) 
	//		);

	//	ActionList.MapAction(
	//		Commands.MeshPaintMode,
	//		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("MODE MESHPAINT") ) ),
	//		FCanExecuteAction(),
	//		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsEditorModeActive, FBuiltinEditorModes::EM_MeshPaint ) 
	//		);

	//	ActionList.MapAction(
	//		Commands.LandscapeMode,
	//		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("MODE LANDSCAPE") ) ),
	//		FCanExecuteAction(),
	//		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsEditorModeActive, FBuiltinEditorModes::EM_Landscape ) 
	//		);

	//	ActionList.MapAction(
	//		Commands.FoliageMode,
	//		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("MODE FOLIAGE") ) ),
	//		FCanExecuteAction(),
	//		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsEditorModeActive, FBuiltinEditorModes::EM_Foliage ) 
	//		);
	//}

	ActionList.MapAction(
		Commands.RecompileShaders,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("RECOMPILESHADERS CHANGED" ) ) )
		);

	ActionList.MapAction(
		Commands.ProfileGPU,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("PROFILEGPU") ) )
		);
	
	ActionList.MapAction(
		Commands.DumpGPU,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("DUMPGPU") ) )
		);

	ActionList.MapAction(
		Commands.ResetAllParticleSystems,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("PARTICLE RESET ALL") ) )
		);

	ActionList.MapAction(
		Commands.ResetSelectedParticleSystem,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("PARTICLE RESET SELECTED") ) )
		);

	ActionList.MapAction(
		FEditorViewportCommands::Get().LocationGridSnap,
		FExecuteAction::CreateStatic(FLevelEditorActionCallbacks::LocationGridSnap_Clicked),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(FLevelEditorActionCallbacks::LocationGridSnap_IsChecked)
		);
	ActionList.MapAction(
		FEditorViewportCommands::Get().RotationGridSnap,
		FExecuteAction::CreateStatic(FLevelEditorActionCallbacks::RotationGridSnap_Clicked),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(FLevelEditorActionCallbacks::RotationGridSnap_IsChecked)
		);
	ActionList.MapAction(
		FEditorViewportCommands::Get().ScaleGridSnap,
		FExecuteAction::CreateStatic(FLevelEditorActionCallbacks::ScaleGridSnap_Clicked),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(FLevelEditorActionCallbacks::ScaleGridSnap_IsChecked)
		);
	
	ActionList.MapAction(
		Commands.ToggleHideViewportUI,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleShowViewportUI ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsViewportUIVisible ) 
		);

	ActionList.MapAction( 
		Commands.MaterialQualityLevel_Low, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetMaterialQualityLevel, (EMaterialQualityLevel::Type)EMaterialQualityLevel::Low ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsMaterialQualityLevelChecked, (EMaterialQualityLevel::Type)EMaterialQualityLevel::Low ) );
	ActionList.MapAction(
		Commands.MaterialQualityLevel_Medium,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SetMaterialQualityLevel, (EMaterialQualityLevel::Type)EMaterialQualityLevel::Medium),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsMaterialQualityLevelChecked, (EMaterialQualityLevel::Type)EMaterialQualityLevel::Medium));
	ActionList.MapAction( 
		Commands.MaterialQualityLevel_High, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::SetMaterialQualityLevel, (EMaterialQualityLevel::Type)EMaterialQualityLevel::High ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsMaterialQualityLevelChecked, (EMaterialQualityLevel::Type)EMaterialQualityLevel::High ) );
	ActionList.MapAction(
		Commands.MaterialQualityLevel_Epic,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SetMaterialQualityLevel, (EMaterialQualityLevel::Type)EMaterialQualityLevel::Epic),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsMaterialQualityLevelChecked, (EMaterialQualityLevel::Type)EMaterialQualityLevel::Epic));

	ActionList.MapAction(
		Commands.TogglePreviewPlatform,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::TogglePreviewPlatform),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsPreviewPlatformEnabled),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsPreviewPlatformActive),
		FIsActionButtonVisible::CreateStatic(FLevelEditorActionCallbacks::IsPreviewModeButtonVisible));

	ActionList.MapAction(
		Commands.OrbitCameraAroundSelection,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ToggleOrbitCameraAroundSelection),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsOrbitCameraAroundSelectionChecked));

	ActionList.MapAction(
		Commands.LinkOrthographicViewports,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ToggleLinkOrthographicViewports),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsLinkOrthographicViewportsChecked));

	ActionList.MapAction(
		Commands.OrthoZoomToCursor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ToggleOrthoZoomToCursor),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsOrthoZoomToCursorChecked));

	ActionList.MapAction(
		Commands.InvertMiddleMousePan,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ToggleInvertMiddleMousePan),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsInvertMiddleMousePanChecked));
	ActionList.MapAction(
		Commands.InvertOrbitYAxis,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ToggleInvertOrbitYAxis),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsInvertOrbitYAxisChecked));
	ActionList.MapAction(
		Commands.InvertRightMouseDollyYAxis,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ToggleInvertRightMouseDollyYAxis),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsInvertRightMouseDollyYAxisChecked));

	ActionList.MapAction(
		Commands.AllowArcballRotation,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ToggleAllowArcballRotation),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsAllowArcballRotationChecked)
	);

	ActionList.MapAction(
		Commands.AllowScreenspaceRotation,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ToggleAllowScreenspaceRotation),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsAllowScreenspaceRotationChecked)
	);

	ActionList.MapAction(
		Commands.EnableViewportHoverFeedback,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ToggleEnableViewportHoverFeedback),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsEnableViewportHoverFeedbackChecked)
	);

	const TArray<FPreviewPlatformMenuItem>& MenuItems = FDataDrivenPlatformInfoRegistry::GetAllPreviewPlatformMenuItems();
	// We need one extra slot for the Disable Preview option
	check(MenuItems.Num() == Commands.PreviewPlatformOverrides.Num());

	for (int32 Index=0; Index < MenuItems.Num(); Index++)
	{
		const FPreviewPlatformMenuItem& Item = MenuItems[Index];
		EShaderPlatform ShaderPlatform = FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(Item.PreviewShaderPlatformName);
		
		if (ShaderPlatform < SP_NumPlatforms)
		{
			const bool bIsDefaultShaderPlatform = FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(ShaderPlatform) == GMaxRHIShaderPlatform;

			auto GetPreviewFeatureLevelInfo = [&]()
			{
				if (bIsDefaultShaderPlatform)
				{
					return FPreviewPlatformInfo(GMaxRHIFeatureLevel, GMaxRHIShaderPlatform, NAME_None, NAME_None, NAME_None, false, NAME_None);
				}

				const ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);
				return FPreviewPlatformInfo(FeatureLevel, ShaderPlatform, Item.PlatformName, Item.ShaderFormat, Item.DeviceProfileName, true, Item.PreviewShaderPlatformName);
			};

			FPreviewPlatformInfo PreviewFeatureLevelInfo = GetPreviewFeatureLevelInfo();

			ActionList.MapAction(
				Commands.PreviewPlatformOverrides[Index],
				FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SetPreviewPlatform, PreviewFeatureLevelInfo),
				bIsDefaultShaderPlatform ? FCanExecuteAction() : FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::CanExecutePreviewPlatform, PreviewFeatureLevelInfo),
				FIsActionChecked::CreateStatic(&FLevelEditorActionCallbacks::IsPreviewPlatformChecked, PreviewFeatureLevelInfo));

			FConfigCacheIni* PlatformEngineIni = FConfigCacheIni::ForPlatform(*Item.PlatformName.ToString());
			FString DeviceProfileSelectionModule;
			if (PlatformEngineIni && PlatformEngineIni->GetString(TEXT("DeviceProfileManager"), TEXT("PreviewDeviceProfileSelectionModule"), DeviceProfileSelectionModule, GEngineIni))
			{
				const TArray<FLevelEditorCommands::PreviewPlatformCommand>* CommandListJson = Commands.PlatformToPreviewJsonPlatformOverrides.Find(Item.PlatformName);

				if (CommandListJson != nullptr)
				{
					for (const FLevelEditorCommands::PreviewPlatformCommand& PreviewJsonPlatform : *CommandListJson)
					{
						if (PreviewJsonPlatform.bIsGeneratingJsonCommand)
						{
							FUIAction Action;
							Action.ExecuteAction = FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::GeneratePreviewJson_Clicked, Item.PlatformName.ToString());
							Action.IsActionVisibleDelegate = FIsActionButtonVisible::CreateStatic(&FLevelEditorActionCallbacks::IsGeneratePreviewJsonVisible, Item.PlatformName);
							ActionList.MapAction(PreviewJsonPlatform.CommandInfo, Action);
						}
						else
						{
							FUIAction ActionJsonFile;
							ActionJsonFile.ExecuteAction = FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::PreviewJson_Clicked, Item.PlatformName, PreviewJsonPlatform.FilePath);
							ActionJsonFile.IsActionVisibleDelegate = FIsActionButtonVisible::CreateStatic(&FLevelEditorActionCallbacks::IsPreviewJsonVisible, Item.PlatformName);
							ActionList.MapAction(PreviewJsonPlatform.CommandInfo, ActionJsonFile);
						}
					}
				}
			}
		}
	}

	ActionList.MapAction(
		Commands.OpenMergeActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::OpenMergeActor_Clicked)
		);

	ActionList.MapAction(
		Commands.FixupGroupActor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::FixupGroupActor_Clicked)
	);
}

TSharedPtr<FLevelEditorOutlinerSettings> FLevelEditorModule::GetLevelEditorOutlinerSettings() const
{
	return OutlinerSettings;
}

void FLevelEditorModule::RefreshLevelEditorOutliners() const
{
	static const FName TabIDS[] = {
		LevelEditorTabIds::LevelEditorSceneOutliner,
		LevelEditorTabIds::LevelEditorSceneOutliner2,
		LevelEditorTabIds::LevelEditorSceneOutliner3,
		LevelEditorTabIds::LevelEditorSceneOutliner4 };

	if (LevelEditorTabManager)
	{
		for (const FName& TabID : TabIDS)
		{
			if (LevelEditorTabManager->FindExistingLiveTab(TabID).IsValid())
			{
				LevelEditorTabManager->TryInvokeTab(TabID)->RequestCloseTab();
				LevelEditorTabManager->TryInvokeTab(TabID);
			}
		}
	}
}

void FLevelEditorModule::AddCustomFilterToOutliner(TSharedRef<FFilterBase<const ISceneOutlinerTreeItem&>> InCustomFilter)
{
	// Disable deprecation warnings so we can call the deprecated function to support this function (which is also deprecated)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OutlinerSettings->AddCustomFilter(InCustomFilter);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FLevelEditorModule::AddCustomFilterToOutliner(FLevelEditorOutlinerSettings::FOutlinerFilterFactory InCreateCustomFilter)
{
	OutlinerSettings->AddCustomFilter(InCreateCustomFilter);
}

void FLevelEditorModule::AddCustomClassFilterToOutliner(TSharedRef<FCustomClassFilterData> InCustomClassFilterData)
{
	OutlinerSettings->AddCustomClassFilter(InCustomClassFilterData);
}

TSharedPtr<FFilterCategory> FLevelEditorModule::GetOutlinerFilterCategory(const FName& CategoryName) const
{
	return OutlinerSettings->GetFilterCategory(CategoryName);
}

#undef LOCTEXT_NAMESPACE
