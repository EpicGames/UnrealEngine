// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorActions.h"
#include "Algo/Unique.h"
#include "SceneView.h"
#include "Factories/Factory.h"
#include "Animation/AnimSequence.h"
#include "Components/LightComponent.h"
#include "DataStorage/Features.h"
#include "DataStorage/Handles.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h" // FTypedElementActorTag
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "TexAlignTools.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/Material.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Factories/BlueprintFactory.h"
#include "Editor/GroupActor.h"
#include "Materials/MaterialInstance.h"
#include "Engine/Light.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/DebuggerCommands.h"
#include "Engine/Selection.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectIterator.h"
#include "EngineUtils.h"
#include "EditorModes.h"
#include "UnrealEdMisc.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionEditorHash.h"
#include "WorldPartition/WorldPartitionHLODsBuilder.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/HLOD/HLODCreationFilter.h"
#include "WorldBrowserModule.h"
#include "ExternalPackageHelper.h"

#include "Elements/Framework/TypedElementCommonActions.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementDetailsInterface.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Subsystems/EditorElementSubsystem.h"

#include "LevelEditor.h"
#include "LevelEditorInternalTools.h"
#include "Engine/LevelScriptBlueprint.h"
#include "LightingBuildOptions.h"
#include "EditorSupportDelegates.h"
#include "SLevelEditor.h"
#include "EditorBuildUtils.h"
#include "ScopedTransaction.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "DlgDeltaTransform.h"
#include "NewLevelDialogModule.h"
#include "MRUFavoritesList.h"
#include "SSocketChooser.h"
#include "SnappingUtils.h"
#include "LevelEditorViewport.h"
#include "Layers/LayersSubsystem.h"
#include "IPlacementModeModule.h"
#include "AssetSelection.h"
#include "IDocumentation.h"
#include "SourceCodeNavigation.h"
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EditorClassUtils.h"

#include "EditorActorFolders.h"
#include "ActorPickerMode.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/HotReloadInterface.h"
#include "SourceControlWindows.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "CreateBlueprintFromActorDialog.h"
#include "Settings/EditorProjectSettings.h"
#include "Engine/LODActor.h"
#include "IHierarchicalLODUtilities.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "Application/IPortalApplicationWindow.h"
#include "IPortalServiceLocator.h"
#include "MaterialShaderQualitySettings.h"
#include "IVREditorModule.h"
#include "ComponentRecreateRenderStateContext.h"
#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LevelStreamingDynamic.h"
#include "EditorLevelUtils.h"
#include "ActorGroupingUtils.h"
#include "LevelUtils.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "ISettingsModule.h"
#include "PlatformInfo.h"
#include "Misc/CoreMisc.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/ScopeExit.h"
#include "Dialogs/DlgPickPath.h"
#include "AssetToolsModule.h"
#include "Editor/UnrealEdEngine.h"
#include "Preferences/UnrealEdOptions.h"
#include "UnrealEdGlobals.h"
#include "EditorModeManager.h"
#include "DataDrivenShaderPlatformInfo.h"

#include "Internationalization/Culture.h"
#include "Misc/FileHelper.h"
#include "EditorDirectories.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "IDeviceProfileSelectorModule.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"

#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/EditorActorSubsystem.h"

DEFINE_LOG_CATEGORY_STATIC(LevelEditorActions, Log, All);

#define LOCTEXT_NAMESPACE "LevelEditorActions"

const FName HotReloadModule("HotReload");

FLevelEditorActionCallbacks::FNewLevelOverride FLevelEditorActionCallbacks::NewLevelOverride;

namespace LevelEditorActionsHelpers
{
	/** 
	 * If the passed in class is generated by a Blueprint, it will open that Blueprint, otherwise it will help the user create a Blueprint based on that class
	 *
	 * @param InWindowTitle			The window title if the Blueprint needs to be created
	 * @param InBlueprintClass		The class to create a Blueprint based on or to open if it is a Blueprint
	 * @param InLevelEditor			When opening the Blueprint, this level editor is the parent window
	 * @param InNewBPName			If we have to create a new BP, this is the suggested name
	 */
	UBlueprint* OpenOrCreateBlueprintFromClass(FText InWindowTitle, UClass* InBlueprintClass, TWeakPtr< SLevelEditor > InLevelEditor, FString InNewBPName = TEXT(""))
	{
		UBlueprint* Blueprint = NULL;

		// If the current set class is not a Blueprint, we need to allow the user to create one to edit
		if(!InBlueprintClass->ClassGeneratedBy)
		{
			Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(InWindowTitle, InBlueprintClass, InNewBPName);
		}
		else
		{
			Blueprint = Cast<UBlueprint>(InBlueprintClass->ClassGeneratedBy);
		}

		if(Blueprint)
		{
			// @todo Re-enable once world centric works
			const bool bOpenWorldCentric = false;
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
				Blueprint,
				bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
				InLevelEditor.Pin()  );
		}

		return Blueprint;
	}

	/** Check to see whether this world is a persistent world with a valid file on disk */
	bool IsPersistentWorld(UWorld* InWorld)
	{
		UPackage* Pkg = InWorld ? InWorld->GetOutermost() : nullptr;
		if (Pkg && FPackageName::IsValidLongPackageName(Pkg->GetName()))
		{
			FString FileName;
			return FPackageName::DoesPackageExist(Pkg->GetName(), &FileName);
		}
		return false;
	}

	// Call VisitFn on each selected non-actor with a FVisibleInEditorColumn. Returns true if completed fully.
	bool ForEachSelectedHideableNonActor(UWorld* WorldIn,
		TFunctionRef<void(UE::Editor::DataStorage::Queries::EFlowControl& Flow, const FVisibleInEditorColumn& VisibilityColumn)> VisitFn)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;

		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (!DataStorage)
		{
			return false;
		}
		// Note: this query currently never gets unregistered because it is constantly used in CanExecute calls
		//  while the visibility submenu is open, and we can't unregister on module shutdown because the level
		//  editor isn't really a module that would be deactivated. 
		static QueryHandle Query = DataStorage->RegisterQuery(
			Select()
			.ReadOnly<FTypedElementWorldColumn, FVisibleInEditorColumn>()
			.Where()
			.None<FTypedElementActorTag>()
			.All<FTypedElementSelectionColumn>()
			.Compile());

		FQueryResult QueryResult = DataStorage->RunQuery(Query, ERunQueryFlags::Default,  
			[WorldIn, &VisitFn](EFlowControl& Flow, const FTypedElementWorldColumn& WorldColumn, const FVisibleInEditorColumn& VisibilityColumn)
				{
					if (WorldColumn.World.Get() != WorldIn)
					{
						return;
					}
					VisitFn(Flow, VisibilityColumn);
				});

		return QueryResult.Completed == FQueryResult::ECompletion::Fully;
	}

	bool HaveSelectedVisibleNonActors(UWorld* WorldIn)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;

		bool bToReturn = false;
		bool bSuccess = ForEachSelectedHideableNonActor(WorldIn, [&bToReturn](EFlowControl& Flow, const FVisibleInEditorColumn& VisibilityColumn)
		{
			if (VisibilityColumn.bIsVisibleInEditor)
			{
				bToReturn = true;
				Flow = EFlowControl::Break;
			}
		});
		return bSuccess && bToReturn;
	}

	bool HaveSelectedHiddenNonActors(UWorld* WorldIn)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;

		bool bToReturn = false;
		bool bSuccess = ForEachSelectedHideableNonActor(WorldIn, [&bToReturn](EFlowControl& Flow, const FVisibleInEditorColumn& VisibilityColumn)
		{
			if (!VisibilityColumn.bIsVisibleInEditor)
			{
				bToReturn = true;
				Flow = EFlowControl::Break;
			}
		});
		return bSuccess && bToReturn;
	}

	bool HaveSelectedHideableNonActors(UWorld* WorldIn)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;

		bool bToReturn = false;
		bool bSuccess = ForEachSelectedHideableNonActor(WorldIn, [&bToReturn](EFlowControl& Flow, const FVisibleInEditorColumn& VisibilityColumn)
		{
			bToReturn = true;
			Flow = EFlowControl::Break;
		});
		return bSuccess && bToReturn;
	}
}

bool FLevelEditorActionCallbacks::DefaultCanExecuteAction()
{
	return FSlateApplication::Get().IsNormalExecution();
}

void FLevelEditorActionCallbacks::BrowseDocumentation()
{
	IDocumentation::Get()->Open(TEXT("BuildingWorlds/LevelEditor"), FDocumentationSourceInfo(TEXT("help_menu")));
}

void FLevelEditorActionCallbacks::BrowseViewportControls()
{
	FString URL;
	if (FUnrealEdMisc::Get().GetURL(TEXT("ViewportControlsURL"), URL))
	{
		IDocumentation::Get()->Open(URL, FDocumentationSourceInfo(TEXT("help_menu")));
	}
}

void FLevelEditorActionCallbacks::NewLevel()
{
	bool bLevelCreated = false;
	NewLevel(bLevelCreated);
}

void FLevelEditorActionCallbacks::NewLevel(bool& bOutLevelCreated)
{
	bOutLevelCreated = false;

	if (NewLevelOverride.IsBound())
	{
		NewLevelOverride.Execute(bOutLevelCreated);
		return;
	}

	if (GUnrealEd->WarnIfLightingBuildIsCurrentlyRunning())
	{
		return;
	}

	IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked<IMainFrameModule>("MainFrame");

	FString TemplateMapPackageName;
	bool bOutIsPartitionedWorld = false;
	const bool bShowPartitionedTemplates = true;

	FNewLevelDialogModule& NewLevelDialogModule = FModuleManager::LoadModuleChecked<FNewLevelDialogModule>("NewLevelDialog");
	
	if (!NewLevelDialogModule.CreateAndShowNewLevelDialog(MainFrameModule.GetParentWindow(), TemplateMapPackageName, bShowPartitionedTemplates, bOutIsPartitionedWorld))
	{
		return;
	}

	// The new map screen will return a blank TemplateName if the user has selected to begin a new blank map
	if (TemplateMapPackageName.IsEmpty())
	{
		GEditor->CreateNewMapForEditing(/*bPromptUserToSave=*/true, bOutIsPartitionedWorld);
	}
	else
	{
		// New map screen returned a non-empty TemplateName, so the user has selected to begin from a template map
		bool TemplateFound = false;

		// Search all template map folders for a match with TemplateName
		const bool bIncludeReadOnlyRoots = true;
		if ( FPackageName::IsValidLongPackageName(TemplateMapPackageName, bIncludeReadOnlyRoots) )
		{
			const FString MapPackageFilename = FPackageName::LongPackageNameToFilename(TemplateMapPackageName, FPackageName::GetMapPackageExtension());
			if ( FPaths::FileExists(MapPackageFilename) )
			{
				// File found because the size check came back non-zero
				TemplateFound = true;

				// If there are any unsaved changes to the current level, see if the user wants to save those first.
				if ( FEditorFileUtils::SaveDirtyPackages(/*bPromptUserToSave*/true, /*bSaveMapPackages*/true, /*bSaveContentPackages*/false) )
				{
					// Load the template map file - passes LoadAsTemplate==true making the
					// level load into an untitled package that won't save over the template
					FEditorFileUtils::LoadMap(*MapPackageFilename, /*bLoadAsTemplate=*/true);
				}
			}
		}

		if (!TemplateFound)
		{
			UE_LOGF( LevelEditorActions, Warning, "Couldn't find template map package %ls", *TemplateMapPackageName);
			GEditor->CreateNewMapForEditing();
		}
	}

	bOutLevelCreated = true;
}

bool FLevelEditorActionCallbacks::NewLevel_CanExecute()
{
	return FSlateApplication::Get().IsNormalExecution() && !GLevelEditorModeTools().IsTracking();
}

void FLevelEditorActionCallbacks::OpenLevel()
{
	FEditorFileUtils::LoadMap();
}

bool FLevelEditorActionCallbacks::OpenLevel_CanExecute()
{
	return FSlateApplication::Get().IsNormalExecution() && !GLevelEditorModeTools().IsTracking();
}

void FLevelEditorActionCallbacks::DeltaTransform()
{
	if (GUnrealEd->WarnIfLightingBuildIsCurrentlyRunning())
	{
		return;
	}

	FDlgDeltaTransform DeltaDialog;

	const FDlgDeltaTransform::EResult MoveDialogResult = DeltaDialog.ShowModal();
}

void FLevelEditorActionCallbacks::OpenRecentFile( int32 RecentFileIndex )
{
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
	
	if (FMainMRUFavoritesList* RecentsAndFavorites = MainFrameModule.GetMRUFavoritesList())
	{
		FString NewPackageName;
		if( RecentsAndFavorites->VerifyMRUFile( RecentFileIndex, NewPackageName ) )
		{
			// Prompt the user to save any outstanding changes.
			if( FEditorFileUtils::SaveDirtyPackages(true, true, false) )
			{
				FString NewFilename;
				if (FPackageName::TryConvertLongPackageNameToFilename(NewPackageName, NewFilename, FPackageName::GetMapPackageExtension()))
				{
					// Load the requested level.
					FEditorFileUtils::LoadMap(NewFilename);
				}
			}
		}
	}
}

void FLevelEditorActionCallbacks::ClearRecentFiles()
{
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
	
	if (FMainMRUFavoritesList* RecentsAndFavorites = MainFrameModule.GetMRUFavoritesList())
	{
		RecentsAndFavorites->ClearMRUItems();
	}
}

void FLevelEditorActionCallbacks::OpenFavoriteFile( int32 FavoriteFileIndex )
{
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
	FMainMRUFavoritesList* MRUFavoritesList = MainFrameModule.GetMRUFavoritesList();

	const FString PackageName = MRUFavoritesList->GetFavoritesItem( FavoriteFileIndex );

	if( MRUFavoritesList->VerifyFavoritesFile( FavoriteFileIndex ) )
	{
		// Prompt the user to save any outstanding changes
		if( FEditorFileUtils::SaveDirtyPackages(true, true, false) )
		{
			FString FileName;
			if (FPackageName::TryConvertLongPackageNameToFilename(PackageName, FileName, FPackageName::GetMapPackageExtension()))
			{
				// Load the requested level.
				FEditorFileUtils::LoadMap(FileName);
			}

			// Move the item to the head of the list
			MRUFavoritesList->MoveFavoritesItemToHead(PackageName);
		}
		else
		{
			// something went wrong or the user pressed cancel.  Return to the editor so the user doesn't lose their changes		
		}
	}
}


void FLevelEditorActionCallbacks::ToggleFavorite()
{
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
	FMainMRUFavoritesList* MRUFavoritesList = MainFrameModule.GetMRUFavoritesList();
	check( MRUFavoritesList );

	if (LevelEditorActionsHelpers::IsPersistentWorld(GetWorld()))
	{
		const FString PackageName = GetWorld()->GetOutermost()->GetName();

		// If the map was already favorited, remove it from the favorites
		if ( MRUFavoritesList->ContainsFavoritesItem(PackageName) )
		{
			MRUFavoritesList->RemoveFavoritesItem(PackageName);
		}
		// If the map was not already favorited, add it to the favorites
		else
		{
			MRUFavoritesList->AddFavoritesItem(PackageName);
		}
	}
}


void FLevelEditorActionCallbacks::RemoveFavorite( int32 FavoriteFileIndex )
{
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
	FMainMRUFavoritesList* MRUFavoritesList = MainFrameModule.GetMRUFavoritesList();

	const FString PackageName = MRUFavoritesList->GetFavoritesItem( FavoriteFileIndex );

	if( MRUFavoritesList->VerifyFavoritesFile( FavoriteFileIndex ) )
	{
		if ( MRUFavoritesList->ContainsFavoritesItem(PackageName) )
		{
			MRUFavoritesList->RemoveFavoritesItem(PackageName);
		}
	}
}


bool FLevelEditorActionCallbacks::ToggleFavorite_CanExecute()
{
	if (LevelEditorActionsHelpers::IsPersistentWorld(GetWorld()))
	{
		const FMainMRUFavoritesList& MRUFavorites = *FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame").GetMRUFavoritesList();
		const int32 NumFavorites = MRUFavorites.GetNumFavorites();
		// Disable the favorites button if the map isn't associated to a file yet (new map, never before saved, etc.)
		const FString PackageName = GetWorld()->GetOutermost()->GetName();
		return (NumFavorites < FLevelEditorCommands::Get().OpenFavoriteFileCommands.Num() || MRUFavorites.ContainsFavoritesItem(PackageName));
	}
	return false;
}


bool FLevelEditorActionCallbacks::ToggleFavorite_IsChecked()
{
	bool bIsChecked = false;

	if (LevelEditorActionsHelpers::IsPersistentWorld(GetWorld()))
	{
		const FString PackageName = GetWorld()->GetOutermost()->GetName();

		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
		bIsChecked = MainFrameModule.GetMRUFavoritesList()->ContainsFavoritesItem(PackageName);
	}

	return bIsChecked;
}

bool FLevelEditorActionCallbacks::CanSaveWorld()
{
	return FSlateApplication::Get().IsNormalExecution() && (!GUnrealEd || !GUnrealEd->GetPackageAutoSaver().IsAutoSaving()) && GLevelEditorModeTools().IsOperationSupportedForCurrentAsset(EAssetOperation::Save);
}

bool FLevelEditorActionCallbacks::CanSaveUnpartitionedWorld()
{
	if (!CanSaveWorld())
	{
		return false;
	}

	return !UWorld::IsPartitionedWorld(GetWorld());
}

void FLevelEditorActionCallbacks::Save()
{
	// If the world is a template, go through the save current as path as it handles loading all external actors properly
	UWorld* World = GetWorld();
	if (FPackageName::IsTempPackage(World->GetPackage()->GetName()) && CanSaveCurrentAs())
	{
		SaveCurrentAs();
	}
	else
	{
		FEditorFileUtils::SaveCurrentLevel();
	}
}

bool FLevelEditorActionCallbacks::CanSaveCurrentAs()
{
	return CanSaveWorld() && GLevelEditorModeTools().IsOperationSupportedForCurrentAsset(EAssetOperation::Duplicate);
}

void FLevelEditorActionCallbacks::SaveCurrentAs()
{
	check(CanSaveCurrentAs());
	UWorld* World = GetWorld();
	ULevel* CurrentLevel = World->GetCurrentLevel();

	UClass* CurrentStreamingLevelClass = ULevelStreamingDynamic::StaticClass();
	if (ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(CurrentLevel))
	{
		CurrentStreamingLevelClass = StreamingLevel->GetClass();
	}
	
	
	const bool bSavedPersistentLevelAs = CurrentLevel == World->PersistentLevel;
	FString SavedFilename;
	bool bSaved = FEditorFileUtils::SaveLevelAs(CurrentLevel, &SavedFilename);
	if(bSaved)
	{
		if (bSavedPersistentLevelAs)
		{
			FEditorFileUtils::LoadMap(SavedFilename);
		}
		else if(EditorLevelUtils::RemoveLevelFromWorld(CurrentLevel))
		{
			// Add the new level we just saved as to the plevel
			FString PackageName;
			if (FPackageName::TryConvertFilenameToLongPackageName(SavedFilename, PackageName))
			{
				ULevelStreaming* StreamingLevel = UEditorLevelUtils::AddLevelToWorld(World, *PackageName, CurrentStreamingLevelClass);

				// Make the level we just added current because the expectation is that the new level replaces the existing current level
				EditorLevelUtils::MakeLevelCurrent(StreamingLevel->GetLoadedLevel());
			}

			FEditorDelegates::RefreshLevelBrowser.Broadcast();
		}
	}
}

void FLevelEditorActionCallbacks::SaveAllLevels()
{
	const bool bPromptUserToSave = false;
	const bool bSaveMapPackages = true;
	const bool bSaveContentPackages = false;
	const bool bFastSave = false;
	FEditorFileUtils::SaveDirtyPackages( bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave );
}

void FLevelEditorActionCallbacks::Browse()
{
	if (const ULevel* CurrentLevel = GetWorld()->GetCurrentLevel())
	{
		TArray<UObject*> Assets = { CurrentLevel->GetOuter() };
		GEditor->SyncBrowserToObjects(Assets);
	}
}

bool FLevelEditorActionCallbacks::CanBrowse()
{
	return !FPackageName::IsTempPackage(GetWorld()->GetPackage()->GetName());
}


void FLevelEditorActionCallbacks::ImportScene_Clicked()
{
	FEditorFileUtils::Import();
}

void FLevelEditorActionCallbacks::PreviewJson_Clicked(FName PlatformName, FString JsonFile)
{
	if (JsonFile.IsEmpty())
	{
		TArray<FString> OpenedFiles;
		FString DefaultLocation = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*(FPaths::ProjectSavedDir() / TEXT("PreviewJsonDevices") / PlatformName.ToString()));

		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		bool bOpened = false;
		if (DesktopPlatform)
		{
			bOpened = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				NSLOCTEXT("UnrealEd", "PreviewJson", "Preview Json").ToString(),
				DefaultLocation,
				TEXT(""),
				TEXT("json (*.json)|*.json"),
				EFileDialogFlags::None,
				OpenedFiles
			);
		}
		if (bOpened && OpenedFiles.Num() > 0 && OpenedFiles[0].IsEmpty() == false)
		{
			JsonFile = OpenedFiles[0];
		}
	}

	if (!JsonFile.IsEmpty())
	{
		FConfigCacheIni* PlatformEngineIni = FConfigCacheIni::ForPlatform(*PlatformName.ToString());
		FString DeviceProfileSelectionModule;
		if (PlatformEngineIni && PlatformEngineIni->GetString(TEXT("DeviceProfileManager"), TEXT("PreviewDeviceProfileSelectionModule"), DeviceProfileSelectionModule, GEngineIni))
		{
			if (IDeviceProfileSelectorModule* DPSelectorModule = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>(*DeviceProfileSelectionModule))
			{
				if(DPSelectorModule->IsJsonCompatible(JsonFile))
				{
					FString DevileProfileName;

					TMap<FString, FString> DeviceParameters;
					DPSelectorModule->GetDeviceParametersFromJson(JsonFile, DeviceParameters);
					DPSelectorModule->SetSelectorProperties(DeviceParameters);
					float ConstrainedAspectRatio = DPSelectorModule->GetConstrainedAspectRatio();

					DevileProfileName = DPSelectorModule->GetDeviceProfileName();
					EShaderPlatform PreviewShaderPlatform = DPSelectorModule->GetPreviewShaderPlatform();

					// Apply the PreviewPlatform returned by the DPSelectorModule.
					if (FDataDrivenShaderPlatformInfo::IsValid(PreviewShaderPlatform) && FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(PreviewShaderPlatform))
					{
						FName PreviewShaderPlatformName = FDataDrivenShaderPlatformInfo::GetName(PreviewShaderPlatform);

						FName FriendlyDeviceProfilePreviewName = FName(FDataDrivenShaderPlatformInfo::GetFriendlyName(PreviewShaderPlatform).ToString() + TEXT(" DeviceProfile: ") + DevileProfileName);

						UDeviceProfile* DevileProfile = UDeviceProfileManager::Get().FindProfile(DevileProfileName, false);
						if (DevileProfile)
						{
							auto GetPreviewFeatureLevelInfo = [&]()
								{
									const ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel(PreviewShaderPlatform);
									return FPreviewPlatformInfo(
										FeatureLevel,
										PreviewShaderPlatform, PlatformName,
										FDataDrivenShaderPlatformInfo::GetShaderFormat(PreviewShaderPlatform),
										DevileProfile->GetFName(),
										true,
										PreviewShaderPlatformName,
										FText::FromName(FriendlyDeviceProfilePreviewName),
										ConstrainedAspectRatio,
										DPSelectorModule->GetSafeZones());
								};

							FPreviewPlatformInfo PreviewFeatureLevelInfo = GetPreviewFeatureLevelInfo();
							GEditor->SetPreviewPlatform(PreviewFeatureLevelInfo, false);
						}
						else
						{
							UE_LOGF(LevelEditorActions, Warning, "JsonFile %ls did not match any MatchProfile", *JsonFile);
						}
					}
					else
					{
						UE_LOGF(LevelEditorActions, Warning, "JsonFile %ls did not return a valid Preview Platform", *JsonFile);
					}
				}
				else
				{
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("PreviewJsonError_NoFiles", "The Json is not compatible with this Engine Version. Please see log for details."), LOCTEXT("PreviewJsonError_Title", "Preview Json"));
				}
			}
		}
	}
}

bool FLevelEditorActionCallbacks::IsPreviewJsonVisible(FName PlatformName)
{
	FConfigCacheIni* PlatformEngineIni = FConfigCacheIni::ForPlatform(*PlatformName.ToString());
	FString DeviceProfileSelectionModule;
	if (PlatformEngineIni && PlatformEngineIni->GetString(TEXT("DeviceProfileManager"), TEXT("PreviewDeviceProfileSelectionModule"), DeviceProfileSelectionModule, GEngineIni))
	{
		if (IDeviceProfileSelectorModule* DPSelectorModule = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>(*DeviceProfileSelectionModule))
		{
			return DPSelectorModule->CanGetDeviceParametersFromJson();
		}
	}
	return false;
}

bool FLevelEditorActionCallbacks::IsGeneratePreviewJsonVisible(FName PlatformName)
{
	FConfigCacheIni* PlatformEngineIni = FConfigCacheIni::ForPlatform(*PlatformName.ToString());
	FString DeviceProfileSelectionModule;
	if (PlatformEngineIni && PlatformEngineIni->GetString(TEXT("DeviceProfileManager"), TEXT("PreviewDeviceProfileSelectionModule"), DeviceProfileSelectionModule, GEngineIni))
	{
		if (IDeviceProfileSelectorModule* DPSelectorModule = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>(*DeviceProfileSelectionModule))
		{
			return DPSelectorModule->CanExportDeviceParametersToJson();
		}
	}
	return false;
}

void FLevelEditorActionCallbacks::GeneratePreviewJson_Clicked(FString PlatformName)
{
	FString AbsoluteDebugInfoDirectory = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*(FPaths::ProjectSavedDir() / TEXT("PreviewJsonDevices") / PlatformName));
	{
		FScopedSlowTask SlowTask(100.f, NSLOCTEXT("Engine", "GeneratePlatformJson", "Generate Platform Json"), true);
		SlowTask.Visibility = ESlowTaskVisibility::ForceVisible;
		SlowTask.MakeDialog();

		SlowTask.EnterProgressFrame(35.0f);

		FConfigCacheIni* PlatformEngineIni = FConfigCacheIni::ForPlatform(*PlatformName);
		FString DeviceProfileSelectionModule;
		if (PlatformEngineIni && PlatformEngineIni->GetString(TEXT("DeviceProfileManager"), TEXT("PreviewDeviceProfileSelectionModule"), DeviceProfileSelectionModule, GEngineIni))
		{
			if (IDeviceProfileSelectorModule* DPSelectorModule = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>(*DeviceProfileSelectionModule))
			{
				TArray<FString> OutExportedFiles;
				DPSelectorModule->ExportDeviceParametersToJson(AbsoluteDebugInfoDirectory, OutExportedFiles);
				
				if (!OutExportedFiles.IsEmpty())
				{
					FString FormatedString = FString::Join(OutExportedFiles, TEXT("\n"));
					FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("GeneratePreviewJsonDialog", "{0}"), FText::FromString(FormatedString)), LOCTEXT("GeneratePreviewJsonTitle", "Generated Preview Json Files"));
				}
				else
				{
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("GeneratePreviewJsonDialog_NoFiles", "No Files Generated"), LOCTEXT("GeneratePreviewJsonTitle", "Generated Preview Json Files"));
				}
			}
		}
	}
}

void FLevelEditorActionCallbacks::ExportAll_Clicked()
{
	const bool bExportSelectedActorsOnly = false;
	FEditorFileUtils::Export( bExportSelectedActorsOnly );
}


void FLevelEditorActionCallbacks::ExportSelected_Clicked()
{
	const bool bExportSelectedActorsOnly = true;
	FEditorFileUtils::Export( bExportSelectedActorsOnly );
}


bool FLevelEditorActionCallbacks::ExportSelected_CanExecute()
{
	// Only enable the option if at least one thing is selected and its not a worldsettings
	return GEditor->GetSelectedActors()->Num() > 0 && !GEditor->IsWorldSettingsSelected();
}

void FLevelEditorActionCallbacks::AttachToActor(AActor* ParentActorPtr)
{
	USceneComponent* ComponentWithSockets = NULL;

	//@TODO: Should create a menu for each component that contains sockets, or have some form of disambiguation within the menu (like a fully qualified path)
	// Instead, we currently only display the sockets on the root component
	if (ParentActorPtr != NULL)
	{
		if (USceneComponent* RootComponent = ParentActorPtr->GetRootComponent())
		{
			if (RootComponent->HasAnySockets())
			{
				ComponentWithSockets = RootComponent;
			}
		}
	}

	// Show socket chooser if we have sockets to select
	if (ComponentWithSockets != NULL)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( "LevelEditor");
		TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

		// Create as context menu
		FSlateApplication::Get().PushMenu(
			LevelEditor.ToSharedRef(),
			FWidgetPath(),
			SNew(SSocketChooserPopup)
			.SceneComponent( ComponentWithSockets )
			.OnSocketChosen_Static( &FLevelEditorActionCallbacks::AttachToSocketSelection, ParentActorPtr ),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
			);
	}
	else
	{
		AttachToSocketSelection( NAME_None, ParentActorPtr );
	}
}

void FLevelEditorActionCallbacks::AttachToSocketSelection(const FName SocketName, AActor* ParentActorPtr)
{
	FSlateApplication::Get().DismissAllMenus();

	if(ParentActorPtr != NULL)
	{
		// Attach each child
		FScopedTransaction Transaction(LOCTEXT("AttachActors", "Attach actors"));
		bool bAttached = false;

		for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = Cast<AActor>( *It );
			if (GEditor->CanParentActors(ParentActorPtr, Actor))
			{
				bAttached = true;
				GEditor->ParentActors(ParentActorPtr, Actor, SocketName);
			}
		}

		if (!bAttached)
		{
			Transaction.Cancel();
		}
	}	
}

void FLevelEditorActionCallbacks::SetMaterialQualityLevel( EMaterialQualityLevel::Type NewQualityLevel )
{
	static IConsoleVariable* MaterialQualityLevelVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MaterialQualityLevel"));
	EConsoleVariableFlags MaxPriority = GEditor && GEditor->IsPreviewPlatformActive() ? ECVF_SetByPreview: ECVF_SetByScalability;
	MaterialQualityLevelVar->ReplaceCurrentPriorityAndTag(NewQualityLevel, MaxPriority);

	GUnrealEd->OnSceneMaterialsModified();

	GUnrealEd->RedrawAllViewports();
}

bool FLevelEditorActionCallbacks::IsMaterialQualityLevelChecked( EMaterialQualityLevel::Type TestQualityLevel )
{
	static const auto MaterialQualityLevelVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MaterialQualityLevel"));
	EMaterialQualityLevel::Type MaterialQualityLevel = (EMaterialQualityLevel::Type)FMath::Clamp(MaterialQualityLevelVar->GetValueOnGameThread(), 0, (int32)EMaterialQualityLevel::Num-1);
	return TestQualityLevel == MaterialQualityLevel;
}

void FLevelEditorActionCallbacks::TogglePreviewPlatform()
{
	GEditor->TogglePreviewPlatform();
}

bool FLevelEditorActionCallbacks::IsPreviewPlatformEnabled()
{
	if (GUnrealEd->IsLightingBuildCurrentlyRunning())
	{
		return false;
	}
	return GEditor->IsPreviewPlatformEnabled();
}

bool FLevelEditorActionCallbacks::IsPreviewPlatformActive()
{
	return GEditor->IsPreviewPlatformEnabled() && GEditor->IsPreviewPlatformActive();
}

bool FLevelEditorActionCallbacks::IsPreviewModeButtonVisible()
{
	return GEditor->IsPreviewPlatformEnabled();
}

void FLevelEditorActionCallbacks::SetPreviewPlatform(FPreviewPlatformInfo NewPreviewPlatform)
{
	// When called through SMenuEntryBlock::OnClicked(), the popup menus are not dismissed when
	// clicking on a checkbox, but they are dismissed when clicking on a button. We need the popup
	// menus to go away, or SetFeaturePlatform() is unable to display a progress dialog. Force
	// the dismissal here.
	FSlateApplication::Get().DismissAllMenus();

	GEditor->SetPreviewPlatform(NewPreviewPlatform, true);
}

bool FLevelEditorActionCallbacks::CanExecutePreviewPlatform(FPreviewPlatformInfo NewPreviewPlatform)
{
	// TODO: Carl - Disable SM6 platforms for Metal SM6 until we fix issues.
	if (IsMetalSM6Platform(GMaxRHIShaderPlatform) && NewPreviewPlatform.PreviewFeatureLevel >= GMaxRHIFeatureLevel)
	{
		return false;
	}
	
	if (NewPreviewPlatform.PreviewFeatureLevel > GMaxRHIFeatureLevel)
	{
		return false;
	}

	// Disable shader format SF_VULKAN_SM5 when the preferred shader code format for preview platforms is DXBC (D3D11).
	// Many features supported in Vulkan SM5 are not supported by D3D11, such as UAV limits.
	if (NewPreviewPlatform.PreviewShaderFormatName == TEXT("SF_VULKAN_SM5") && GRHIGlobals.PreferredPreviewShaderCodeFormat == TEXT("DXBC"))
	{
		return false;
	}

	const EShaderPlatform PreviewShaderPlatform = NewPreviewPlatform.ShaderPlatform;

	if (FDataDrivenShaderPlatformInfo::IsValid(PreviewShaderPlatform) && FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(PreviewShaderPlatform))
	{
		// When the preview platform's DDSPI MaxSamplers is > 16 and the current RHI device has support
		// for > 16 samplers we rely on the shader compiler being able to choose an appropriate profile for the
		// preview feature level that supports > 16 samplers. On D3D12 SM5 the D3D shader compiler will use Dxc and
		// sm6.0. Vulkan SM5 also appears to handle > 16 samplers fine.
		if ((int32)FDataDrivenShaderPlatformInfo::GetMaxSamplers(PreviewShaderPlatform) > GRHIGlobals.MaxTextureSamplers)
		{
			return false;
		}
	}

	return true;
}

bool FLevelEditorActionCallbacks::IsPreviewPlatformChecked(FPreviewPlatformInfo PreviewPlatform)
{
	return GEditor->PreviewPlatform.Matches(PreviewPlatform);
}

void FLevelEditorActionCallbacks::ConfigureLightingBuildOptions( const FLightingBuildOptions& Options )
{
	GConfig->SetBool( TEXT("LightingBuildOptions"), TEXT("OnlyBuildSelected"),		Options.bOnlyBuildSelected,			GEditorPerProjectIni );
	GConfig->SetBool( TEXT("LightingBuildOptions"), TEXT("OnlyBuildCurrentLevel"),	Options.bOnlyBuildCurrentLevel,		GEditorPerProjectIni );
	GConfig->SetBool( TEXT("LightingBuildOptions"), TEXT("OnlyBuildSelectedLevels"),Options.bOnlyBuildSelectedLevels,	GEditorPerProjectIni );
	GConfig->SetBool( TEXT("LightingBuildOptions"), TEXT("OnlyBuildVisibility"),	Options.bOnlyBuildVisibility,		GEditorPerProjectIni );
}

bool FLevelEditorActionCallbacks::CanBuildLighting()
{
	// Building lighting modifies the BuildData package, which the PIE session will also be referencing without getting notified
	return !(GEditor->PlayWorld || GUnrealEd->bIsSimulatingInEditor)
		&& GetWorld()->GetFeatureLevel() >= ERHIFeatureLevel::SM5;
}

bool FLevelEditorActionCallbacks::CanBuildReflectionCaptures()
{
	// Building reflection captures modifies the BuildData package, which the PIE session will also be referencing without getting notified
	return !(GEditor->PlayWorld || GUnrealEd->bIsSimulatingInEditor);
}


void FLevelEditorActionCallbacks::Build_Execute()
{
	// Reset build options
	ConfigureLightingBuildOptions( FLightingBuildOptions() );

	// Build everything!
	FEditorBuildUtils::EditorBuild( GetWorld(), FBuildOptions::BuildAll );
}

bool FLevelEditorActionCallbacks::Build_CanExecute()
{
	return CanBuildLighting() && CanBuildReflectionCaptures();
}

void FLevelEditorActionCallbacks::BuildAndSubmitToSourceControl_Execute()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	LevelEditorModule.SummonBuildAndSubmit();
}


void FLevelEditorActionCallbacks::BuildLightingOnly_Execute()
{
	// Reset build options
	ConfigureLightingBuildOptions( FLightingBuildOptions() );

	// Build lighting!
	const bool bAllowLightingDialog = false;
	FEditorBuildUtils::EditorBuild( GetWorld(), FBuildOptions::BuildLighting, bAllowLightingDialog );
}

bool FLevelEditorActionCallbacks::BuildLighting_CanExecute()
{	
	return IsStaticLightingAllowed() && CanBuildLighting() && CanBuildReflectionCaptures();
}

void FLevelEditorActionCallbacks::BuildReflectionCapturesOnly_Execute()
{
	if (GWorld != nullptr && GWorld->GetFeatureLevel() == ERHIFeatureLevel::ES3_1)
	{
		// When we feature change from SM5 to ES31 we call BuildReflectionCapture if we have Unbuilt Reflection Components, so no reason to call it again here
		// This is to make sure that we have valid data for Mobile Preview.
		
		// ES31->SM5 to be able to capture
		TogglePreviewPlatform();
		// SM5->ES31 BuildReflectionCaptures are triggered here on callback
		TogglePreviewPlatform();
	}
	else
	{
		GEditor->BuildReflectionCaptures();
	}
}

bool FLevelEditorActionCallbacks::BuildReflectionCapturesOnly_CanExecute()
{
	return CanBuildReflectionCaptures();
}

void FLevelEditorActionCallbacks::BuildLightingOnly_VisibilityOnly_Execute()
{
	// Configure build options
	FLightingBuildOptions LightingBuildOptions;
	LightingBuildOptions.bOnlyBuildVisibility = true;
	ConfigureLightingBuildOptions( LightingBuildOptions );

	// Build lighting!
	const bool bAllowLightingDialog = false;
	FEditorBuildUtils::EditorBuild( GetWorld(), FBuildOptions::BuildLighting, bAllowLightingDialog );

	// Reset build options
	ConfigureLightingBuildOptions( FLightingBuildOptions() );
}

bool FLevelEditorActionCallbacks::LightingBuildOptions_UseErrorColoring_IsChecked()
{
	bool bUseErrorColoring = false;
	GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("UseErrorColoring"), bUseErrorColoring, GEditorPerProjectIni);
	return bUseErrorColoring;
}


void FLevelEditorActionCallbacks::LightingBuildOptions_UseErrorColoring_Toggled()
{
	bool bUseErrorColoring = false;
	GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("UseErrorColoring"), bUseErrorColoring, GEditorPerProjectIni );
	GConfig->SetBool( TEXT("LightingBuildOptions"), TEXT("UseErrorColoring"), !bUseErrorColoring, GEditorPerProjectIni );
}


bool FLevelEditorActionCallbacks::LightingBuildOptions_ShowLightingStats_IsChecked()
{
	bool bShowLightingBuildInfo = false;
	GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("ShowLightingBuildInfo"), bShowLightingBuildInfo, GEditorPerProjectIni);
	return bShowLightingBuildInfo;
}


void FLevelEditorActionCallbacks::LightingBuildOptions_ShowLightingStats_Toggled()
{
	bool bShowLightingBuildInfo = false;
	GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("ShowLightingBuildInfo"), bShowLightingBuildInfo, GEditorPerProjectIni );
	GConfig->SetBool( TEXT("LightingBuildOptions"), TEXT("ShowLightingBuildInfo"), !bShowLightingBuildInfo, GEditorPerProjectIni );
}



void FLevelEditorActionCallbacks::BuildGeometryOnly_Execute()
{
	// Build geometry!
	FEditorBuildUtils::EditorBuild( GetWorld(), FBuildOptions::BuildVisibleGeometry );
}


void FLevelEditorActionCallbacks::BuildGeometryOnly_OnlyCurrentLevel_Execute()
{
	// Build geometry (current level)!
	FEditorBuildUtils::EditorBuild( GetWorld(), FBuildOptions::BuildGeometry );
}


void FLevelEditorActionCallbacks::BuildPathsOnly_Execute()
{
	// Build paths!
	FEditorBuildUtils::EditorBuild( GetWorld(), FBuildOptions::BuildAIPaths );
}

bool FLevelEditorActionCallbacks::IsWorldPartitionEnabled()
{
	return UWorld::IsPartitionedWorld(GetWorld());
}

bool FLevelEditorActionCallbacks::IsWorldPartitionStreamingEnabled()
{
	if (!IsWorldPartitionEnabled())
	{
		return false;
	}

	return GetWorld()->GetWorldPartition()->IsStreamingEnabled();
}

void FLevelEditorActionCallbacks::BuildHLODs_Execute()
{
	// Build HLOD
	FEditorBuildUtils::EditorBuild(GetWorld(), FBuildOptions::BuildHierarchicalLOD);
}

void FLevelEditorActionCallbacks::BuildHLODs_SelectedActors_Execute()
{
	TArray<FBox> FilterVolumes;
	for (FSelectionIterator It = GEditor->GetSelectedActorIterator(); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			FBox RuntimeBounds;
			FBox EditorBounds;
			Actor->GetStreamingBounds(RuntimeBounds, EditorBounds);
			FilterVolumes.Add(RuntimeBounds);
		}
	}

	UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(GetWorld());
	if (WorldPartitionSubsystem)
	{
		for (const FWorldPartitionHandle& SelectedActorHandle : WorldPartitionSubsystem->SelectedActorHandles)
		{
			FilterVolumes.Add(SelectedActorHandle->GetRuntimeBounds());
		}
	}

	if (FilterVolumes.Num() > 0)
	{
		UWorldPartitionHLODsBuilder::BuildWithFilters(GetWorld(), { MakeShared<FHLODCreationVolumeFilter>(MoveTemp(FilterVolumes)) });
	}
	else
	{
		UE_LOGF(LevelEditorActions, Warning, "Build HLODs - No actors selected");
	}
}

void FLevelEditorActionCallbacks::BuildMinimap_Execute()
{
	// Build Minimap
	FEditorBuildUtils::EditorBuild(GetWorld(), FBuildOptions::BuildMinimap);
}

void FLevelEditorActionCallbacks::BuildLandscapeSplineMeshes_Execute()
{
	// Build Landscape Spline Meshes
	FEditorBuildUtils::EditorBuild(GetWorld(), FBuildOptions::BuildLandscapeSplineMeshes);
}

void FLevelEditorActionCallbacks::BuildTextureStreamingOnly_Execute()
{
	FEditorBuildUtils::EditorBuildTextureStreaming(GetWorld());
	GEngine->DeferredCommands.AddUnique(TEXT("MAP CHECK NOTIFYRESULTS"));
}

void FLevelEditorActionCallbacks::BuildVirtualTextureOnly_Execute()
{
	FEditorBuildUtils::EditorBuildVirtualTexture(GetWorld());
	GEngine->DeferredCommands.AddUnique(TEXT("MAP CHECK NOTIFYRESULTS"));
}

void FLevelEditorActionCallbacks::BuildAllLandscape_Execute()
{
	FEditorBuildUtils::EditorBuild(GetWorld(), FBuildOptions::BuildAllLandscape);
}

bool FLevelEditorActionCallbacks::BuildExternalType_CanExecute(const int32 Index)
{
	TArray<FName> BuildTypeNames;
	FEditorBuildUtils::GetBuildTypes(BuildTypeNames);

	if (BuildTypeNames.IsValidIndex(Index))
	{
		return FEditorBuildUtils::EditorCanBuild(GetWorld(), BuildTypeNames[Index]);
	}

	return false;
}

void FLevelEditorActionCallbacks::BuildExternalType_Execute(const int32 Index)
{
	TArray<FName> BuildTypeNames;
	FEditorBuildUtils::GetBuildTypes(BuildTypeNames);

	if (BuildTypeNames.IsValidIndex(Index))
	{
		FEditorBuildUtils::EditorBuild(GetWorld(), BuildTypeNames[Index]);
	}
}

bool FLevelEditorActionCallbacks::IsLightingQualityChecked( ELightingBuildQuality TestQuality )
{
	int32 CurrentQualityLevel;
	GConfig->GetInt(TEXT("LightingBuildOptions"), TEXT("QualityLevel"), CurrentQualityLevel, GEditorPerProjectIni);
	return TestQuality == CurrentQualityLevel;
}

void FLevelEditorActionCallbacks::SetLightingQuality( ELightingBuildQuality NewQuality )
{
	GConfig->SetInt(TEXT("LightingBuildOptions"), TEXT("QualityLevel"), (int32)NewQuality, GEditorPerProjectIni);
}

float FLevelEditorActionCallbacks::GetLightingDensityIdeal()
{
	return ( GEngine->IdealLightMapDensity );
}

void FLevelEditorActionCallbacks::SetLightingDensityIdeal( float Value )
{
	GEngine->IdealLightMapDensity = Value;

	// We need to make sure that Maximum is always slightly larger than ideal...
	if (GEngine->IdealLightMapDensity >= GEngine->MaxLightMapDensity - 0.01f)
	{
		SetLightingDensityMaximum( GEngine->IdealLightMapDensity + 0.01f );
	}

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

float FLevelEditorActionCallbacks::GetLightingDensityMaximum()
{
	return ( GEngine->MaxLightMapDensity );
}

void FLevelEditorActionCallbacks::SetLightingDensityMaximum( float Value )
{
	GEngine->MaxLightMapDensity = Value;

	// We need to make sure that Maximum is always slightly larger than ideal...
	if (GEngine->MaxLightMapDensity <= GEngine->IdealLightMapDensity + 0.01f)
	{
		GEngine->MaxLightMapDensity = GEngine->IdealLightMapDensity + 0.01f;
	}

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

float FLevelEditorActionCallbacks::GetLightingDensityColorScale()
{
	return ( GEngine->RenderLightMapDensityColorScale );
}

void FLevelEditorActionCallbacks::SetLightingDensityColorScale( float Value )
{
	GEngine->RenderLightMapDensityColorScale = Value;

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

float FLevelEditorActionCallbacks::GetLightingDensityGrayscaleScale()
{
	return ( GEngine->RenderLightMapDensityGrayscaleScale );
}

void FLevelEditorActionCallbacks::SetLightingDensityGrayscaleScale( float Value )
{
	GEngine->RenderLightMapDensityGrayscaleScale = Value;

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

void FLevelEditorActionCallbacks::SetLightingDensityRenderGrayscale()
{
	GEngine->bRenderLightMapDensityGrayscale = !GEngine->bRenderLightMapDensityGrayscale;
	GEngine->SaveConfig();
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

bool FLevelEditorActionCallbacks::IsLightingDensityRenderGrayscaleChecked()
{
	return GEngine->bRenderLightMapDensityGrayscale;
}

void FLevelEditorActionCallbacks::SetLightingResolutionStaticMeshes( ECheckBoxState NewCheckedState )
{
	FLightmapResRatioAdjustSettings& Settings = FLightmapResRatioAdjustSettings::Get();
	Settings.bStaticMeshes = ( NewCheckedState == ECheckBoxState::Checked );
}

ECheckBoxState FLevelEditorActionCallbacks::IsLightingResolutionStaticMeshesChecked()
{
	return ( FLightmapResRatioAdjustSettings::Get().bStaticMeshes ? ECheckBoxState::Checked : ECheckBoxState::Unchecked );
}

void FLevelEditorActionCallbacks::SetLightingResolutionBSPSurfaces( ECheckBoxState NewCheckedState )
{
	FLightmapResRatioAdjustSettings& Settings = FLightmapResRatioAdjustSettings::Get();
	Settings.bBSPSurfaces = ( NewCheckedState == ECheckBoxState::Checked );
}

ECheckBoxState FLevelEditorActionCallbacks::IsLightingResolutionBSPSurfacesChecked()
{
	return ( FLightmapResRatioAdjustSettings::Get().bBSPSurfaces ? ECheckBoxState::Checked : ECheckBoxState::Unchecked );
}

void FLevelEditorActionCallbacks::SetLightingResolutionLevel( FLightmapResRatioAdjustSettings::AdjustLevels NewLevel )
{
	FLightmapResRatioAdjustSettings& Settings = FLightmapResRatioAdjustSettings::Get();
	Settings.LevelOptions = NewLevel;
}

bool FLevelEditorActionCallbacks::IsLightingResolutionLevelChecked( FLightmapResRatioAdjustSettings::AdjustLevels TestLevel )
{
	return ( FLightmapResRatioAdjustSettings::Get().LevelOptions == TestLevel );
}

void FLevelEditorActionCallbacks::SetLightingResolutionSelectedObjectsOnly()
{
	FLightmapResRatioAdjustSettings& Settings = FLightmapResRatioAdjustSettings::Get();
	Settings.bSelectedObjectsOnly = !Settings.bSelectedObjectsOnly;
}

bool FLevelEditorActionCallbacks::IsLightingResolutionSelectedObjectsOnlyChecked()
{
	return FLightmapResRatioAdjustSettings::Get().bSelectedObjectsOnly;
}

float FLevelEditorActionCallbacks::GetLightingResolutionMinSMs()
{
	return static_cast<float>( FLightmapResRatioAdjustSettings::Get().Min_StaticMeshes );
}

void FLevelEditorActionCallbacks::SetLightingResolutionMinSMs( float Value )
{
	FLightmapResRatioAdjustSettings& Settings = FLightmapResRatioAdjustSettings::Get();
	Settings.Min_StaticMeshes = static_cast<int32>( Value );
}

float FLevelEditorActionCallbacks::GetLightingResolutionMaxSMs()
{
	return static_cast<float>( FLightmapResRatioAdjustSettings::Get().Max_StaticMeshes );
}

void FLevelEditorActionCallbacks::SetLightingResolutionMaxSMs( float Value )
{
	FLightmapResRatioAdjustSettings& Settings = FLightmapResRatioAdjustSettings::Get();
	Settings.Max_StaticMeshes = static_cast<int32>( Value );
}

float FLevelEditorActionCallbacks::GetLightingResolutionMinBSPs()
{
	return static_cast<float>( FLightmapResRatioAdjustSettings::Get().Min_BSPSurfaces );
}

void FLevelEditorActionCallbacks::SetLightingResolutionMinBSPs( float Value )
{
	FLightmapResRatioAdjustSettings& Settings = FLightmapResRatioAdjustSettings::Get();
	Settings.Min_BSPSurfaces = static_cast<int32>( Value );
}

float FLevelEditorActionCallbacks::GetLightingResolutionMaxBSPs()
{
	return static_cast<float>( FLightmapResRatioAdjustSettings::Get().Max_BSPSurfaces );
}

void FLevelEditorActionCallbacks::SetLightingResolutionMaxBSPs( float Value )
{
	FLightmapResRatioAdjustSettings& Settings = FLightmapResRatioAdjustSettings::Get();
	Settings.Max_BSPSurfaces = static_cast<int32>( Value );
}

int32 FLevelEditorActionCallbacks::GetLightingResolutionRatio()
{
	return FMath::RoundToInt(FLightmapResRatioAdjustSettings::Get().Ratio * 100.0f);
}

void FLevelEditorActionCallbacks::SetLightingResolutionRatio( int32 Value )
{
	FLightmapResRatioAdjustSettings& Settings = FLightmapResRatioAdjustSettings::Get();
	const float NewValue = static_cast<float>(Value) / 100.0f;
	if ( Settings.Ratio != NewValue )
	{
		Settings.Ratio = NewValue;
		Settings.ApplyRatioAdjustment();
	}
}

void FLevelEditorActionCallbacks::SetLightingResolutionRatioCommit( int32 Value, ETextCommit::Type CommitInfo)
{
	if ((CommitInfo == ETextCommit::OnEnter) || (CommitInfo == ETextCommit::OnUserMovedFocus))
	{
		SetLightingResolutionRatio( Value );
	}
}

void FLevelEditorActionCallbacks::ShowLightingStaticMeshInfo()
{
	if (GUnrealEd)
	{
		GUnrealEd->ShowLightingStaticMeshInfoWindow();
	}
}

void FLevelEditorActionCallbacks::ShowSceneStats()
{
	if (GUnrealEd)
	{
		GUnrealEd->OpenSceneStatsWindow();
	}
}

void FLevelEditorActionCallbacks::ShowTextureStats()
{
	if (GUnrealEd)
	{
		GUnrealEd->OpenTextureStatsWindow();
	}
}

void FLevelEditorActionCallbacks::MapCheck_Execute()
{
	GEditor->Exec( GetWorld(), TEXT("MAP CHECK") );
}

bool FLevelEditorActionCallbacks::CanShowSourceCodeActions()
{
	if (GEditor)
	{
		// Don't allow hot reloading if we're running networked PIE instances
		// The reason, is it's fairly complicated to handle the re-wiring that needs to happen when we re-instance objects like player controllers, possessed pawns, etc...
		const TIndirectArray<FWorldContext>& WorldContextList = GEditor->GetWorldContexts();

		for (const FWorldContext& WorldContext : WorldContextList)
		{
			if (WorldContext.World() && WorldContext.World()->WorldType == EWorldType::PIE && WorldContext.World()->NetDriver)
			{
				return false;
			}
		}
	}

	IHotReloadInterface& HotReloadSupport = FModuleManager::LoadModuleChecked<IHotReloadInterface>(HotReloadModule);
	// If there is at least one loaded game module, source code actions should be available.
	return HotReloadSupport.IsAnyGameModuleLoaded();
}

void FLevelEditorActionCallbacks::RecompileGameCode_Clicked()
{
#if WITH_LIVE_CODING
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	if (LiveCoding != nullptr && LiveCoding->IsEnabledByDefault())
	{
		LiveCoding->EnableForSession(true);
		if (LiveCoding->IsEnabledForSession())
		{
			LiveCoding->Compile();
		}
		else
		{
			FText EnableErrorText = LiveCoding->GetEnableErrorText();
			if (EnableErrorText.IsEmpty())
			{
				EnableErrorText = LOCTEXT("NoLiveCodingCompileAfterHotReload", "Live Coding cannot be enabled while hot-reloaded modules are active. Please close the editor and build from your IDE before restarting.");
			}
			FMessageDialog::Open(EAppMsgType::Ok, EnableErrorText);
		}
		return;
	}
#endif

	// Don't allow a recompile while already compiling!
	IHotReloadInterface& HotReloadSupport = FModuleManager::LoadModuleChecked<IHotReloadInterface>(HotReloadModule);
	if( !HotReloadSupport.IsCurrentlyCompiling() )
	{
		// We want compiling to happen asynchronously
		HotReloadSupport.DoHotReloadFromEditor(EHotReloadFlags::None);
	}
}

bool FLevelEditorActionCallbacks::Recompile_CanExecute()
{
#if WITH_LIVE_CODING
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	if (LiveCoding != nullptr && LiveCoding->IsEnabledByDefault())
	{
		return !LiveCoding->IsCompiling();
	}
#endif

	// We can't recompile while in PIE
	if (GEditor->IsPlaySessionInProgress())
	{
		return false;
	}

	// We're not able to recompile if a compile is already in progress!

	IHotReloadInterface& HotReloadSupport = FModuleManager::LoadModuleChecked<IHotReloadInterface>(HotReloadModule);
	return !HotReloadSupport.IsCurrentlyCompiling() && !(FApp::GetEngineIsPromotedBuild() && FEngineBuildSettings::IsPerforceBuild());
}

#if WITH_LIVE_CODING
void FLevelEditorActionCallbacks::LiveCoding_ToggleEnabled()
{
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	if (LiveCoding != nullptr)
	{
		LiveCoding->EnableByDefault(!LiveCoding->IsEnabledByDefault());

		if (LiveCoding->IsEnabledByDefault() && !LiveCoding->IsEnabledForSession())
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoEnableLiveCodingAfterHotReload", "Live Coding cannot be enabled while hot-reloaded modules are active. Please close the editor and build from your IDE before restarting."));
		}
	}
}

bool FLevelEditorActionCallbacks::LiveCoding_IsEnabled( )
{
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	return LiveCoding != nullptr && LiveCoding->IsEnabledByDefault();
}

void FLevelEditorActionCallbacks::LiveCoding_StartSession_Clicked()
{
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	if (LiveCoding != nullptr)
	{
		LiveCoding->EnableForSession(true);

		if (!LiveCoding->IsEnabledForSession())
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoStartedLiveCodingAfterHotReload", "Live Coding cannot be started after hot-reload has been used. Please close the editor and build from your IDE before restarting."));
		}
	}
}

bool FLevelEditorActionCallbacks::LiveCoding_CanStartSession()
{
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	return LiveCoding != nullptr && LiveCoding->IsEnabledByDefault() && !LiveCoding->HasStarted();
}

void FLevelEditorActionCallbacks::LiveCoding_ShowConsole_Clicked()
{
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	if (LiveCoding!= nullptr)
	{
		LiveCoding->ShowConsole();
	}
}

bool FLevelEditorActionCallbacks::LiveCoding_CanShowConsole()
{
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	return LiveCoding!= nullptr && LiveCoding->IsEnabledForSession();
}

void FLevelEditorActionCallbacks::LiveCoding_Settings_Clicked()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "Live Coding");
}
#endif


void FLevelEditorActionCallbacks::GoToCodeForActor_Clicked()
{
	const FSelectedActorInfo SelectedActorInfo = AssetSelectionUtils::GetSelectedActorInfo();
	FSourceCodeNavigation::NavigateToClass(SelectedActorInfo.SelectionClass);
}

bool FLevelEditorActionCallbacks::GoToCodeForActor_CanExecute()
{
	const FSelectedActorInfo SelectedActorInfo = AssetSelectionUtils::GetSelectedActorInfo();
	return FSourceCodeNavigation::CanNavigateToClass(SelectedActorInfo.SelectionClass);
}

bool FLevelEditorActionCallbacks::GoToCodeForActor_IsVisible()
{
	return ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed();
}

void FLevelEditorActionCallbacks::GoToDocsForActor_Clicked()
{
	const auto& SelectedActorInfo = AssetSelectionUtils::GetSelectedActorInfo();
	if( SelectedActorInfo.SelectionClass != nullptr )
	{
		FString DocumentationLink = FEditorClassUtils::GetDocumentationLink(SelectedActorInfo.SelectionClass);
		if (!DocumentationLink.IsEmpty())
		{
			FString DocumentationLinkBaseUrl = FEditorClassUtils::GetDocumentationLinkBaseUrl(SelectedActorInfo.SelectionClass);
			IDocumentation::Get()->Open( DocumentationLink, FDocumentationSourceInfo(TEXT("rightclick_viewdoc")), DocumentationLinkBaseUrl);
		}
	}
}

void FLevelEditorActionCallbacks::FindInContentBrowser_Clicked()
{
	GEditor->SyncToContentBrowser();
}

bool FLevelEditorActionCallbacks::FindInContentBrowser_CanExecute()
{
	return GEditor->CanSyncToContentBrowser();
}

void FLevelEditorActionCallbacks::EditAsset_Clicked( const EToolkitMode::Type ToolkitMode, TWeakPtr< SLevelEditor > LevelEditor, bool bConfirmMultiple )
{
	if( GEditor->GetSelectedActorCount() > 0 )
	{
		TArray< UObject* > ReferencedAssets;
		const bool bIgnoreOtherAssetsIfBPReferenced = true;
		GEditor->GetReferencedAssetsForEditorSelection( ReferencedAssets, bIgnoreOtherAssetsIfBPReferenced );

		bool bShouldOpenEditors = (ReferencedAssets.Num() == 1);

		if (ReferencedAssets.Num() > 1)
		{
			if (bConfirmMultiple)
			{
				int32 Response = FMessageDialog::Open(
					EAppMsgType::YesNo,
					LOCTEXT("OpenAllAssetEditors", "There is more than one referenced asset in the selection. Do you want to open them all for editing?")
					);

				bShouldOpenEditors = (Response == EAppReturnType::Yes);
			}
			else
			{
				bShouldOpenEditors = true;
			}
		}

		if (bShouldOpenEditors)
		{
			// Clear focus so the level viewport can receive its focus lost call (and clear pending keyup events which wouldn't arrive)
			FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::WindowActivate);

			auto LevelEditorSharedPtr = LevelEditor.Pin();

			if (LevelEditorSharedPtr.IsValid())
			{
				for (auto Asset : ReferencedAssets)
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Asset, ToolkitMode, LevelEditorSharedPtr);
				}
			}
		}
	}
}

bool FLevelEditorActionCallbacks::EditAsset_CanExecute()
{
	if (GEditor->GetSelectedActorCount() > 0)
	{
		TArray< UObject* > ReferencedAssets;
		const bool bIgnoreOtherAssetsIfBPReferenced = true;
		GEditor->GetReferencedAssetsForEditorSelection(ReferencedAssets, bIgnoreOtherAssetsIfBPReferenced);

		return ReferencedAssets.Num() > 0;
	}

	return false;
}

void FLevelEditorActionCallbacks::OpenSelectionInPropertyMatrix_Clicked()
{
	// Gather the selected objects through the ITypedElementDetailsInterface so that any items that use
	//  stand-in actors are properly gathered instead of their actors.
	TArray<UObject*> SelectedObjects;
	USelection* Selection = GEditor->GetSelectedActors();
	UTypedElementSelectionSet* SelectionSet = Selection ? Selection->GetElementSelectionSet() : nullptr;
	if (ensure(SelectionSet))
	{
		SelectionSet->ForEachSelectedElement<ITypedElementDetailsInterface>([&SelectedObjects](const TTypedElement<ITypedElementDetailsInterface>& InDetailsElement)
		{
			if (TUniquePtr<ITypedElementDetailsObject> ElementDetailsObject = InDetailsElement.GetDetailsObject())
			{
				if (UObject* Object = ElementDetailsObject->GetObject())
				{
					SelectedObjects.Add(Object);
				}
			}
			return true;
		});
	}
	
	if (!SelectedObjects.IsEmpty())
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
		PropertyEditorModule.CreatePropertyEditorToolkit(TSharedPtr<IToolkitHost>(), SelectedObjects );
	}
}

bool FLevelEditorActionCallbacks::OpenSelectionInPropertyMatrix_IsVisible()
{
	auto SelectionIsNonActors = []()->bool
	{
		USelection* Selection = GEditor->GetSelectedActors();
		if (!Selection)
		{
			return false;
		}
		return InternalEditorLevelLibrary::DoesDetailSelectionContainNonActorsOrComponents(Selection->GetElementSelectionSet());
	};

	return FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" ).GetCanUsePropertyMatrix() 
		&& GEditor->GetSelectedActorCount() > 1
		&& (FPropertyEditorModule::CVarAllowPropertyMatrixForNonActors.GetValueOnAnyThread()
			|| !SelectionIsNonActors());
}

void FLevelEditorActionCallbacks::LockActorMovement_Clicked()
{
	GEditor->ToggleSelectedItemMovementLock();
}

void FLevelEditorActionCallbacks::DetachActor_Clicked()
{
	GEditor->DetachSelectedActors();
}

bool FLevelEditorActionCallbacks::DetachActor_CanExecute()
{
	FSelectedActorInfo SelectionInfo = AssetSelectionUtils::GetSelectedActorInfo();
	
	if (SelectionInfo.NumSelected > 0 && SelectionInfo.bHaveAttachedActor)
	{
		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);
		FText DetachErrorMsg;
		for (AActor* SelectedActor : SelectedActors)
		{
			if (!SelectedActor->EditorCanDetachFrom(SelectedActor->GetSceneOutlinerParent(), DetachErrorMsg))
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

void FLevelEditorActionCallbacks::AttachSelectedActors()
{
	GUnrealEd->AttachSelectedActors();
}

void FLevelEditorActionCallbacks::AttachActorIteractive()
{
	if(GUnrealEd->GetSelectedActorCount())
	{
		FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

		ActorPickerMode.BeginActorPickingMode(
			FOnGetAllowedClasses(), 
			FOnShouldFilterActor::CreateStatic(&FLevelEditorActionCallbacks::IsAttachableActor), 
			FOnActorSelected::CreateStatic(&FLevelEditorActionCallbacks::AttachToActor)
			);
	}
}

bool FLevelEditorActionCallbacks::IsAttachableActor( const AActor* const ParentActor )
{
	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		if (!GEditor->CanParentActors(ParentActor, Actor))
		{
			return false;
		}

		USceneComponent* ChildRoot = Actor->GetRootComponent();
		USceneComponent* ParentRoot = ParentActor->GetRootComponent();

		if (ChildRoot != nullptr && ParentRoot != nullptr && ChildRoot->IsAttachedTo(ParentRoot))
		{
			return false;
		}
	}
	return true;
}

void FLevelEditorActionCallbacks::CreateNewOutlinerFolder_Clicked()
{
	const FFolder NewFolderName = FActorFolders::Get().GetDefaultFolderForSelection(*GetWorld());
	FActorFolders::Get().CreateFolderContainingSelection(*GetWorld(), NewFolderName);
}

void FLevelEditorActionCallbacks::PlayFromHere_Clicked(bool bFloatingWindow)
{
	if (GEditor->GetSelectedActorCount() == 1)
	{
		if (AActor* Actor = Cast<AActor>(*GEditor->GetSelectedActorIterator()))
		{
			Actor->GetWorld()->PersistentLevel->PlayFromHereActor = Actor;
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			FPlayWorldCommandCallbacks::StartPlayFromHere(Actor->GetActorLocation(), Actor->GetActorRotation(), bFloatingWindow ? nullptr : LevelEditorModule.GetFirstActiveViewport());
		}
	}
}

bool FLevelEditorActionCallbacks::PlayFromHere_IsVisible()
{
	if (GEditor->GetSelectedActorCount() == 1)
	{
		if (const AActor* Actor = Cast<AActor>(*GEditor->GetSelectedActorIterator()))
		{
			return Actor->CanPlayFromHere();
		}
	}

	return false;
}

void FLevelEditorActionCallbacks::PlayFromHere()
{
	FPlayWorldCommandCallbacks::FStartPlayFromHereOptions PlayOptions;

	// Determine whether to spawn a new window or reuse existing viewport
	const ULevelEditorPlaySettings* PlaySettings = GetDefault<ULevelEditorPlaySettings>();
	const bool bSpawnNewWindow = PlaySettings->LastExecutedPlayModeType != EPlayModeType::PlayMode_InViewPort;
	if (!bSpawnNewWindow)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		PlayOptions.DestinationSlateViewport = LevelEditorModule.GetFirstActiveViewport();
	}

	if (GEditor->GetSelectedActorCount() == 1)
	{
		if (const AActor* Actor = Cast<AActor>(*GEditor->GetSelectedActorIterator()))
		{
			if (Actor->CanPlayFromHere())
			{
				// Play from selected actor position
				PlayOptions.Mode = FPlayWorldCommandCallbacks::SelectedActor;
				if (FPlayWorldCommandCallbacks::StartPlayFromHere(PlayOptions))
				{
					return;
				}
			}
			else
			{
				UE_LOGF(LevelEditorActions, Log, "PlayFromHere: Selected actor to play from must set bCanPlayFromHere to true, falling back on last clicked location...");
			}
		}
	}

	// Fallback to click position
	PlayOptions.Mode = FPlayWorldCommandCallbacks::Click;
	FPlayWorldCommandCallbacks::StartPlayFromHere(PlayOptions);
}

void FLevelEditorActionCallbacks::GoHere_Clicked( const FVector* Point )
{
	if( GCurrentLevelEditingViewportClient )
	{
		FVector ZoomToPoint = FVector::ZeroVector;
		if( !Point )
		{
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
				GCurrentLevelEditingViewportClient->Viewport,
				GCurrentLevelEditingViewportClient->GetWorld()->Scene,
				GCurrentLevelEditingViewportClient->EngineShowFlags)
				.SetRealtimeUpdate(true));


			FSceneView* SceneView = GCurrentLevelEditingViewportClient->CalcSceneView(&ViewFamily);

			if(SceneView)
			{
				FIntPoint MousePosition;
				FVector WorldOrigin;
				FVector WorldDirection;
				GCurrentLevelEditingViewportClient->Viewport->GetMousePos(MousePosition);

				SceneView->DeprojectFVector2D(MousePosition, WorldOrigin, WorldDirection);

				FHitResult HitResult;

				FCollisionQueryParams LineParams(SCENE_QUERY_STAT(FocusOnPoint), true);

				if(GCurrentLevelEditingViewportClient->GetWorld()->LineTraceSingleByObjectType(HitResult, WorldOrigin, WorldOrigin + WorldDirection * HALF_WORLD_MAX, FCollisionObjectQueryParams(ECC_WorldStatic), LineParams))
				{
					ZoomToPoint = HitResult.ImpactPoint;
				}
			}
		}
		else
		{
			ZoomToPoint = *Point;
		}

		const float PushOutSize = 500;
		FBox BoundingBox(ZoomToPoint-PushOutSize, ZoomToPoint+PushOutSize);

		GCurrentLevelEditingViewportClient->FocusViewportOnBox(BoundingBox);
	}
}

bool FLevelEditorActionCallbacks::LockActorMovement_IsChecked()
{
	return GEditor->HasSelectedMovementLockedItems();
}

void FLevelEditorActionCallbacks::AddActor_Clicked( UActorFactory* ActorFactory, FAssetData AssetData)
{
	FLevelEditorActionCallbacks::AddActor(ActorFactory, AssetData, NULL);
}

AActor* FLevelEditorActionCallbacks::AddActor( UActorFactory* ActorFactory, const FAssetData& AssetData, const FTransform* ActorTransform )
{
	AActor* NewActor = GEditor->UseActorFactory( ActorFactory, AssetData, ActorTransform );

	if ( NewActor != NULL && IPlacementModeModule::IsAvailable() )
	{
		IPlacementModeModule::Get().AddToRecentlyPlaced( AssetData.GetAsset(), ActorFactory );
	}

	return NewActor;
}

void FLevelEditorActionCallbacks::AddActorFromClass_Clicked( UClass* ActorClass )
{
	FLevelEditorActionCallbacks::AddActorFromClass(ActorClass);
}

AActor* FLevelEditorActionCallbacks::AddActorFromClass( UClass* ActorClass )
{
	AActor* NewActor = NULL;

	if ( ActorClass )
	{
		// Look for an actor factory capable of creating actors of that type.
		UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass( ActorClass );
		if( ActorFactory )
		{
			NewActor = GEditor->UseActorFactoryOnCurrentSelection( ActorFactory, nullptr );

			if ( NewActor != NULL && IPlacementModeModule::IsAvailable() )
			{
				IPlacementModeModule::Get().AddToRecentlyPlaced( ActorClass, ActorFactory );
			}
		}
		else
		{
			// No actor factory was found; use SpawnActor instead.
			GUnrealEd->Exec( GetWorld(), *FString::Printf( TEXT("ACTOR ADD CLASS=%s"), *ActorClass->GetName() ) );
		}
	}

	return NewActor;
}

extern bool GReplaceSelectedActorsWithSelectedClassCopyProperties;
void FLevelEditorActionCallbacks::ReplaceActors_Clicked( UActorFactory* ActorFactory, FAssetData AssetData )
{
	FLevelEditorActionCallbacks::ReplaceActors(ActorFactory, AssetData, GReplaceSelectedActorsWithSelectedClassCopyProperties);
}

AActor* FLevelEditorActionCallbacks::ReplaceActors( UActorFactory* ActorFactory, const FAssetData& AssetData, bool bCopySourceProperties )
{
	AActor* NewActor = NULL;

	// Have a first stab at filling in the factory properties.
	FText ErrorMessage;
	if( ActorFactory->CanCreateActorFrom( AssetData, ErrorMessage ) )
	{
		// Replace all selected actors with actors created from the specified factory
		UEditorActorSubsystem::ReplaceSelectedActors( ActorFactory, AssetData, bCopySourceProperties );

		if ( IPlacementModeModule::IsAvailable() )
		{
			IPlacementModeModule::Get().AddToRecentlyPlaced( AssetData.GetAsset(), ActorFactory );
		}
	}	
	else
	{
		FNotificationInfo ErrorNotification( ErrorMessage );
		ErrorNotification.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));
		ErrorNotification.bFireAndForget = true;
		ErrorNotification.ExpireDuration = 3.0f; // Need this message to last a little longer than normal since the user may want to "Show Log"
		ErrorNotification.bUseThrobber = true;

		FSlateNotificationManager::Get().AddNotification(ErrorNotification);
	}

	return NewActor;
}


void FLevelEditorActionCallbacks::ReplaceActorsFromClass_Clicked( UClass* ActorClass )
{
	if ( ActorClass )
	{
		// Look for an actor factory capable of creating actors of that type.
		UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass( ActorClass );
		if( ActorFactory )
		{
			// Replace all selected actors with actors created from the specified factory
			UObject* TargetAsset = GEditor->GetSelectedObjects()->GetTop<UObject>();

			FText ErrorMessage;
			FText UnusedErrorMessage;
			const FAssetData NoAssetData {};
			const FAssetData TargetAssetData(TargetAsset);
			if( ActorFactory->CanCreateActorFrom( TargetAssetData, ErrorMessage ) )
			{
				// Replace all selected actors with actors created from the specified factory
				UEditorActorSubsystem::ReplaceSelectedActors( ActorFactory, TargetAssetData );
			}
			else if ( ActorFactory->CanCreateActorFrom( NoAssetData, UnusedErrorMessage ) )
			{
				// Replace all selected actors with actors created from the specified factory
				UEditorActorSubsystem::ReplaceSelectedActors( ActorFactory, NoAssetData );
			}
			else
			{
				FNotificationInfo ErrorNotification( ErrorMessage );
				ErrorNotification.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));
				ErrorNotification.bFireAndForget = true;
				ErrorNotification.ExpireDuration = 3.0f; // Need this message to last a little longer than normal since the user may want to "Show Log"
				ErrorNotification.bUseThrobber = true;

				FSlateNotificationManager::Get().AddNotification(ErrorNotification);
			}
		}
		else
		{
			// No actor factory was found; use SpawnActor instead.
			GUnrealEd->Exec( GetWorld(), *FString::Printf( TEXT("ACTOR REPLACE CLASS=%s"), *ActorClass->GetName() ) );
		}
	}
}

bool FLevelEditorActionCallbacks::Duplicate_CanExecute()
{
	const EEditAction::Type CanProcess = GLevelEditorModeTools().GetActionEditDuplicate();
	if (CanProcess == EEditAction::Process)
	{
		return true;
	}
	else if (CanProcess == EEditAction::Halt)
	{
		return false;
	}
	
	static const FName NAME_LevelEditor = "LevelEditor";
	if (TSharedPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(NAME_LevelEditor).GetLevelEditorInstance().Pin())
	{
		bool bCanDuplicate = false;

		const UTypedElementSelectionSet* SelectionSet = LevelEditor->GetElementSelectionSet();
		SelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>([&bCanDuplicate](const TTypedElement<ITypedElementWorldInterface>& InWorldElement)
		{
			bCanDuplicate |= InWorldElement.CanDuplicateElement();
			return !bCanDuplicate;
		});

		if (!bCanDuplicate)
		{
			if (TSharedPtr<ISceneOutliner> SceneOutlinerPtr = LevelEditor->GetMostRecentlyUsedSceneOutliner())
			{
				bCanDuplicate = SceneOutlinerPtr->Copy_CanExecute(); // If we can copy, we can duplicate
			}
		}

		return bCanDuplicate;
	}

	return false;
}

bool FLevelEditorActionCallbacks::Delete_CanExecute()
{
	const EEditAction::Type CanProcess = GLevelEditorModeTools().GetActionEditDelete();
	if (CanProcess == EEditAction::Process)
	{
		return true;
	}
	else if (CanProcess == EEditAction::Halt)
	{
		return false;
	}

	static const FName NAME_LevelEditor = "LevelEditor";
	if (TSharedPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(NAME_LevelEditor).GetLevelEditorInstance().Pin())
	{
		bool bCanDelete = false;

		const UTypedElementSelectionSet* SelectionSet = LevelEditor->GetElementSelectionSet();
		SelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>([&bCanDelete](const TTypedElement<ITypedElementWorldInterface>& InWorldElement)
		{
			bCanDelete |= InWorldElement.CanDeleteElement();
			return !bCanDelete;
		});

		if (!bCanDelete)
		{
			if (TSharedPtr<ISceneOutliner> SceneOutlinerPtr = LevelEditor->GetMostRecentlyUsedSceneOutliner())
			{
				bCanDelete = SceneOutlinerPtr->Delete_CanExecute();
			}
		}

		return bCanDelete;
	}

	return false;
}

void FLevelEditorActionCallbacks::Rename_Execute()
{
	UActorComponent* Component = Cast<UActorComponent>(*GEditor->GetSelectedComponentIterator());
	if (Component)
	{
		GEditor->BroadcastLevelComponentRequestRename(Component);
	}
	else if (AActor* Actor = Cast<AActor>(*GEditor->GetSelectedActorIterator()))
	{
		GEditor->BroadcastLevelActorRequestRename(Actor);
	}
	else
	{
		TWeakPtr<class ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
		if (LevelEditor.IsValid())
		{
			TSharedPtr<class ISceneOutliner> SceneOutlinerPtr = LevelEditor.Pin()->GetMostRecentlyUsedSceneOutliner();
			if (SceneOutlinerPtr.IsValid())
			{
				SceneOutlinerPtr->Rename_Execute();
			}
		}
	}
}

bool FLevelEditorActionCallbacks::Rename_CanExecute()
{
	bool bCanRename = false;
	if (GEditor->GetSelectedComponentCount() == 1)
	{
		if (UActorComponent* ComponentToRename = GEditor->GetSelectedComponents()->GetTop<UActorComponent>())
		{
			// We can't edit non-instance components or the default scene root
			bCanRename = ComponentToRename->CreationMethod == EComponentCreationMethod::Instance && ComponentToRename->GetFName() != USceneComponent::GetDefaultSceneRootVariableName();
		}
	}
	else
	{
		bCanRename = GEditor->GetSelectedActorCount() == 1;
	}

	if (!bCanRename)
	{
		TWeakPtr<class ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
		if (LevelEditor.IsValid())
		{
			TSharedPtr<class ISceneOutliner> SceneOutlinerPtr = LevelEditor.Pin()->GetMostRecentlyUsedSceneOutliner();
			if (SceneOutlinerPtr.IsValid())
			{
				bCanRename = SceneOutlinerPtr->Rename_CanExecute();
			}
		}
	}

	return bCanRename;
}

bool FLevelEditorActionCallbacks::Cut_CanExecute()
{
	const EEditAction::Type CanProcess = GLevelEditorModeTools().GetActionEditCut();
	if (CanProcess == EEditAction::Process)
	{
		return true;
	}
	else if (CanProcess == EEditAction::Halt)
	{
		return false;
	}

	bool bCanCut = false;

	FTypedElementCommonActions_CopyPasteCapability CanCopy;
	{
		static const FName NAME_LevelEditor = "LevelEditor";
		if (TSharedPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(NAME_LevelEditor).GetLevelEditorInstance().Pin())
		{
			const UTypedElementSelectionSet* SelectionSet = LevelEditor->GetElementSelectionSet();
			UTypedElementCommonActions* CommonActions = LevelEditor->GetCommonActions();
			
			if (CommonActions && SelectionSet)
			{
				CanCopy = CommonActions->CanCopySelectedElements(SelectionSet);
			}
		}
	}
	
	// If the level editor common action customizations are capable of copying the elements
	// then defer to it to allowing cut
	if (CanCopy.bCapable)
	{
		bCanCut = CanCopy.bAllowed;
	} 
	else if (GEditor->GetSelectedComponentCount() > 0)
	{
		// Make sure the components can be copied and deleted
		TArray<UActorComponent*> SelectedComponents;
		for (FSelectionIterator It(GEditor->GetSelectedComponentIterator()); It; ++It)
		{
			SelectedComponents.Add(CastChecked<UActorComponent>(*It));
		}

		bCanCut = FComponentEditorUtils::CanCopyComponents(SelectedComponents) && FComponentEditorUtils::CanDeleteComponents(SelectedComponents);
	}
	else
	{
		// For actors, if we can copy, we can cut
		UWorld* World = GetWorld();
		if (World)
		{
			bCanCut = GUnrealEd->CanCopySelectedActorsToClipboard(World);
		}
	}

	if (!bCanCut)
	{
		TWeakPtr<class ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
		if (LevelEditor.IsValid())
		{
			TSharedPtr<class ISceneOutliner> SceneOutlinerPtr = LevelEditor.Pin()->GetMostRecentlyUsedSceneOutliner();
			if (SceneOutlinerPtr.IsValid())
			{
				bCanCut = SceneOutlinerPtr->Cut_CanExecute();
			}
		}
	}

	return bCanCut;
}

bool FLevelEditorActionCallbacks::Copy_CanExecute()
{
	const EEditAction::Type CanProcess = GLevelEditorModeTools().GetActionEditCopy();
	if (CanProcess == EEditAction::Process)
	{
		return true;
	}
	else if (CanProcess == EEditAction::Halt)
	{
		return false;
	}

	bool bCanCopy = false;
	
	FTypedElementCommonActions_CopyPasteCapability CanCopy;
	{
		static const FName NAME_LevelEditor = "LevelEditor";
		if (TSharedPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(NAME_LevelEditor).GetLevelEditorInstance().Pin())
		{
			const UTypedElementSelectionSet* SelectionSet = LevelEditor->GetElementSelectionSet();
			UTypedElementCommonActions* CommonActions = LevelEditor->GetCommonActions();
			
			if (CommonActions && SelectionSet)
			{
				CanCopy = CommonActions->CanCopySelectedElements(SelectionSet);
			}
		}
	}
	
	// If the level editor common action customizations are capable of copying the elements
	// then defer to it to allowing copy
	if (CanCopy.bCapable)
	{
		bCanCopy = CanCopy.bAllowed;
	} 
	else if (GEditor->GetSelectedComponentCount() > 0)
	{
		TArray<UActorComponent*> SelectedComponents;
		for (FSelectionIterator It(GEditor->GetSelectedComponentIterator()); It; ++It)
		{
			SelectedComponents.Add(CastChecked<UActorComponent>(*It));
		}

		bCanCopy = FComponentEditorUtils::CanCopyComponents(SelectedComponents);
	}
	else
	{
		UWorld* World = GetWorld();
		if (World)
		{
			bCanCopy = GUnrealEd->CanCopySelectedActorsToClipboard(World);
		}
	}

	if (!bCanCopy)
	{
		TWeakPtr<class ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
		if (LevelEditor.IsValid())
		{
			TSharedPtr<class ISceneOutliner> SceneOutlinerPtr = LevelEditor.Pin()->GetMostRecentlyUsedSceneOutliner();
			if (SceneOutlinerPtr.IsValid())
			{
				bCanCopy = SceneOutlinerPtr->Copy_CanExecute();
			}
		}
	}

	return bCanCopy;
}

bool FLevelEditorActionCallbacks::Paste_CanExecute()
{
	const EEditAction::Type CanProcess = GLevelEditorModeTools().GetActionEditPaste();
	if (CanProcess == EEditAction::Process)
	{
		return true;
	}
	else if (CanProcess == EEditAction::Halt)
	{
		return false;
	}

	FTypedElementCommonActions_CopyPasteCapability CanPaste;
	{
		static const FName NAME_LevelEditor = "LevelEditor";
		if (TSharedPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(NAME_LevelEditor).GetLevelEditorInstance().Pin())
		{
			UTypedElementSelectionSet* SelectionSet = LevelEditor->GetMutableElementSelectionSet();
			UTypedElementCommonActions* CommonActions = LevelEditor->GetCommonActions();
			
			if (CommonActions && SelectionSet)
			{
				FTypedElementPasteOptions Options;
				Options.SelectionSetToModify = SelectionSet;
				CanPaste = CommonActions->CanPasteElements(Options);
			}
		}
	}
	
	bool bCanPaste = false;
	if (CanPaste.bCapable)
	{
		bCanPaste = CanPaste.bAllowed;
	}
	// Legacy style copy and paste format
	else
	{
		if (GEditor->GetSelectedComponentCount() > 0)
		{
			if(ensureMsgf(GEditor->GetSelectedActorCount() == 1, TEXT("Expected SelectedActorCount to be 1 but was %d"), GEditor->GetSelectedActorCount()))
			{
				auto SelectedActor = CastChecked<AActor>(*GEditor->GetSelectedActorIterator());
				bCanPaste = FComponentEditorUtils::CanPasteComponents(SelectedActor->GetRootComponent());
			}
		}
		else
		{
			UWorld* World = GetWorld();
			if (World)
			{
				bCanPaste = GUnrealEd->CanPasteSelectedActorsFromClipboard(World);
			}
		}
	}

	if (!bCanPaste)
	{
		TWeakPtr<class ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
		if (LevelEditor.IsValid())
		{
			TSharedPtr<class ISceneOutliner> SceneOutlinerPtr = LevelEditor.Pin()->GetMostRecentlyUsedSceneOutliner();
			if (SceneOutlinerPtr.IsValid())
			{
				bCanPaste = SceneOutlinerPtr->Paste_CanExecute();
			}
		}
	}

	return bCanPaste;
}

bool FLevelEditorActionCallbacks::PasteHere_CanExecute()
{
	return Paste_CanExecute();	// For now, just do the same check as Paste
}

void FLevelEditorActionCallbacks::ExecuteExecCommand( FString Command )
{
	UWorld* OldWorld = nullptr;

	// The play world needs to be selected if it exists
	if (GIsEditor && GEditor->PlayWorld && !GIsPlayInEditorWorld)
	{
		OldWorld = SetPlayInEditorWorld(GEditor->PlayWorld);
	}

	GUnrealEd->Exec(GetWorld(), *Command);

	// Restore the old world if there was one
	if (OldWorld)
	{
		RestoreEditorWorld(OldWorld);
	}
}

void FLevelEditorActionCallbacks::OnSelectAllActorsOfClass( bool bArchetype )
{
	GEditor->SelectAllActorsWithClass( bArchetype );
}

bool FLevelEditorActionCallbacks::CanSelectAllActorsOfClass()
{
	return GEditor->GetSelectedActorCount() > 0;
}

void FLevelEditorActionCallbacks::OnSelectComponentOwnerActor()
{
	auto ComponentOwner = Cast<AActor>(*GEditor->GetSelectedActorIterator());
	check(ComponentOwner);

	GEditor->SelectNone(true, true, false);
	GEditor->SelectActor(ComponentOwner, true, true, true);
}

bool FLevelEditorActionCallbacks::CanSelectComponentOwnerActor()
{
	return GEditor->GetSelectedComponentCount() > 0;
}

bool FLevelEditorActionCallbacks::CanHideSelectedActors()
{
	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		if (!Actor->IsHiddenEd())
		{
			return true;
		}
	}

	return LevelEditorActionsHelpers::HaveSelectedVisibleNonActors(GetWorld());
}

bool FLevelEditorActionCallbacks::CanShowSelectedActors()
{
	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		if (Actor->IsHiddenEd())
		{
			return true;
		}
	}

	return LevelEditorActionsHelpers::HaveSelectedHiddenNonActors(GetWorld());
}

bool FLevelEditorActionCallbacks::CanToggleSelectedActorsVisibility()
{
	return GEditor->GetSelectedActorCount() > 0
		|| LevelEditorActionsHelpers::HaveSelectedHideableNonActors(GetWorld());
}

void FLevelEditorActionCallbacks::OnSelectOwningHLODCluster()
{
	if (GEditor->GetSelectedActorCount() > 0)
	{
		AActor* Actor = Cast<AActor>(GEditor->GetSelectedActors()->GetSelectedObject(0));

		FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
		IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

		ALODActor* ParentActor = Utilities->GetParentLODActor(Actor);
		if (Actor && ParentActor)
		{
			GEditor->SelectNone(false, true);
			GEditor->SelectActor(ParentActor, true, false);
			GEditor->NoteSelectionChange();
		}
	}
}

void FLevelEditorActionCallbacks::OnApplyMaterialToSurface()
{
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	GUnrealEd->Exec( GetWorld(), TEXT("POLY SETMATERIAL") );
}

void FLevelEditorActionCallbacks::OnSelectAllLights()
{
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();

	GEditor->SelectNone(false, true);

	// Select all light actors.
	for( ALight* Light : TActorRange<ALight>(GetWorld()) )
	{
		GUnrealEd->SelectActor( Light, true, false, false );
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation();

}

void FLevelEditorActionCallbacks::OnSelectStationaryLightsExceedingOverlap()
{
	GEditor->SelectNone( true, true );
	for( FActorIterator It(GetWorld()); It; ++It )
	{
		AActor* Actor = *It;

		TInlineComponentArray<ULightComponent*> Components;
		Actor->GetComponents(Components);

		for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
		{
			ULightComponent* LightComponent = Components[ComponentIndex];

			if (LightComponent->GetOwner()
				// Use the component's lighting properties to determine if this is a stationary light, instead of checking the actor type
				// Because blueprint lights may be operating as stationary lights
				&& LightComponent->HasStaticShadowing()
				&& !LightComponent->HasStaticLighting()
				&& LightComponent->bAffectsWorld
				&& LightComponent->CastShadows
				&& LightComponent->CastStaticShadows
				&& LightComponent->PreviewShadowMapChannel == INDEX_NONE)
			{
				GUnrealEd->SelectActor( LightComponent->GetOwner(), true, true, false );
			}
		}
	}
}

void FLevelEditorActionCallbacks::OnSurfaceAlignment( ETexAlign AlignmentMode )
{
	GTexAlignTools.GetAligner( AlignmentMode )->Align( GetWorld(), AlignmentMode );
}

bool FLevelEditorActionCallbacks::GroupActors_CanExecute()
{
	return UActorGroupingUtils::Get()->CanGroupSelectedActors();
}

void FLevelEditorActionCallbacks::RegroupActor_Clicked()
{
	UActorGroupingUtils::Get()->GroupSelected();
}

void FLevelEditorActionCallbacks::UngroupActor_Clicked()
{
	UActorGroupingUtils::Get()->UngroupSelected();
}

void FLevelEditorActionCallbacks::LockGroup_Clicked()
{
	UActorGroupingUtils::Get()->LockSelectedGroups();
}

void FLevelEditorActionCallbacks::UnlockGroup_Clicked()
{
	UActorGroupingUtils::Get()->UnlockSelectedGroups();
}

void FLevelEditorActionCallbacks::AddActorsToGroup_Clicked()
{
	UActorGroupingUtils::Get()->AddSelectedToGroup();
}

void FLevelEditorActionCallbacks::RemoveActorsFromGroup_Clicked()
{
	UActorGroupingUtils::Get()->RemoveSelectedFromGroup();
}


void FLevelEditorActionCallbacks::LocationGridSnap_Clicked()
{
	GUnrealEd->Exec( GetWorld(), *FString::Printf( TEXT("MODE GRID=%d"), !GetDefault<ULevelEditorViewportSettings>()->GridEnabled ? 1 : 0 ) );	
}


bool FLevelEditorActionCallbacks::LocationGridSnap_IsChecked()
{
	return GetDefault<ULevelEditorViewportSettings>()->GridEnabled;
}


void FLevelEditorActionCallbacks::RotationGridSnap_Clicked()
{
	GUnrealEd->Exec( GetWorld(), *FString::Printf( TEXT("MODE ROTGRID=%d"), !GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled ? 1 : 0 ) );
}


bool FLevelEditorActionCallbacks::RotationGridSnap_IsChecked()
{
	return GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled;
}


void FLevelEditorActionCallbacks::ScaleGridSnap_Clicked()
{
	GUnrealEd->Exec( GetWorld(), *FString::Printf( TEXT("MODE SCALEGRID=%d"), !GetDefault<ULevelEditorViewportSettings>()->SnapScaleEnabled ? 1 : 0 ) );
}

bool FLevelEditorActionCallbacks::ScaleGridSnap_IsChecked()
{
	return GetDefault<ULevelEditorViewportSettings>()->SnapScaleEnabled;
}

bool FLevelEditorActionCallbacks::SaveAnimationFromSkeletalMeshComponent(AActor * EditorActor, AActor * SimActor, TArray<USkeletalMeshComponent*> & OutEditorComponents)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );

	// currently blueprint actors don't work because their property can't get copied over. 
	if (Cast<UBlueprintGeneratedClass>(EditorActor->GetClass()) != nullptr)
	{
		return false;
	}

	// find all skel components
	TInlineComponentArray<USkeletalMeshComponent*> SimSkelComponents;
	SimActor->GetComponents(SimSkelComponents);

	if(SimSkelComponents.Num() > 0)
	{
		// see if simulating, 
		bool bSimulating = false;
		for (auto & Comp : SimSkelComponents)
		{
			bSimulating |= (Comp->GetSkeletalMeshAsset() && Comp->GetSkeletalMeshAsset()->GetSkeleton() && Comp->IsSimulatingPhysics());
		}

		// if any of them are legitimately simulating
		if (bSimulating)
		{
			// ask users if you'd like to make an animation
			FFormatNamedArguments Args;
			Args.Add(TEXT("ActorName"), FText::FromString(GetNameSafe(EditorActor)));
			FText AskQuestion = FText::Format(LOCTEXT("KeepSimulationChanges_AskSaveAnimation", "Would you like to save animations from simulation for {ActorName} actor"), Args);
			if(EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, AskQuestion))
			{
				for (auto & Comp : SimSkelComponents)
				{
					if (Comp->GetSkeletalMeshAsset() && Comp->GetSkeletalMeshAsset()->GetSkeleton() && Comp->IsSimulatingPhysics())
					{
						// now record to animation
						class UAnimSequence* Sequence = LevelEditorModule.OnCaptureSingleFrameAnimSequence().IsBound() ? LevelEditorModule.OnCaptureSingleFrameAnimSequence().Execute(Comp) : nullptr;
						if(Sequence)
						{
							Comp->SetAnimationMode(EAnimationMode::AnimationSingleNode);
							Comp->AnimationData.AnimToPlay = Sequence;
							Comp->SetAnimation(Sequence);
							Comp->SetSimulatePhysics(false);

							// add the matching component to EditorCompoennts
							class USkeletalMeshComponent * MatchingComponent = Cast<USkeletalMeshComponent>(EditorUtilities::FindMatchingComponentInstance(Comp, EditorActor));
							if (MatchingComponent)
							{
								OutEditorComponents.Add(MatchingComponent);
							}
							else
							{
								UE_LOGF(LevelEditorActions, Warning, "Matching component could not be found %ls(%ls)", *GetNameSafe(Comp), *GetNameSafe(EditorActor));
							}
						}
					}
				}

				return true;
			}
		}
	}

	return false;
}

void FLevelEditorActionCallbacks::OpenMergeActor_Clicked()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FName("MergeActors"));
}

void FLevelEditorActionCallbacks::OnKeepSimulationChanges()
{
	// @todo simulate: There are lots of types of changes that can't be "kept", like attachment or newly-spawned actors.  This
	//    feature currently only supports propagating changes to regularly-editable properties on an instance of a PIE actor
	//    that still exists in the editor world.

	// Make sure we have some actors selected, and PIE is running
	if( GEditor->GetSelectedActorCount() > 0 && GEditor->PlayWorld != NULL )
	{
		int32 UpdatedActorCount = 0;
		int32 TotalCopiedPropertyCount = 0;
		FString FirstUpdatedActorLabel;
		{
			const FScopedTransaction Transaction( NSLOCTEXT( "LevelEditorCommands", "KeepSimulationChanges", "Keep Simulation Changes" ) );

			TArray<class USkeletalMeshComponent*> ComponentsToReinitialize;

			for( auto ActorIt( GEditor->GetSelectedActorIterator() ); ActorIt; ++ActorIt )
			{
				auto* SimWorldActor = CastChecked<AActor>( *ActorIt );

				// Find our counterpart actor
				AActor* EditorWorldActor = EditorUtilities::GetEditorWorldCounterpartActor( SimWorldActor );
				if( EditorWorldActor != NULL )
				{
					SaveAnimationFromSkeletalMeshComponent(EditorWorldActor, SimWorldActor, ComponentsToReinitialize);

					// We only want to copy CPF_Edit properties back, or properties that are set through editor manipulation
					// NOTE: This needs to match what we're doing in the BuildSelectedActorInfo() function
					const auto CopyOptions = ( EditorUtilities::ECopyOptions::Type )(
						EditorUtilities::ECopyOptions::CallPostEditChangeProperty |
						EditorUtilities::ECopyOptions::CallPostEditMove |
						EditorUtilities::ECopyOptions::OnlyCopyEditOrInterpProperties |
						EditorUtilities::ECopyOptions::FilterBlueprintReadOnly);
					const int32 CopiedPropertyCount = EditorUtilities::CopyActorProperties( SimWorldActor, EditorWorldActor, CopyOptions );

					if( CopiedPropertyCount > 0 )
					{
						++UpdatedActorCount;
						TotalCopiedPropertyCount += CopiedPropertyCount;

						if( FirstUpdatedActorLabel.IsEmpty() )
						{
							FirstUpdatedActorLabel = EditorWorldActor->GetActorLabel();
						}
					}
				}

				// need to reinitialize animation
				for (auto MeshComp : ComponentsToReinitialize)
				{
					if(MeshComp->GetSkeletalMeshAsset())
					{
						MeshComp->InitAnim(true);
					}
				}
			}
		}

		// Let the user know what happened
		{
			FNotificationInfo NotificationInfo( FText::GetEmpty() );
			NotificationInfo.bFireAndForget = true;
			NotificationInfo.FadeInDuration = 0.25f;
			NotificationInfo.FadeOutDuration = 1.0f;
			NotificationInfo.ExpireDuration = 1.0f;
			NotificationInfo.bUseLargeFont = false;
			NotificationInfo.bUseSuccessFailIcons = true;
			NotificationInfo.bAllowThrottleWhenFrameRateIsLow = false;	// Don't throttle as it causes distracting hitches in Simulate mode
			SNotificationItem::ECompletionState CompletionState;
			if( UpdatedActorCount > 0 )
			{
				if( UpdatedActorCount > 1 )
				{
					FFormatNamedArguments Args;
					Args.Add( TEXT("UpdatedActorCount"), UpdatedActorCount );
					Args.Add( TEXT("TotalCopiedPropertyCount"), TotalCopiedPropertyCount );
					NotificationInfo.Text = FText::Format( NSLOCTEXT( "LevelEditorCommands", "KeepSimulationChanges_MultipleActorsUpdatedNotification", "Saved state for {UpdatedActorCount} actors  ({TotalCopiedPropertyCount} properties)" ), Args );
				}
				else
				{
					FFormatNamedArguments Args;
					Args.Add( TEXT("FirstUpdatedActorLabel"), FText::FromString( FirstUpdatedActorLabel ) );
					Args.Add( TEXT("TotalCopiedPropertyCount"), TotalCopiedPropertyCount );
					NotificationInfo.Text = FText::Format( NSLOCTEXT( "LevelEditorCommands", "KeepSimulationChanges_ActorUpdatedNotification", "Saved state for {FirstUpdatedActorLabel} ({TotalCopiedPropertyCount} properties)" ), Args );
				}
				CompletionState = SNotificationItem::CS_Success;
			}
			else
			{
				NotificationInfo.Text = NSLOCTEXT( "LevelEditorCommands", "KeepSimulationChanges_NoActorsUpdated", "No properties were copied" );
				CompletionState = SNotificationItem::CS_Fail;
			}
			const auto Notification = FSlateNotificationManager::Get().AddNotification( NotificationInfo );
			Notification->SetCompletionState( CompletionState );
		}
	}
}


bool FLevelEditorActionCallbacks::CanExecuteKeepSimulationChanges()
{
	return AssetSelectionUtils::GetSelectedActorInfo().NumSimulationChanges > 0;
}


void FLevelEditorActionCallbacks::OnMakeSelectedActorLevelCurrent()
{
	GUnrealEd->MakeSelectedActorsLevelCurrent();
}

void FLevelEditorActionCallbacks::OnMoveSelectedToCurrentLevel()
{
	UEditorLevelUtils::MoveSelectedActorsToLevel(GetWorld()->GetCurrentLevel());
}

void FLevelEditorActionCallbacks::OnFindActorLevelInContentBrowser()
{
	GEditor->SyncActorLevelsToContentBrowser();
}

bool FLevelEditorActionCallbacks::CanExecuteFindActorLevelInContentBrowser()
{
	return GEditor->CanSyncActorLevelsToContentBrowser();
}

void FLevelEditorActionCallbacks::OnFindLevelsInLevelBrowser()
{
	const bool bDeselectOthers = true;
	GEditor->SelectLevelInLevelBrowser( bDeselectOthers );
}

void FLevelEditorActionCallbacks::OnSelectLevelInLevelBrowser()
{
	const bool bDeselectOthers = false;
	GEditor->SelectLevelInLevelBrowser( bDeselectOthers );
}

void FLevelEditorActionCallbacks::OnDeselectLevelInLevelBrowser()
{
	GEditor->DeselectLevelInLevelBrowser();
}

void FLevelEditorActionCallbacks::OnFindActorInLevelScript()
{
	GUnrealEd->FindSelectedActorsInLevelScript();
}

void FLevelEditorActionCallbacks::OnShowWorldProperties( TWeakPtr< SLevelEditor > LevelEditor )
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(FName("WorldSettingsTab"));
}

void FLevelEditorActionCallbacks::OnFocusOutlinerToSelection(TWeakPtr<SLevelEditor> LevelEditor)
{
	if (const TSharedPtr<SLevelEditor> Editor = LevelEditor.Pin())
	{
		for (TWeakPtr<ISceneOutliner> SceneOutliner : Editor->GetAllSceneOutliners())
		{
			if (const TSharedPtr<ISceneOutliner> Outliner = SceneOutliner.Pin())
			{
				Outliner->FrameSelectedItems();
			}
		}
	}
}

void FLevelEditorActionCallbacks::OnFocusOutlinerToContextFolder(TWeakPtr<SLevelEditor> LevelEditor)
{
	if (const TSharedPtr<SLevelEditor> Editor = LevelEditor.Pin())
	{
		if (UWorld* World = Editor->GetWorld())
		{
			const FFolder ContextFolder = FActorFolders::Get().GetActorEditorContextFolder(*World);
			if (ContextFolder.IsValid())
			{
				for (TWeakPtr<ISceneOutliner> SceneOutliner : Editor->GetAllSceneOutliners())
				{
					if (const TSharedPtr<ISceneOutliner> Outliner = SceneOutliner.Pin())
					{
						Outliner->FrameItem(ContextFolder);
					}
				}
			}
		}
	}
}

void FLevelEditorActionCallbacks::OpenPlaceActors()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(LevelEditorTabIds::PlacementBrowser);
}

void FLevelEditorActionCallbacks::OpenContentBrowser()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().FocusPrimaryContentBrowser(true);
}

void FLevelEditorActionCallbacks::ImportContent()
{
	FString Path = "/Game";

	//Ask the user for the root path where they want to any content to be placed
	TSharedRef<SDlgPickPath> PickContentPathDlg =
		SNew(SDlgPickPath)
		.Title(LOCTEXT("ChooseImportRootContentPath", "Choose a location to import the content into"));

	if (PickContentPathDlg->ShowModal() == EAppReturnType::Cancel)
	{
		return;
	}

	Path = PickContentPathDlg->GetPath().ToString();

	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

	AssetToolsModule.Get().ImportAssetsWithDialogAsync(Path);
}

void FLevelEditorActionCallbacks::ToggleVR()
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	VREditorModule.EnableVREditor( VREditorModule.GetVRModeBase() == nullptr );
}


bool FLevelEditorActionCallbacks::ToggleVR_CanExecute()
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	return VREditorModule.IsVREditorAvailable();
}

bool FLevelEditorActionCallbacks::ToggleVR_IsButtonActive()
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	return VREditorModule.IsVREditorButtonActive();
}

bool FLevelEditorActionCallbacks::ToggleVR_IsChecked()
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	return VREditorModule.IsVREditorEnabled();
}

bool FLevelEditorActionCallbacks::CanSelectGameModeBlueprint()
{
	bool bCheckOutNeeded = false;

	FString ConfigFilePath = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%sDefaultEngine.ini"), *FPaths::SourceConfigDir()));
	if(ISourceControlModule::Get().IsEnabled())
	{
		// note: calling QueueStatusUpdate often does not spam status updates as an internal timer prevents this
		//ISourceControlModule::Get().QueueStatusUpdate(ConfigFilePath);

		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(ConfigFilePath, EStateCacheUsage::Use);
		bCheckOutNeeded = SourceControlState.IsValid() && SourceControlState->CanCheckout();
	}
	else
	{
		bCheckOutNeeded = (FPaths::FileExists(ConfigFilePath) && IFileManager::Get().IsReadOnly(*ConfigFilePath));
	}
	return !bCheckOutNeeded;
}

void FLevelEditorActionCallbacks::OpenLevelBlueprint( TWeakPtr< SLevelEditor > LevelEditor )
{
	if( LevelEditor.Pin()->GetWorld()->GetCurrentLevel() )
	{
		ULevelScriptBlueprint* LevelScriptBlueprint = LevelEditor.Pin()->GetWorld()->PersistentLevel->GetLevelScriptBlueprint();
		if (LevelScriptBlueprint)
		{
			// @todo Re-enable once world centric works
			const bool bOpenWorldCentric = false;
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
				LevelScriptBlueprint,
				bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
				LevelEditor.Pin()  );
		}
		else
		{
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "UnableToCreateLevelScript", "Unable to find or create a level blueprint for this level.") );
		}
	}
}

void FLevelEditorActionCallbacks::CreateBlankBlueprintClass()
{
	// Use the BlueprintFactory to allow the user to pick a parent class for the new Blueprint class
	UBlueprintFactory* NewFactory = Cast<UBlueprintFactory>(NewObject<UFactory>(GetTransientPackage(), UBlueprintFactory::StaticClass()));
	FEditorDelegates::OnConfigureNewAssetProperties.Broadcast(NewFactory);
	if ( NewFactory->ConfigureProperties() )
	{
		UClass* SelectedClass = NewFactory->ParentClass;

		// Now help the user pick a path and name for the new Blueprint
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(NSLOCTEXT("LevelEditorCommands", "CreateBlankBlueprintClass_Title", "Create Blank Blueprint Class"), SelectedClass);

		if( Blueprint )
		{
			// @todo Re-enable once world centric works
			const bool bOpenWorldCentric = false;
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
				Blueprint,
				bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone);
		}
	}
}

bool FLevelEditorActionCallbacks::CanConvertSelectedActorsIntoBlueprintClass()
{
	return (FCreateBlueprintFromActorDialog::GetValidCreationMethods() != ECreateBlueprintFromActorMode::None);
}

void FLevelEditorActionCallbacks::ConvertSelectedActorsIntoBlueprintClass()
{
	const ECreateBlueprintFromActorMode ValidCreateModes = FCreateBlueprintFromActorDialog::GetValidCreationMethods();
	ECreateBlueprintFromActorMode DefaultCreateMode = ECreateBlueprintFromActorMode::Harvest;

	// Check all of the selected actors for any that can't be converted
	bool bHasAnyValidActors = false;
	TArray<AActor*> UnconvertibleSelectedActors;

	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			if (FKismetEditorUtilities::CanCreateBlueprintOfClass(Actor->GetClass()))
			{
				bHasAnyValidActors = true;
			}
			else
			{
				UnconvertibleSelectedActors.Add(Actor);
			}
		}
	}

	if (UnconvertibleSelectedActors.Num())
	{
		// Let the user know that some or all of the selected actors are not convertible to BP. 
		FString UnconvertedActorsList = FString::JoinBy(UnconvertibleSelectedActors, TEXT("\n"), [](const AActor* Actor) 
			{
				return FString::Format(TEXT("{0} (type '{1}')"), { Actor->GetName(), Actor->GetClass()->GetName() });
			});

		if (bHasAnyValidActors)
		{
			// If there are some convertible actors, give the user a choice to proceed with only the valid ones
			const FText Message = FText::Format(LOCTEXT("ConfirmPartialConversionToBlueprint", "These selected actors cannot be used to create a blueprint. Do you want to continue conversion without them?\n\n{0}"), FText::FromString(UnconvertedActorsList));
			if (FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::No, Message) == EAppReturnType::No)
			{
				return;
			}
		}
		else
		{
			// There are no convertible actors. Just let the user know and bail.
			const FText Message = FText::Format(LOCTEXT("SelectedActorsCannotBeBlueprint", "No selected actors can be used to create a blueprint:\n\n{0}"), FText::FromString(UnconvertedActorsList));
			FMessageDialog::Open(EAppMsgType::Ok, Message);
			return;
		}

		// Deselect the unconvertible actors and clear any surface selection (common with unconvertible Brush actors)
		GEditor->GetSelectedActors()->BeginBatchSelectOperation();
		GEditor->DeselectAllSurfaces();

		const bool bShouldSelect = false;
		const bool bShouldNotify = false;
		for (AActor* UnconvertibleActor : UnconvertibleSelectedActors)
		{
			GEditor->SelectActor(UnconvertibleActor, bShouldSelect, bShouldNotify);
		}

		GEditor->GetSelectedActors()->EndBatchSelectOperation(bShouldNotify);
		GEditor->NoteSelectionChange();
	}

	if (!!(ValidCreateModes & ECreateBlueprintFromActorMode::Subclass) && (GEditor->GetSelectedActorCount() == 1))
	{
		// If a single actor is selected and it can be subclassed, use that as default
		DefaultCreateMode = ECreateBlueprintFromActorMode::Subclass;
	}
	else if (!!(ValidCreateModes & ECreateBlueprintFromActorMode::ChildActor))
	{
		// Otherwise if there is an actor that can be spawned as a child actor, use that as default
		DefaultCreateMode = ECreateBlueprintFromActorMode::ChildActor;
	}

	FCreateBlueprintFromActorDialog::OpenDialog(DefaultCreateMode);
}

void FLevelEditorActionCallbacks::CheckOutProjectSettingsConfig( )
{
	FString ConfigFilePath = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%sDefaultEngine.ini"), *FPaths::SourceConfigDir()));
	if(ISourceControlModule::Get().IsEnabled())
	{
		USourceControlHelpers::CheckOutOrAddFile(ConfigFilePath);
	}
	else
	{
		FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*ConfigFilePath, false);
	}
}

void FLevelEditorActionCallbacks::OnShowOnlySelectedActors()
{
	const FScopedTransaction Transaction( NSLOCTEXT( "LevelEditorCommands", "ShowOnlySelectedActors", "Show Only Selected Actors" ) );
	// First hide unselected as this will also hide group actor members
	GUnrealEd->edactHideUnselected( GetWorld() );
	// Then unhide selected to ensure that everything that's selected will be unhidden
	GUnrealEd->edactUnhideSelected(GetWorld());
}

void FLevelEditorActionCallbacks::OnToggleTransformWidgetVisibility()
{
	GLevelEditorModeTools().SetShowWidget( !GLevelEditorModeTools().GetShowWidget() );
	GUnrealEd->RedrawAllViewports();
}

bool FLevelEditorActionCallbacks::OnGetTransformWidgetVisibility()
{
	return GLevelEditorModeTools().GetShowWidget();
}

void FLevelEditorActionCallbacks::OnToggleShowSelectionSubcomponents()
{
	UEditorPerProjectUserSettings* Settings = GetMutableDefault<UEditorPerProjectUserSettings>();
	Settings->bShowSelectionSubcomponents = !Settings->bShowSelectionSubcomponents;
	Settings->PostEditChange();

	GUnrealEd->RedrawAllViewports();
}

bool FLevelEditorActionCallbacks::OnGetShowSelectionSubcomponents()
{
	return GetDefault<UEditorPerProjectUserSettings>()->bShowSelectionSubcomponents == true;
}

void FLevelEditorActionCallbacks::OnAllowTranslucentSelection()
{
	UEditorPerProjectUserSettings* Settings = GetMutableDefault<UEditorPerProjectUserSettings>();

	// Toggle 'allow select translucent'
	Settings->bAllowSelectTranslucent = !Settings->bAllowSelectTranslucent;
	Settings->PostEditChange();

	// Need to refresh hit proxies as we changed what should be rendered into them
	GUnrealEd->RedrawAllViewports();
}

bool FLevelEditorActionCallbacks::OnIsAllowTranslucentSelectionEnabled()
{
	return GetDefault<UEditorPerProjectUserSettings>()->bAllowSelectTranslucent == true;
}

void FLevelEditorActionCallbacks::OnAllowGroupSelection()
{
	AGroupActor::ToggleGroupMode();
}

bool FLevelEditorActionCallbacks::OnIsAllowGroupSelectionEnabled() 
{
	return UActorGroupingUtils::IsGroupingActive();
}

void FLevelEditorActionCallbacks::OnToggleStrictBoxSelect()
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->bStrictBoxSelection = !ViewportSettings->bStrictBoxSelection;
	ViewportSettings->PostEditChange();
}

bool FLevelEditorActionCallbacks::OnIsStrictBoxSelectEnabled() 
{
	return GetDefault<ULevelEditorViewportSettings>()->bStrictBoxSelection;
}

void FLevelEditorActionCallbacks::OnToggleTransparentBoxSelect()
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->bTransparentBoxSelection = !ViewportSettings->bTransparentBoxSelection;
	ViewportSettings->PostEditChange();
}

bool FLevelEditorActionCallbacks::OnIsTransparentBoxSelectEnabled() 
{
	return GetDefault<ULevelEditorViewportSettings>()->bTransparentBoxSelection;
}

void FLevelEditorActionCallbacks::OnDrawBrushMarkerPolys()
{
	const bool bShowBrushMarkerPolys = GetDefault<ULevelEditorViewportSettings>()->bShowBrushMarkerPolys;
	GEditor->Exec(GetWorld(), *FString::Printf(TEXT("MODE SHOWBRUSHMARKERPOLYS=%d"), !bShowBrushMarkerPolys ? 1 : 0));
	GEditor->SaveConfig();
}

bool FLevelEditorActionCallbacks::OnIsDrawBrushMarkerPolysEnabled()
{
	return GetDefault<ULevelEditorViewportSettings>()->bShowBrushMarkerPolys;
}

void FLevelEditorActionCallbacks::OnToggleOnlyLoadVisibleInPIE()
{
	ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
	PlaySettings->bOnlyLoadVisibleLevelsInPIE = !PlaySettings->bOnlyLoadVisibleLevelsInPIE;
	PlaySettings->PostEditChange();
	PlaySettings->SaveConfig();
}

bool FLevelEditorActionCallbacks::OnIsOnlyLoadVisibleInPIEEnabled()
{
	return GetDefault<ULevelEditorPlaySettings>()->bOnlyLoadVisibleLevelsInPIE;
}

void FLevelEditorActionCallbacks::OnToggleSocketSnapping()
{
	GEditor->bEnableSocketSnapping = !GEditor->bEnableSocketSnapping;
	GEditor->RedrawLevelEditingViewports();
}

bool FLevelEditorActionCallbacks::OnIsSocketSnappingEnabled()
{
	return GEditor->bEnableSocketSnapping;
}

void FLevelEditorActionCallbacks::OnToggleParticleSystemLOD()
{
	GEngine->bEnableEditorPSysRealtimeLOD = !GEngine->bEnableEditorPSysRealtimeLOD;
	GEditor->RedrawLevelEditingViewports();
}

bool FLevelEditorActionCallbacks::OnIsParticleSystemLODEnabled()
{
	return GEditor->bEnableEditorPSysRealtimeLOD;
}

void FLevelEditorActionCallbacks::OnToggleFreezeParticleSimulation()
{
	IConsoleManager& ConsoleManager = IConsoleManager::Get();
	IConsoleVariable* CVar = ConsoleManager.FindConsoleVariable(TEXT("FX.FreezeParticleSimulation"));
	if (CVar)
	{
		CVar->Set(CVar->GetInt() == 0 ? 1 : 0, ECVF_SetByConsole);
	}
}

bool FLevelEditorActionCallbacks::OnIsParticleSimulationFrozen()
{
	IConsoleManager& ConsoleManager = IConsoleManager::Get();
	static const auto* CVar = ConsoleManager.FindConsoleVariable(TEXT("FX.FreezeParticleSimulation"));
	if (CVar)
	{
		return CVar->GetInt() != 0;
	}
	return false;
}

void FLevelEditorActionCallbacks::OnToggleParticleSystemHelpers()
{
	GEditor->bDrawParticleHelpers = !GEditor->bDrawParticleHelpers;
}

bool FLevelEditorActionCallbacks::OnIsParticleSystemHelpersEnabled()
{
	return GEditor->bDrawParticleHelpers;
}

void FLevelEditorActionCallbacks::OnToggleLODViewLocking()
{
	uint32 bUseLODViewLocking = !GetDefault<ULevelEditorViewportSettings>()->bUseLODViewLocking;
	GetMutableDefault<ULevelEditorViewportSettings>()->bUseLODViewLocking = bUseLODViewLocking;

	GEditor->RedrawLevelEditingViewports();
}

bool FLevelEditorActionCallbacks::OnIsLODViewLockingEnabled()
{
	return GetDefault<ULevelEditorViewportSettings>()->bUseLODViewLocking;
}

void FLevelEditorActionCallbacks::OnToggleLevelStreamingVolumePrevis()
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();

	ViewportSettings->bLevelStreamingVolumePrevis = !ViewportSettings->bLevelStreamingVolumePrevis;
	ViewportSettings->PostEditChange();
}

bool FLevelEditorActionCallbacks::OnIsLevelStreamingVolumePrevisEnabled()
{
	return GetDefault<ULevelEditorViewportSettings>()->bLevelStreamingVolumePrevis;
}

FText FLevelEditorActionCallbacks::GetAudioVolumeToolTip()
{
	float VolumeDecibels = -60.0f;
	if (!GEditor->IsRealTimeAudioMuted())
	{
		const float Volume = GEditor->GetRealTimeAudioVolume();
		VolumeDecibels = 20.0f * FMath::LogX(10.0f, FMath::Max(Volume, UE_SMALL_NUMBER));
		VolumeDecibels = FMath::Max(VolumeDecibels, -60.0f);
	}
	return FText::Format(NSLOCTEXT("LevelEditorCommands", "LevelEditorVolumeToolTip", "Level Editor Volume is {0} dB."), FText::AsNumber(VolumeDecibels));
}

float FLevelEditorActionCallbacks::GetAudioVolume()
{
	return GEditor->GetRealTimeAudioVolume();
}

void FLevelEditorActionCallbacks::OnAudioVolumeChanged(float Volume)
{
	GEditor->SetRealTimeAudioVolume(Volume);
}

bool FLevelEditorActionCallbacks::GetAudioMuted()
{
	return GEditor->IsRealTimeAudioMuted();
}

void FLevelEditorActionCallbacks::OnAudioMutedChanged(bool bMuted)
{
	GEditor->MuteRealTimeAudio(bMuted);
}

void FLevelEditorActionCallbacks::SnapObjectToView_Clicked()
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SnapObjectToView", "Snap Object to View"));

	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;

	// Get the new location and rotation for the actor from the viewport client's view.
	const FVector NewLocation = GCurrentLevelEditingViewportClient->GetViewLocation();
	const FQuat NewRotation = GCurrentLevelEditingViewportClient->GetViewRotation().Quaternion();

	UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet();
	SelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>([&NewLocation, &NewRotation, &LevelDirtyCallback](const TTypedElement<ITypedElementWorldInterface>& InElement)
		{
			// Get the actor's current transform.
			FTransform CurrentTransform;
			if (InElement.GetWorldTransform(CurrentTransform))
			{
				// Set new location and rotation to the current transform.
				FTransform NewTransform = CurrentTransform;
				NewTransform.SetLocation(NewLocation);
				NewTransform.SetRotation(NewRotation);

				// Find a suitable transform, if the actor can't be at the exact desired transform.
				FTransform SuitableTransform;
				if (!InElement.FindSuitableTransformAtPoint(NewTransform, SuitableTransform))
				{
					SuitableTransform = NewTransform;
				}

				InElement.NotifyMovementStarted();
				InElement.SetWorldTransform(SuitableTransform);
				InElement.NotifyMovementEnded();

				LevelDirtyCallback.Request();
			}

			return true;
		});

	GEditor->SetPivot(NewLocation, false, true); // Update the pivot location of the editor to the new actor location.
	GEditor->RedrawLevelEditingViewports();
}

void FLevelEditorActionCallbacks::ViewActorReferences_Clicked()
{
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			FAssetIdentifier AssetIdentifier(Actor->GetPackage()->GetFName());
			FEditorDelegates::OnOpenReferenceViewer.Broadcast({ AssetIdentifier }, FReferenceViewerParams());
			break;
		}
	}
}

bool FLevelEditorActionCallbacks::ViewActorReferences_CanExecute()
{
	TSet<FName> PackageNames;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			PackageNames.Add(Actor->GetPackage()->GetFName());
			if (PackageNames.Num() > 1)
			{
				return false;
			}
		}
	}
	return PackageNames.Num() == 1;
}

void FLevelEditorActionCallbacks::CopyActorFilePathtoClipboard_Clicked()
{
	TStringBuilder<1024> Result;

	TArray<AActor*> SelectedActors;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		SelectedActors.Add(Cast<AActor>(*It));
	}

	for (AActor* Actor : SelectedActors)
	{
		if (Result.Len() != 0)
		{
			Result.Append(LINE_TERMINATOR);
		}

		const UPackage* SceneOutlinerPackage = Actor->GetSceneOutlinerTopParentPackage();
		FString LocalFullPath(SceneOutlinerPackage->GetLoadedPath().GetLocalFullPath());

		if (SelectedActors.Num() > 1)
		{
			const FString& ActorLabel = Actor->GetActorLabel(false);
			if (ActorLabel.Len())
			{
				Result.Append(ActorLabel);
			}
			Result.Append(TEXT("("));
			Result.Append(Actor->GetName());
			Result.Append(TEXT("): "));
		}

		LocalFullPath = FPaths::ConvertRelativePathToFull(LocalFullPath);
		FPaths::MakePlatformFilename(LocalFullPath);

		Result.Append(LocalFullPath);
	}

	if (Result.Len())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Result);
	}
}

void FLevelEditorActionCallbacks::SaveActor_Clicked()
{
	TSet<UPackage*> PackagesToSave;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		AActor* Actor = Cast<AActor>(*It);
		if (UPackage* ActorPackage = Actor->GetSceneOutlinerItemPackage())
		{
			PackagesToSave.Add(ActorPackage);
		}
	}

	if (PackagesToSave.Num())
	{
		FEditorFileUtils::FPromptForCheckoutAndSaveParams SaveParams;
		SaveParams.bCheckDirty = false;
		SaveParams.bPromptToSave = false;
		SaveParams.bIsExplicitSave = true;

		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave.Array(), SaveParams);
	}
}

bool FLevelEditorActionCallbacks::SaveActor_CanExecute()
{
	TSet<UPackage*> PackagesToSave;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		AActor* Actor = Cast<AActor>(*It);
		if (UPackage* ActorPackage = Actor->GetSceneOutlinerItemPackage())
		{
			return true;
		}
	}
	return false;
}

static TArray<FString> GetSelectedActorsPackageFullpath()
{
	TArray<FString> PackageFullpaths;

	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (const AActor* Actor = Cast<AActor>(*It))
		{
			if (const UPackage* Package = Actor->GetSceneOutlinerItemPackage())
			{
				const FString LocalFullPath(Package->GetLoadedPath().GetLocalFullPath());

				if (!LocalFullPath.IsEmpty())
				{
					PackageFullpaths.Add(FPaths::ConvertRelativePathToFull(LocalFullPath));
				}
			}
		}
	}

	return PackageFullpaths;
}

void FLevelEditorActionCallbacks::ShowActorHistory_Clicked()
{
	TArray<FString> PackageFullpaths = GetSelectedActorsPackageFullpath();

	// Sort then remove consecutive identical elements to avoid displaying multiple times the same history.
	PackageFullpaths.Sort();
	PackageFullpaths.SetNum(Algo::Unique(PackageFullpaths));

	FSourceControlWindows::DisplayRevisionHistory(PackageFullpaths);
}

bool FLevelEditorActionCallbacks::ShowActorHistory_CanExecute()
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	if (!SourceControlProvider.IsEnabled())
	{
		return false;
	}

	TArray<FString> PackageFullpaths = GetSelectedActorsPackageFullpath();

	return PackageFullpaths.Num() > 0;
}

void FLevelEditorActionCallbacks::OnEnableActorSnap()
{
	FSnappingUtils::EnableActorSnap( !FSnappingUtils::IsSnapToActorEnabled() );

	// If the setting is enabled and there's no distance, revert to default
	if ( FSnappingUtils::IsSnapToActorEnabled() && FSnappingUtils::GetActorSnapDistance() == 0.0f )
	{
		FSnappingUtils::SetActorSnapDistance( 1.0f );
	}
}

bool FLevelEditorActionCallbacks::OnIsActorSnapEnabled()
{
	return FSnappingUtils::IsSnapToActorEnabled();
}


void FLevelEditorActionCallbacks::OnEnableVertexSnap()
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->bSnapVertices = !ViewportSettings->bSnapVertices;
}

bool FLevelEditorActionCallbacks::OnIsVertexSnapEnabled()
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return ViewportSettings->bSnappingActive && ViewportSettings->bSnapVertices;
}

FText FLevelEditorActionCallbacks::GetActorSnapTooltip()
{
	// If the setting is enabled, return the distance, otherwise say disabled
	if ( FSnappingUtils::IsSnapToActorEnabled() )
	{
		static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(2)
			.SetMaximumFractionalDigits(2);
		return FText::AsNumber( FSnappingUtils::GetActorSnapDistance(), &FormatOptions );
	}
	return NSLOCTEXT("UnrealEd", "Disabled", "Disabled" );
}

float FLevelEditorActionCallbacks::GetActorSnapSetting()
{
	// If the setting is enabled, return the distance, otherwise say 0
	if (FSnappingUtils::IsSnapToActorEnabled() )
	{
		return FSnappingUtils::GetActorSnapDistance(true) ;
	}
	return 0.0f;
}

void FLevelEditorActionCallbacks::SetActorSnapSetting(float Distance)
{
	FSnappingUtils::SetActorSnapDistance( Distance );

	// If the distance is 0, disable the setting until it's > 0
	FSnappingUtils::EnableActorSnap( ( Distance > 0.0f ? true : false ) );
}

void FLevelEditorActionCallbacks::OnToggleShowViewportToolbar()
{
	FLevelEditorModule& Module = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<SLevelViewport> Viewport = Module.GetFirstActiveLevelViewport())
	{
		Viewport->ToggleViewportToolbarVisibility();
	}
}

bool FLevelEditorActionCallbacks::IsViewportToolbarVisible()
{
	FLevelEditorModule& Module = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<SLevelViewport> Viewport = Module.GetFirstActiveLevelViewport())
	{
		return Viewport->IsViewportToolbarVisible();
	}
	return false;
}

void FLevelEditorActionCallbacks::OnToggleShowViewportUI()
{
	GLevelEditorModeTools().SetHideViewportUI( !GLevelEditorModeTools().IsViewportUIHidden() );
}

bool FLevelEditorActionCallbacks::IsViewportUIVisible()
{
	return !GLevelEditorModeTools().IsViewportUIHidden();
}

bool FLevelEditorActionCallbacks::IsEditorModeActive( FEditorModeID EditorMode )
{
	return GLevelEditorModeTools().IsModeActive( EditorMode );
}

void FLevelEditorActionCallbacks::OnAddVolume( UClass* VolumeClass )
{
	GUnrealEd->Exec( GetWorld(), *FString::Printf( TEXT("BRUSH ADDVOLUME CLASS=%s"), *VolumeClass->GetName() ) );

	// A new volume actor was added, update the volumes visibility.
	// This volume should be hidden if the user doesn't have this type of volume visible.
	GUnrealEd->UpdateVolumeActorVisibility( VolumeClass );

	GEditor->RedrawAllViewports();
}

void FLevelEditorActionCallbacks::SelectActorsInLayers()
{
	// Iterate over selected actors and make a list of all layers the selected actors belong to.
	TArray< FName > SelectedLayers;
	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = Cast<AActor>( *It );

		// Add them to the list of selected layers.
		for( int32 LayerIndex = 0 ; LayerIndex < Actor->Layers.Num() ; ++LayerIndex )
		{
			SelectedLayers.AddUnique( Actor->Layers[ LayerIndex ] );
		}
	}

	ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	const bool bSelect = true;
	const bool bNotify = true;
	Layers->SelectActorsInLayers( SelectedLayers, bSelect, bNotify );
}

void FLevelEditorActionCallbacks::SetWidgetMode( UE::Widget::EWidgetMode WidgetMode )
{
	if( !GLevelEditorModeTools().IsTracking() )
	{
		GLevelEditorModeTools().SetWidgetMode( WidgetMode );
		GEditor->RedrawAllViewports();
	}
}

bool FLevelEditorActionCallbacks::IsWidgetModeActive( UE::Widget::EWidgetMode WidgetMode )
{
	return GLevelEditorModeTools().GetWidgetMode() == WidgetMode;
}

bool FLevelEditorActionCallbacks::CanSetWidgetMode( UE::Widget::EWidgetMode WidgetMode )
{
	return GLevelEditorModeTools().UsesTransformWidget(WidgetMode) == true;
}

bool FLevelEditorActionCallbacks::IsTranslateRotateModeVisible()
{
	return GetDefault<ULevelEditorViewportSettings>()->bAllowTranslateRotateZWidget;
}

void FLevelEditorActionCallbacks::SetCoordinateSystem( ECoordSystem CoordinateSystem )
{
	GLevelEditorModeTools().SetCoordSystem( CoordinateSystem );
}

bool FLevelEditorActionCallbacks::IsCoordinateSystemActive( ECoordSystem CoordinateSystem )
{
	return GLevelEditorModeTools().GetCoordSystem() == CoordinateSystem;
}

void FLevelEditorActionCallbacks::MoveElementsToGrid_Clicked( bool InAlign, bool InPerElement )
{
	// TODO: Ideally this would come from some level editor context
	if (const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MoveElementsToGrid", "Snap Origin to Grid"));
		MoveTo_Clicked(SelectionSet, InAlign, InPerElement);
	}
}

void FLevelEditorActionCallbacks::MoveElementsToElement_Clicked(bool InAlign)
{
	// TODO: Ideally this would come from some level editor context
	if (const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
	{
		if (const TTypedElement<ITypedElementWorldInterface> DestElement = UEditorElementSubsystem::GetLastSelectedEditorManipulableElement(UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*SelectionSet)))
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MoveElementsToElement", "Snap Origin to Element"));
			MoveTo_Clicked(SelectionSet, InAlign, /*bPerElement*/false, DestElement);
		}
	}
}

void FLevelEditorActionCallbacks::MoveTo_Clicked( const UTypedElementSelectionSet* InSelectionSet, const bool InAlign, bool InPerElement, const TTypedElement<ITypedElementWorldInterface>& InDestination )
{
	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied		LevelDirtyCallback;

	// Update the pivot location.
	FVector Delta = FVector::ZeroVector;
	FVector NewLocation = FVector::ZeroVector;
	FQuat NewRotation = FQuat::Identity;

	if (!InPerElement)
	{
		if (InDestination)
		{
			FTransform DestinationTransform;
			if (InDestination.GetWorldTransform(DestinationTransform))
			{
				NewLocation = DestinationTransform.GetLocation();
				NewRotation = DestinationTransform.GetRotation();
				GEditor->SetPivot(NewLocation, false, true);
			}
		}
		else
		{
			const FVector OldPivot = GEditor->GetPivotLocation();
			const FVector NewPivot = OldPivot.GridSnap(GEditor->GetGridSize());
			Delta = NewPivot - OldPivot;
			GEditor->SetPivot(NewPivot, false, true);
		}
	}

	InSelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>([InAlign, InPerElement, &InDestination, &Delta, &NewLocation, &NewRotation, &LevelDirtyCallback](const TTypedElement<ITypedElementWorldInterface>& InElement)
	{
		// Skip moving the destination element
		if (InElement == InDestination)
		{
			return true;
		}

		FTransform CurrentTransform;
		if (InElement.GetWorldTransform(CurrentTransform))
		{
			if (!InDestination)
			{
				if (InPerElement)
				{
					const FVector OldPivot = CurrentTransform.GetLocation();
					const FVector NewPivot = OldPivot.GridSnap(GEditor->GetGridSize());
					Delta = NewPivot - OldPivot;
					GEditor->SetPivot(NewPivot, false, true);
				}

				NewLocation = CurrentTransform.GetLocation() + Delta;
			}

			FTransform NewTransform = CurrentTransform;
			NewTransform.SetLocation(NewLocation);
			if (InAlign)
			{
				NewTransform.SetRotation(NewRotation);
			}

			FTransform SuitableTransform;
			if (!InElement.FindSuitableTransformAtPoint(NewTransform, SuitableTransform))
			{
				SuitableTransform = NewTransform;
			}

			InElement.NotifyMovementStarted();
			InElement.SetWorldTransform(SuitableTransform);
			InElement.NotifyMovementEnded();

			LevelDirtyCallback.Request();
		}

		return true;
	});

	GEditor->RedrawLevelEditingViewports();
	GEditor->RebuildAlteredBSP(); // Update the Bsp of any levels containing a modified brush
}

void FLevelEditorActionCallbacks::SnapTo2DLayer_Clicked()
{
	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;

	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	if (Settings2D->SnapLayers.IsValidIndex(ViewportSettings->ActiveSnapLayerIndex))
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SnapSelection2D", "Snap Selection to 2D Layer"));

		float SnapDepth = Settings2D->SnapLayers[ViewportSettings->ActiveSnapLayerIndex].Depth;
		USelection* SelectedActors = GEditor->GetSelectedActors();
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			AActor* Actor = CastChecked<AActor>(*Iter);

			// Only snap actors that are not attached to something else
			if (Actor->GetAttachParentActor() == nullptr)
			{
				FTransform Transform = Actor->GetTransform();
				FVector CurrentLocation = Transform.GetLocation();

				switch (Settings2D->SnapAxis)
				{
				case ELevelEditor2DAxis::X: CurrentLocation.X = SnapDepth; break;
				case ELevelEditor2DAxis::Y: CurrentLocation.Y = SnapDepth; break;
				case ELevelEditor2DAxis::Z: CurrentLocation.Z = SnapDepth; break;
				}

				Transform.SetLocation(CurrentLocation);
				Actor->Modify();
				Actor->SetActorTransform(Transform);

				Actor->InvalidateLightingCache();
				Actor->UpdateComponentTransforms();
				Actor->PostEditMove(true);

				Actor->MarkPackageDirty();
				LevelDirtyCallback.Request();
			}
		}

		GEditor->RedrawLevelEditingViewports(true);
		GEditor->RebuildAlteredBSP();
	}
}

bool FLevelEditorActionCallbacks::CanSnapTo2DLayer()
{
	if (!ElementSelected_CanExecuteMove())
	{
		return false;
	}

	const ULevelEditor2DSettings* Settings = GetDefault<ULevelEditor2DSettings>();
	return Settings->SnapLayers.IsValidIndex(GetDefault<ULevelEditorViewportSettings>()->ActiveSnapLayerIndex);
}

void FLevelEditorActionCallbacks::MoveSelectionToDifferent2DLayer_Clicked(bool bGoingUp, bool bForceToTopOrBottom)
{
	// Change the active layer first
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	ULevelEditorViewportSettings* SettingsVP = GetMutableDefault<ULevelEditorViewportSettings>();

	const int32 NumLayers = Settings2D->SnapLayers.Num();

	if (NumLayers > 0)
	{
		if (bGoingUp && (SettingsVP->ActiveSnapLayerIndex > 0))
		{
			SettingsVP->ActiveSnapLayerIndex = bForceToTopOrBottom ? 0 : (SettingsVP->ActiveSnapLayerIndex - 1);
			SettingsVP->PostEditChange();
		}
		else if (!bGoingUp && ((SettingsVP->ActiveSnapLayerIndex + 1) < NumLayers))
		{
			SettingsVP->ActiveSnapLayerIndex = bForceToTopOrBottom ? (NumLayers - 1) : (SettingsVP->ActiveSnapLayerIndex + 1);
			SettingsVP->PostEditChange();
		}
	}

	// Snap the selection to the new active layer
	SnapTo2DLayer_Clicked();
}

bool FLevelEditorActionCallbacks::CanMoveSelectionToDifferent2DLayer(bool bGoingUp)
{
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	const ULevelEditorViewportSettings* SettingsVP = GetMutableDefault<ULevelEditorViewportSettings>();

	const int32 SelectedIndex = SettingsVP->ActiveSnapLayerIndex;
	const int32 NumLayers = Settings2D->SnapLayers.Num();

	const bool bHasLayerInDirection = bGoingUp ? (SelectedIndex > 0) : (SelectedIndex + 1 < NumLayers);
	const bool bHasActorSelected = GEditor->GetSelectedActorCount() > 0;

	// Allow it even if there is no layer in the corresponding direction, to let it double as a snap operation shortcut even when at the end stops
	return bHasLayerInDirection || bHasActorSelected;
}

void FLevelEditorActionCallbacks::Select2DLayerDeltaAway_Clicked(int32 Delta)
{
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	ULevelEditorViewportSettings* SettingsVP = GetMutableDefault<ULevelEditorViewportSettings>();

	const int32 SelectedIndex = SettingsVP->ActiveSnapLayerIndex;
	const int32 NumLayers = Settings2D->SnapLayers.Num();

	if (NumLayers > 0)
	{
		const int32 NewIndex = ((NumLayers + SelectedIndex + Delta) % NumLayers);

		SettingsVP->ActiveSnapLayerIndex = NewIndex;
		SettingsVP->PostEditChange();
	}
}

void FLevelEditorActionCallbacks::SnapToFloor_Clicked( bool InAlign, bool InUseLineTrace, bool InUseBounds, bool InUsePivot )
{
	// TODO: Ideally this would come from some level editor context
	if (const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SnapActorsToFloor", "Snap Elements To Floor"));
		SnapTo_Clicked(SelectionSet, InAlign, InUseLineTrace, InUseBounds, InUsePivot);
	}
}

void FLevelEditorActionCallbacks::SnapElementsToElement_Clicked( bool InAlign, bool InUseLineTrace, bool InUseBounds, bool InUsePivot )
{
	// TODO: Ideally this would come from some level editor context
	if (const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
	{
		if (const TTypedElement<ITypedElementWorldInterface> DestElement = UEditorElementSubsystem::GetLastSelectedEditorManipulableElement(UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*SelectionSet)))
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SnapElementsToElement", "Snap Elements to Element"));
			SnapTo_Clicked(SelectionSet, InAlign, InUseLineTrace, InUseBounds, InUsePivot, DestElement);
		}
	}
}

void FLevelEditorActionCallbacks::SnapTo_Clicked( const UTypedElementSelectionSet* InSelectionSet, const bool InAlign, const bool InUseLineTrace, const bool InUseBounds, const bool InUsePivot, const TTypedElement<ITypedElementWorldInterface>& InDestination )
{
	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied		LevelDirtyCallback;

	// Let the component visualizers try to handle the selection.
	// TODO: Should this also take an element?
	{
		AActor* DestinationActor = nullptr;
		if (TTypedElement<ITypedElementObjectInterface> DestinationObjectHandle = InSelectionSet->GetElementList()->GetElement<ITypedElementObjectInterface>(InDestination))
		{
			DestinationActor = Cast<AActor>(DestinationObjectHandle.GetObject());
		}

		if (GUnrealEd->ComponentVisManager.HandleSnapTo(InAlign, InUseLineTrace, InUseBounds, InUsePivot, DestinationActor))
		{
			return;
		}
	}

	// Ignore the selected elements when sweeping for the snap location
	const TArray<FTypedElementHandle> ElementsToIgnore = InSelectionSet->GetSelectedElementHandles();

	// Snap each selected element
	bool bSnappedElements = false;
	InSelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>([InAlign, InUseLineTrace, InUseBounds, InUsePivot, &InDestination, &ElementsToIgnore, &LevelDirtyCallback, &bSnappedElements](const TTypedElement<ITypedElementWorldInterface>& InElement)
	{
		if (GEditor->SnapElementTo(InElement, InAlign, InUseLineTrace, InUseBounds, InUsePivot, InDestination, ElementsToIgnore))
		{
			bSnappedElements = true;
			LevelDirtyCallback.Request();
		}
		return true;
	});

	// Update the pivot location
	if (bSnappedElements)
	{
		if (TTypedElement<ITypedElementWorldInterface> LastElement = UEditorElementSubsystem::GetLastSelectedEditorManipulableElement(UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*InSelectionSet)))
		{
			FTransform LastElementTransform;
			if (LastElement.GetWorldTransform(LastElementTransform))
			{
				GEditor->SetPivot(LastElementTransform.GetLocation(), false, true);

				if (UActorGroupingUtils::IsGroupingActive())
				{
					if (TTypedElement<ITypedElementObjectInterface> LastObjectElement = InSelectionSet->GetElementList()->GetElement<ITypedElementObjectInterface>(LastElement))
					{
						if (AActor* LastActor = Cast<AActor>(LastObjectElement.GetObject()))
						{
							// Set group pivot for the root-most group
							if (AGroupActor* ActorGroupRoot = AGroupActor::GetRootForActor(LastActor, true, true))
							{
								ActorGroupRoot->CenterGroupLocation();
							}
						}
					}
				}
			}
		}
	}

	GEditor->RedrawLevelEditingViewports();
}

void FLevelEditorActionCallbacks::AlignBrushVerticesToGrid_Execute()
{
	UWorld* World = GUnrealEd->GetWorld();
	GEditor->Exec(World, TEXT("ACTOR ALIGN VERTS"));
}

bool FLevelEditorActionCallbacks::ActorSelected_CanExecute()
{
	return GEditor->GetSelectedActorCount() > 0;
}

bool FLevelEditorActionCallbacks::ActorsSelected_CanExecute()
{
	return GEditor->GetSelectedActorCount() > 1;
}

bool FLevelEditorActionCallbacks::ActorTypesSelected_CanExecute(EActorTypeFlags TypeFlags, bool bSingleOnly)
{
	FSelectedActorInfo SelectionInfo = AssetSelectionUtils::GetSelectedActorInfo();
	if (SelectionInfo.NumSelected > 0 && (!bSingleOnly || SelectionInfo.NumSelected == 1))
	{
		if ((TypeFlags & IncludePawns) && SelectionInfo.bHavePawn)
		{
			return true;
		}

		if ((TypeFlags & IncludeStaticMeshes) && SelectionInfo.bHaveStaticMesh)
		{
			return true;
		}

		if ((TypeFlags & IncludeSkeletalMeshes) && SelectionInfo.bHaveSkeletalMesh)
		{
			return true;
		}

		if ((TypeFlags & IncludeEmitters) && SelectionInfo.bHaveEmitter)
		{
			return true;
		}
	}

	return false;
}

bool FLevelEditorActionCallbacks::ElementSelected_CanExecute()
{
	// TODO: Ideally this would come from some level editor context
	const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet();
	return SelectionSet && SelectionSet->GetNumSelectedElements() > 0;
}

bool FLevelEditorActionCallbacks::ElementsSelected_CanExecute()
{
	// TODO: Ideally this would come from some level editor context
	const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet();
	return SelectionSet && SelectionSet->GetNumSelectedElements() > 1;
}

bool FLevelEditorActionCallbacks::ElementSelected_CanExecuteMove()
{
	// TODO: Ideally this would come from some level editor context
	if (const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
	{
		FTypedElementListRef NormalizedElements = UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*SelectionSet);
		return UEditorElementSubsystem::GetEditorManipulableElements(NormalizedElements)->Num() > 0;
	}

	return false;
}

bool FLevelEditorActionCallbacks::ElementsSelected_CanExecuteMove()
{
	// TODO: Ideally this would come from some level editor context
	if (const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
	{
		FTypedElementListRef NormalizedElements = UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*SelectionSet);
		return UEditorElementSubsystem::GetEditorManipulableElements(NormalizedElements)->Num() > 1;
	}

	return false;
}

bool FLevelEditorActionCallbacks::ElementSelected_CanExecuteScale()
{
	// TODO: Ideally this would come from some level editor context
	if (const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
	{
		FTypedElementListRef NormalizedElements = UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*SelectionSet);
		return UEditorElementSubsystem::GetEditorManipulableElements(NormalizedElements, UE::Widget::WM_Scale)->Num() > 0;
	}

	return false;
}

bool FLevelEditorActionCallbacks::ElementsSelected_CanExecuteScale()
{
	// TODO: Ideally this would come from some level editor context
	if (const UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
	{
		FTypedElementListRef NormalizedElements = UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*SelectionSet);
		return UEditorElementSubsystem::GetEditorManipulableElements(NormalizedElements, UE::Widget::WM_Scale)->Num() > 1;
	}

	return false;
}

void FLevelEditorActionCallbacks::GeometryCollection_SelectAllGeometry()
{
	GEditor->Exec(GetWorld(), TEXT("GeometryCollection.SelectAllGeometry"));
}

void FLevelEditorActionCallbacks::GeometryCollection_SelectNone()
{
	GEditor->Exec(GetWorld(), TEXT("GeometryCollection.SelectNone"));
}

void FLevelEditorActionCallbacks::GeometryCollection_SelectInverseGeometry()
{
	GEditor->Exec(GetWorld(), TEXT("GeometryCollection.SelectInverseGeometry"));
}

bool FLevelEditorActionCallbacks::GeometryCollection_IsChecked()
{
	return true;
}

void FLevelEditorActionCallbacks::ToggleAllowArcballRotation()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bAllowArcballRotate = !ViewportSettings->bAllowArcballRotate;
		ViewportSettings->OnSettingChanged().Broadcast(GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bAllowArcballRotate));
	}
}

bool FLevelEditorActionCallbacks::IsAllowArcballRotationChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bAllowArcballRotate;
	}

	return false;
}

void FLevelEditorActionCallbacks::ToggleAllowScreenspaceRotation()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bAllowScreenRotate = !ViewportSettings->bAllowScreenRotate;
		ViewportSettings->OnSettingChanged().Broadcast(GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bAllowScreenRotate));
	}
}

bool FLevelEditorActionCallbacks::IsAllowScreenspaceRotationChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bAllowScreenRotate;
	}

	return false;
}

void FLevelEditorActionCallbacks::ToggleEnableViewportHoverFeedback()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bEnableViewportHoverFeedback = !ViewportSettings->bEnableViewportHoverFeedback;
	}
}

bool FLevelEditorActionCallbacks::IsEnableViewportHoverFeedbackChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bEnableViewportHoverFeedback;
	}

	return false;
}

void FLevelEditorActionCallbacks::TogglePreviewSelectedCameras(const TWeakPtr<::SLevelViewport>& InLevelViewportWeak)
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bPreviewSelectedCameras = !ViewportSettings->bPreviewSelectedCameras;

		if (TSharedPtr<::SLevelViewport> LevelViewport = InLevelViewportWeak.Pin())
		{
			LevelViewport->OnPreviewSelectedCamerasChange();
		}
	}
}

bool FLevelEditorActionCallbacks::IsPreviewSelectedCamerasChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bPreviewSelectedCameras;
	}

	return false;
}

void FLevelEditorActionCallbacks::ToggleOrbitCameraAroundSelection()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bOrbitCameraAroundSelection = !ViewportSettings->bOrbitCameraAroundSelection;
	}
}

bool FLevelEditorActionCallbacks::IsOrbitCameraAroundSelectionChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bOrbitCameraAroundSelection;
	}

	return false;
}

void FLevelEditorActionCallbacks::ToggleLinkOrthographicViewports()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bUseLinkedOrthographicViewports = !ViewportSettings->bUseLinkedOrthographicViewports;
	}
}

bool FLevelEditorActionCallbacks::IsLinkOrthographicViewportsChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bUseLinkedOrthographicViewports;
	}

	return false;
}

void FLevelEditorActionCallbacks::ToggleOrthoZoomToCursor()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bCenterZoomAroundCursor = !ViewportSettings->bCenterZoomAroundCursor;
	}
}

bool FLevelEditorActionCallbacks::IsOrthoZoomToCursorChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bCenterZoomAroundCursor;
	}

	return false;
}

void FLevelEditorActionCallbacks::ToggleInvertMiddleMousePan()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bInvertMiddleMousePan = !ViewportSettings->bInvertMiddleMousePan;
	}
}

bool FLevelEditorActionCallbacks::IsInvertMiddleMousePanChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bInvertMiddleMousePan;
	}
	return false;
}

void FLevelEditorActionCallbacks::ToggleInvertOrbitYAxis()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bInvertOrbitYAxis = !ViewportSettings->bInvertOrbitYAxis;
	}
}

bool FLevelEditorActionCallbacks::IsInvertOrbitYAxisChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bInvertOrbitYAxis;
	}
	return false;
}

void FLevelEditorActionCallbacks::ToggleInvertRightMouseDollyYAxis()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bInvertRightMouseDollyYAxis = !ViewportSettings->bInvertRightMouseDollyYAxis;
	}
}

bool FLevelEditorActionCallbacks::IsInvertRightMouseDollyYAxisChecked()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bInvertRightMouseDollyYAxis;
	}
	return false;
}

class UWorld* FLevelEditorActionCallbacks::GetWorld()
{
	return GEditor->GetEditorWorldContext().World();
}

namespace
{
	const FName OpenRecentFileBundle = "OpenRecentFile";
	const FName OpenFavoriteFileBundle = "OpenFavoriteFile";
	const FName ExternalBuildTypesBundle = "ExternalBuilds";
}

FLevelEditorCommands::FLevelEditorCommands()
	: TCommands<FLevelEditorCommands>
	(
		"LevelEditor", // Context name for fast lookup
		NSLOCTEXT("Contexts", "LevelEditor", "Level Editor"), // Localized context name for displaying
		"LevelViewport", // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
	AddBundle(OpenRecentFileBundle, NSLOCTEXT("LevelEditorCommands", "OpenRecentFileBundle", "Open Recent File"));
	AddBundle(OpenFavoriteFileBundle, NSLOCTEXT("LevelEditorCommands", "OpenFavoriteFileBundle", "Open Favorite File"));
	AddBundle(ExternalBuildTypesBundle, NSLOCTEXT("LevelEditorCommands", "ExternalBuildTypesBundle", "Build External Type"));
}

/** UI_COMMAND takes long for the compile to optimize */
UE_DISABLE_OPTIMIZATION_SHIP
void FLevelEditorCommands::RegisterCommands()
{
	UI_COMMAND( BrowseDocumentation, "Level Editor Documentation", "Details on how to use the Level Editor", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift, EKeys::F1 ) );
	UI_COMMAND( BrowseViewportControls, "Viewport Controls", "Ways to move around in the 3D viewport", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( NewLevel, "New Level...", "Create a new level, or choose a level template to start from.", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Control, EKeys::N ) );
	UI_COMMAND( Save, "Save Current Level", "Saves the current level to disk", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::S) );
	UI_COMMAND( SaveAs, "Save Current Level As...", "Save the current level as...", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Control|EModifierKey::Alt, EKeys::S ) );
	UI_COMMAND( SaveAllLevels, "Save All Levels", "Saves all unsaved levels to disk", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BrowseLevel, "Browse To Level", "Browses to the associated level and selects it in the most recently used Content Browser (summoning one if necessary)", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ToggleFavorite, "Toggle Favorite", "Sets whether the currently loaded level will appear in the list of favorite levels", EUserInterfaceActionType::Button, FInputChord() );

	for( int32 CurRecentIndex = 0; CurRecentIndex < FLevelEditorCommands::MaxRecentFiles; ++CurRecentIndex )
	{
		// NOTE: The actual label and tool-tip will be overridden at runtime when the command is bound to a menu item, however
		// we still need to set one here so that the key bindings UI can function properly
		TSharedRef< FUICommandInfo > OpenRecentFile =
			FUICommandInfoDecl(
				this->AsShared(),
				FName( *FString::Printf( TEXT( "OpenRecentFile%i" ), CurRecentIndex ) ),
				FText::Format( NSLOCTEXT( "LevelEditorCommands", "OpenRecentFile", "Open Recent File {0}" ), FText::AsNumber( CurRecentIndex ) ),
				NSLOCTEXT( "LevelEditorCommands", "OpenRecentFileToolTip", "Opens a recently opened file" ),
				OpenRecentFileBundle)
			.UserInterfaceType( EUserInterfaceActionType::Button )
			.DefaultChord( FInputChord() );
		OpenRecentFileCommands.Add( OpenRecentFile );
	}
	for (int32 CurFavoriteIndex = 0; CurFavoriteIndex < FLevelEditorCommands::MaxFavoriteFiles; ++CurFavoriteIndex)
	{
		// NOTE: The actual label and tool-tip will be overridden at runtime when the command is bound to a menu item, however
		// we still need to set one here so that the key bindings UI can function properly
		TSharedRef< FUICommandInfo > OpenFavoriteFile =
			FUICommandInfoDecl(
				this->AsShared(),
				FName(*FString::Printf(TEXT("OpenFavoriteFile%i"), CurFavoriteIndex)),
				FText::Format(NSLOCTEXT("LevelEditorCommands", "OpenFavoriteFile", "Open Favorite File {0}"), FText::AsNumber(CurFavoriteIndex)),
				NSLOCTEXT("LevelEditorCommands", "OpenFavoriteFileToolTip", "Opens a favorite file"),
				OpenFavoriteFileBundle)
			.UserInterfaceType(EUserInterfaceActionType::Button)
			.DefaultChord(FInputChord());
		OpenFavoriteFileCommands.Add(OpenFavoriteFile);
	}

	UI_COMMAND( ClearRecentFiles, "Clear Recent Levels", "Clear the list of recently opened levels", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ImportScene, "Import Into Level...", "Import a 3D scene from a file and add it to the current level", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( ExportAll, "Export All...", "Exports the entire level to a file on disk (multiple formats are supported.)", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ExportSelected, "Export Selected...", "Exports currently-selected objects to a file on disk (multiple formats are supported.)", EUserInterfaceActionType::Button, FInputChord() );

	// External Build commands (inspired from RecentFiles/FavoriteFiles)
	for (int32 Index = 0; Index < FLevelEditorCommands::MaxExternalBuildTypes; ++Index)
	{
		// NOTE: The actual label and tool-tip will be overridden at runtime when the command is bound to a menu item, however
		// we still need to set one here so that the key bindings UI can function properly
		ExternalBuildTypeCommands.Add(
			FUICommandInfoDecl(
				this->AsShared(),
				FName(*FString::Printf(TEXT("ExternalBuildType %i"), Index)),
				FText::Format(NSLOCTEXT("LevelEditorCommands", "ExternalBuildType", "Build Type {0}"), FText::AsNumber(Index)),
				/*Description*/NSLOCTEXT("LevelEditorCommands", "ExternalBuildToolTip", "Builds an external type"),
				ExternalBuildTypesBundle)
			.UserInterfaceType(EUserInterfaceActionType::Button)
			.DefaultChord(FInputChord()));
	}

	UI_COMMAND( Build, "Build All Levels", "Builds all levels (precomputes lighting data and visibility data, generates navigation networks and updates brush models.)\nThis action is not available while Play in Editor is active, static lighting is disabled in the project settings, or when previewing less than Shader Model 5", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildAndSubmitToSourceControl, "Build and Submit...", "Displays a window that allows you to build all levels and submit them to revision control", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildLightingOnly, "Build Lighting", "Only precomputes lighting (all levels.)\nThis action is not available while Play in Editor is active, static lighting is disabled in the project settings, or when previewing less than Shader Model 5", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control|EModifierKey::Shift, EKeys::Semicolon) );
	UI_COMMAND( BuildReflectionCapturesOnly, "Build Reflection Captures", "Updates Reflection Captures and stores their data in the BuildData package.\nThis action is not available while Play in Editor is active, static lighting is disabled in the project settings", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildLightingOnly_VisibilityOnly, "Precompute Static Visibility", "Only precomputes static visibility data (all levels.)", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( LightingBuildOptions_UseErrorColoring, "Use Error Coloring", "When enabled, errors during lighting precomputation will be baked as colors into light map data", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( LightingBuildOptions_ShowLightingStats, "Show Lighting Stats", "When enabled, a window containing metrics about lighting performance and memory will be displayed after a successful build.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( BuildGeometryOnly, "Build Geometry", "Only builds geometry (all levels.)", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildGeometryOnly_OnlyCurrentLevel, "Build Geometry (Current Level)", "Builds geometry, only for the current level", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildPathsOnly, "Build Paths", "Only builds paths (all levels.)", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildHLODs, "Build HLODs", "Builds all HLODs for the current world", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildHLODs_SelectedActors, "Build HLODs (Selected Actors)", "Builds HLODs that are affected by selected actors", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildMinimap, "Build World Partition Editor Minimap", "Builds the minimap for the current world", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildLandscapeSplineMeshes, "Build Landscape Spline Meshes", "Builds landscape spline meshes for the current world", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( BuildTextureStreamingOnly, "Build Texture Streaming", "Build texture streaming data", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BuildVirtualTextureOnly, "Build Streaming Runtime Virtual Textures", "Build runtime virtual texture low mips streaming data", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( BuildAllLandscape, "Build Landscape", "Build all data related to landscape (grass maps, physical material, Nanite, dirty height and weight maps)", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND( LightingQuality_Production, "Production", "Sets precomputed lighting quality to highest possible quality (slowest computation time.)", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( LightingQuality_High, "High", "Sets precomputed lighting quality to high quality", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( LightingQuality_Medium, "Medium", "Sets precomputed lighting quality to medium quality", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( LightingQuality_Preview, "Preview", "Sets precomputed lighting quality to preview quality (fastest computation time.)", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( LightingDensity_RenderGrayscale, "Render Grayscale", "Renders the lightmap density.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( LightingResolution_CurrentLevel, "Current Level", "Adjust only primitives in the current level.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( LightingResolution_SelectedLevels, "Selected Levels", "Adjust only primitives in the selected levels.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( LightingResolution_AllLoadedLevels, "All Loaded Levels", "Adjust primitives in all loaded levels.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( LightingResolution_SelectedObjectsOnly, "Selected Objects Only", "Adjust only selected objects in the levels.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( LightingStaticMeshInfo, "Lighting StaticMesh Info...", "Shows the lighting information for the StaticMeshes.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SceneStats, "Open Scene Stats", "Opens the Scene Stats viewer", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( TextureStats, "Open Texture Stats", "Opens the Texture Stats viewer", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MapCheck, "Open Map Check", "Checks map for errors", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( RecompileGameCode, "Recompile Game Code", "Recompiles and reloads C++ code for game systems on the fly", EUserInterfaceActionType::Button, FInputChord( EKeys::P, EModifierKey::Alt | EModifierKey::Control | EModifierKey::Shift ) );

#if WITH_LIVE_CODING
	UI_COMMAND( LiveCoding_Enable, "Enable Live Coding", "Hot-patches C++ function changes into the current process.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( LiveCoding_StartSession, "Start Session", "Starts a live coding session.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( LiveCoding_ShowConsole, "Show Console", "Displays the live coding console window.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( LiveCoding_Settings, "Settings...", "Open the live coding settings", EUserInterfaceActionType::Button, FInputChord() );
#endif

	UI_COMMAND( EditAsset, "Edit Asset", "Edits the asset associated with the selected actor", EUserInterfaceActionType::Button, FInputChord( EKeys::E, EModifierKey::Control ) );
	UI_COMMAND( EditAssetNoConfirmMultiple, "Edit Multiple Assets", "Edits multiple assets associated with the selected actor without a confirmation prompt", EUserInterfaceActionType::Button, FInputChord( EKeys::E, EModifierKey::Control | EModifierKey::Shift ) );
	UI_COMMAND( OpenSelectionInPropertyMatrix, "Edit Selection in Property Matrix", "Bulk edit the selected assets in the Property Matrix", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( GoHere, "Go Here", "Moves the camera to the current mouse position", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( PlayFromHere, "Play From Here", "Starts a game preview from the selected actor or clicked location as fallback", EUserInterfaceActionType::Button, FInputChord(EKeys::P, EModifierKey::Alt | EModifierKey::Shift) );

	UI_COMMAND( SnapCameraToObject, "Move Camera to Object", "Move the current camera to match the location and rotation of the selected object.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SnapObjectToCamera, "Move Object to Camera", "Move the selected object to match the location and rotation of the current camera.", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND(CopyActorFilePathtoClipboard, "Copy Selected Actor(s) File Path", "Copy the file path of the selected actors", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenActorInReferenceViewer, "Open Actor in Reference Viewer...", "Launches the reference viewer showing the selected actor references", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift | EModifierKey::Alt, EKeys::R));
	UI_COMMAND(SaveActor, "Save Selected Actor(s)", "Save the selected actors", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowActorHistory, "Show Actor History", "Shows the history of the file containing the actor.", EUserInterfaceActionType::Button, FInputChord());	

	UI_COMMAND( GoToCodeForActor, "Go to C++ Code for Actor", "Opens a code editing IDE and navigates to the source file associated with the seleced actor", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( GoToDocsForActor, "Go to Documentation for Actor", "Opens documentation for the Actor in the default web browser", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( PasteHere, "Paste Here", "Pastes the actor at the click location", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( SnapOriginToGrid, "Snap Origin to Grid", "Snaps the actor to the nearest grid location at its origin", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Control, EKeys::End ) );
	UI_COMMAND( SnapOriginToGridPerActor, "Snap Origin to Grid Per Actor", "Snaps each selected actor separately to the nearest grid location at its origin", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AlignOriginToGrid, "Align Origin to Grid", "Aligns the actor to the nearest grid location at its origin", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( SnapTo2DLayer, "Snap to 2D Layer", "Snaps the actor to the current 2D snap layer", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MoveSelectionUpIn2DLayers, "Bring selection forward a snap layer", "Bring selection forward a snap layer", EUserInterfaceActionType::Button, FInputChord(EKeys::PageUp, EModifierKey::Control) );
	UI_COMMAND( MoveSelectionDownIn2DLayers, "Send selection backward a snap layer", "Send selection backward a snap layer", EUserInterfaceActionType::Button, FInputChord(EKeys::PageDown, EModifierKey::Control) );
	UI_COMMAND( MoveSelectionToTop2DLayer, "Bring selection to the front snap layer", "Bring selection to the front snap layer", EUserInterfaceActionType::Button, FInputChord(EKeys::PageUp, EModifierKey::Shift | EModifierKey::Control) );
	UI_COMMAND( MoveSelectionToBottom2DLayer, "Send selection to the back snap layer", "Send selection to the back snap layer", EUserInterfaceActionType::Button, FInputChord(EKeys::PageDown, EModifierKey::Shift | EModifierKey::Control) );
	UI_COMMAND( Select2DLayerAbove, "Select next 2D layer", "Changes the active layer to the next 2D layer", EUserInterfaceActionType::Button, FInputChord(EKeys::PageUp, EModifierKey::Alt) );
	UI_COMMAND( Select2DLayerBelow, "Select previous 2D layer", "Changes the active layer to the previous 2D layer", EUserInterfaceActionType::Button, FInputChord(EKeys::PageDown, EModifierKey::Alt) );


	UI_COMMAND( SnapToFloor, "Snap to Floor", "Snaps the actor or component to the floor below it", EUserInterfaceActionType::Button, FInputChord( EKeys::End ) );
	UI_COMMAND( AlignToFloor, "Align to Floor", "Aligns the actor or component with the floor", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SnapPivotToFloor, "Snap Pivot to Floor", "Snaps the actor to the floor at its pivot point", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Alt, EKeys::End ) );	
	UI_COMMAND( AlignPivotToFloor, "Align Pivot to Floor", "Aligns the actor with the floor at its pivot point", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SnapBottomCenterBoundsToFloor, "Snap Bottom Center Bounds to Floor", "Snaps the actor to the floor at its bottom center bounds", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift, EKeys::End ) );
	UI_COMMAND( AlignBottomCenterBoundsToFloor, "Align Bottom Center Bounds to Floor", "Aligns the actor with the floor at its bottom center bounds", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SnapOriginToActor, "Snap Origin to Actor", "SNaps the actor to another actor at its origin", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AlignOriginToActor, "Align Origin to Actor", "Aligns the actor to another actor at its origin", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SnapToActor, "Snap to Actor", "Snaps the actor to another actor", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AlignToActor, "Align to Actor", "Aligns the actor with another actor", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SnapPivotToActor, "Snap Pivot to Actor", "Snaps the actor to another actor at its pivot point", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AlignPivotToActor, "Align Pivot to Actor", "Aligns the actor with another actor at its pivot point", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SnapBottomCenterBoundsToActor, "Snap Bottom Center Bounds to Actor", "Snaps the actor to another actor at its bottom center bounds", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AlignBottomCenterBoundsToActor, "Align Bottom Center Bounds to Actor", "Aligns the actor with another actor at its bottom center bounds", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( DeltaTransformToActors, "Delta Transform", "Apply Delta Transform to selected elements", EUserInterfaceActionType::Button, FInputChord() );

	auto MakeAxisUICommand = [this](TSharedPtr<FUICommandInfo>& Command, const TCHAR* CommandName, EAxisList::Type Axis)
	{
		const FText AxisDisplayName = AxisDisplayInfo::GetAxisDisplayName(Axis);
		FUICommandInfo::MakeCommandInfo(
			AsShared(),
			Command,
			CommandName,
			FText::Format(LOCTEXT("MirrorAxisLabel", "Mirror {0} Axis"), AxisDisplayName),
			FText::Format(LOCTEXT("MirrorAxisTooltip", "Mirrors the element along the {0} axis"), AxisDisplayName),
			FSlateIcon(),
			EUserInterfaceActionType::Button,
			FInputChord()
		);
	};

	#define UI_COMMAND_AXIS(Command, Axis) MakeAxisUICommand(Command, TEXT(#Command), Axis);

	UI_COMMAND_AXIS(MirrorActorY, EAxisList::Left)
	UI_COMMAND_AXIS(MirrorActorZ, EAxisList::Up)
	UI_COMMAND_AXIS(MirrorActorX, EAxisList::Forward)

	#undef UI_COMMAND_AXIS

	UI_COMMAND( LockActorMovement, "Lock Item Movement", "Locks the item so it cannot be moved", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( DetachFromParent, "Detach", "Detach the actor from its parent", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AttachSelectedActors, "Attach Selected Actors", "Attach the selected actors to the last selected actor", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::B) );
	UI_COMMAND( AttachActorIteractive, "Attach Actor Interactive", "Start an interactive actor picker to let you choose a parent for the currently selected actor", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::A) );
	UI_COMMAND( CreateNewOutlinerFolder, "Create Folder", "Place the selected actors in a new folder", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( HoldToEnableVertexSnapping, "Hold to Enable Vertex Snapping", "When the key binding is pressed and held vertex snapping will be enabled", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::V) );
	UI_COMMAND( HoldToEnablePivotVertexSnapping, "Hold to Enable Pivot Vertex Snapping", "Hold to enable vertex snapping while dragging a pivot. Alt must be a modifier in this command or it will not work.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::V) );

	//@ todo Slate better tooltips for pivot options
	UI_COMMAND( SavePivotToPrePivot, "Set as Pivot Offset", "Sets the current pivot location as the pivot offset for this element", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ResetPrePivot, "Reset Pivot Offset", "Resets the pivot offset for this element", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ResetPivot, "Reset Pivot", "Resets the pivot", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MovePivotHere, "Set Pivot Offset Here", "Sets the pivot to the clicked location", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MovePivotHereSnapped, "Set Pivot Offset Here (Snapped)", "Sets the pivot to the nearest grid point to the clicked location", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MovePivotToCenter, "Center on Selection", "Centers the pivot to the middle of the selection", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( ConvertToAdditive, "Additive", "Converts the selected brushes to additive brushes", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ConvertToSubtractive, "Subtractive", "Converts the selected brushes to subtractive brushes", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( OrderFirst, "To First", "Changes the drawing order of the selected brushes so they are the first to draw", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OrderLast, "To Last", "Changes the drawing order of the selected brushes so they are the last to draw", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( MakeSolid, "Solid", "Makes the selected brushes solid", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MakeSemiSolid, "Semi-Solid", "Makes the selected brushes semi-solid", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MakeNonSolid, "Non-Solid", "Makes the selected brushes non-solid", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( MergePolys, "Merge", "Merges multiple polygons on a brush face into as few as possible", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SeparatePolys, "Separate", "Reverses the effect of a previous merge", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AlignBrushVerticesToGrid, "Align Brush Vertices To Grid", "Align brush vertices to the grid", EUserInterfaceActionType::Button, FInputChord() );

	// RegroupActors uses GroupActors for it's label and tooltip when simply grouping a selection of actors using overrides. This is to provide display of the chord which is the same for both.
	UI_COMMAND( GroupActors, "Group", "Groups the selected actors", EUserInterfaceActionType::Button, FInputChord( /*EKeys::G, EModifierKey::Control*/ ) );
	UI_COMMAND( RegroupActors, "Regroup", "Regroups the selected actors into a new group, removing any current groups in the selection", EUserInterfaceActionType::Button, FInputChord( EKeys::G, EModifierKey::Control ) );
	UI_COMMAND( UngroupActors, "Ungroup", "Ungroups the selected actors", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift, EKeys::G ) );
	UI_COMMAND( AddActorsToGroup, "Add to Group", "Adds the selected actors to the selected group", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( RemoveActorsFromGroup, "Remove from Group", "Removes the selected actors from the selected groups", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( LockGroup, "Lock", "Locks the selected groups", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( UnlockGroup, "Unlock", "Unlocks the selected groups", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( FixupGroupActor, "Fixup Group Actor", "Removes null actors and deletes the GroupActor if it is empty.", EUserInterfaceActionType::Button, FInputChord());

#if PLATFORM_MAC
	UI_COMMAND( ShowAll, "Show All", "Shows all items", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Command, EKeys::H ) );
#else
	UI_COMMAND( ShowAll, "Show All", "Shows all items", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Control, EKeys::H ) );
#endif
	UI_COMMAND( ShowSelectedOnly, "Show Only Selected", "Shows only the selected items", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ShowSelected, "Show Selected", "Shows the selected items", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( HideSelected, "Hide Selected", "Hides the selected items", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ToggleSelectedVisibility, "Toggle Selected Visibility", "Hides or shows the selected items", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ShowSelectedHierarchy, "Show Selected Hierarchy", "Shows the selected items and their children", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift, EKeys::H ) );
	UI_COMMAND( HideSelectedHierarchy, "Hide Selected Hierarchy", "Hides the selected items and their children", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ToggleSelectedHierarchyVisibility, "Toggle Selected Hierarchy Visibility", "Hides or shows the selected items and their children", EUserInterfaceActionType::Button, FInputChord( EKeys::H ) );
	UI_COMMAND( ShowAllStartup, "Show All At Startup", "Shows all actors at startup", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ShowSelectedStartup, "Show Selected At Startup", "Shows selected actors at startup", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( HideSelectedStartup, "Hide Selected At Startup", "Hide selected actors at startup", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ShowSelectedHierarchyStartup, "Show Selected Hierarchy At Startup", "Shows selected actors and their current children at startup", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( HideSelectedHierarchyStartup, "Hide Selected Hierarchy At Startup", "Hide selected actors and their current children at startup", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CycleNavigationDataDrawn, "Cycle Navigation Data Drawn", "Cycles through navigation data (navmeshes for example) to draw one at a time", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Alt, EKeys::N ) );

	UI_COMMAND( SelectNone, "Unselect All", "Unselects all actors", EUserInterfaceActionType::Button, FInputChord( EKeys::Escape ) ) ;
	UI_COMMAND( InvertSelection, "Invert Selection", "Inverts the current selection", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( SelectImmediateChildren, "Select Immediate Children", "Selects immediate children of the current selection", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Alt|EModifierKey::Control, EKeys::D) );
	UI_COMMAND( SelectAllDescendants, "Select All Descendants", "Selects all descendants of the current selection", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift|EModifierKey::Control, EKeys::D) );
	UI_COMMAND( SelectAllActorsOfSameClass, "Select All Actors of Same Class", "Selects all the actors that have the same class", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift|EModifierKey::Control, EKeys::A) );
	UI_COMMAND( SelectAllActorsOfSameClassWithArchetype, "Select All Actors with Same Archetype", "Selects all the actors of the same class that have the same archetype", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SelectComponentOwnerActor, "Select Component Owner", "Select the actor that owns the currently selected component(s)", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SelectRelevantLights, "Select Relevant Lights", "Select all lights relevant to the current selection", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( SelectStaticMeshesOfSameClass, "Select All Using Selected Static Meshes (Selected Actor Types)", "Selects all actors with the same static mesh and actor class as the selection", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( SelectOwningHierarchicalLODCluster, "Select Owning Hierarchical LOD cluster Using Selected Static Mesh (Selected Actor Types)", "Select Owning Hierarchical LOD cluster for the selected actor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( SelectStaticMeshesAllClasses, "Select All Using Selected Static Meshes (All Actor Types)", "Selects all actors with the same static mesh as the selection", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift, EKeys::E ) ); 
	UI_COMMAND( SelectSkeletalMeshesOfSameClass, "Select All Using Selected Skeletal Meshes (Selected Actor Types)", "Selects all actors with the same skeletal mesh and actor class as the selection", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( SelectSkeletalMeshesAllClasses, "Select All Using Selected Skeletal Meshes (All Actor Types)", "Selects all actors with the same skeletal mesh as the selection", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( SelectAllWithSameMaterial, "Select All With Same Material", "Selects all actors with the same material as the selection", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( SelectMatchingEmitter, "Select All Matching Emitters", "Selects all emitters with the same particle system as the selection", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( SelectAllLights, "Select All Lights", "Selects all lights", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( SelectStationaryLightsExceedingOverlap, "Select Stationary Lights exceeding overlap", "Selects all stationary lights exceeding the overlap limit", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( SelectAllAddditiveBrushes, "Select All Additive Brushes", "Selects all additive brushes", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( SelectAllSubtractiveBrushes, "Select All Subtractive Brushes", "Selects all subtractive brushes", EUserInterfaceActionType::Button, FInputChord() ); 

	UI_COMMAND( SelectAllSurfaces, "Select All Surfaces", "Selects all bsp surfaces", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::S) ); 

	UI_COMMAND( SurfSelectAllMatchingBrush, "Select Matching Brush", "Selects the surfaces belonging to the same brush as the selected surfaces", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::B) ); 
	UI_COMMAND( SurfSelectAllMatchingTexture, "Select Matching Material", "Selects all surfaces with the same material as the selected surfaces", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::T) ); 
	UI_COMMAND( SurfSelectAllAdjacents, "Select All Adjacent Surfaces", "Selects all surfaces adjacent to the currently selected surfaces", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::J) ); 
	UI_COMMAND( SurfSelectAllAdjacentCoplanars, "Select All Coplanar Surfaces", "Selects all surfaces adjacent and coplanar with the selected surfaces", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::C) ); 
	UI_COMMAND( SurfSelectAllAdjacentWalls, "Select All Adjacent Wall Surfaces", "Selects all adjacent upright surfaces", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::W) ); 
	UI_COMMAND( SurfSelectAllAdjacentFloors, "Select All Adjacent Floor Surfaces", "Selects all adjacent floor sufaces(ones with normals pointing up)", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::U) ); 
	UI_COMMAND( SurfSelectAllAdjacentSlants, "Select All Adjacent Slant Surfaces", "Selects all adjacent slant surfaces (surfaces that are not walls, floors, or ceilings according to their normals) to the currently selected surfaces.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Y) ); 
	UI_COMMAND( SurfSelectReverse, "Invert Surface Selection", "Inverts the current surface selection", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Q) );
	UI_COMMAND( SurfSelectMemorize, "Memorize Surface Selection", "Stores the current surface selection in memory", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::M) ); 
	UI_COMMAND( SurfSelectRecall, "Recall Surface Selection", "Replace the current selection with the selection saved in memory", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::R) ); 
	UI_COMMAND( SurfSelectOr, "Surface Selection OR", "Replace the current selection with only the surfaces which are both currently selected and contained within the saved selection in memory", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::O) ); 
	UI_COMMAND( SurfSelectAnd, "Surface Selection AND", "Add the selection of surfaces saved in memory to the current selection", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::A) ); 
	UI_COMMAND( SurfSelectXor, "Surace Selection XOR", " Replace the current selection with only the surfaces that are not in both the current selection and the selection saved in memory", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::X) ); 
	UI_COMMAND( SurfUnalign, "Align Surface Default", "Default surface alignment", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SurfAlignPlanarAuto, "Align Surface Planar", "Planar surface alignment", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SurfAlignPlanarWall, "Align Surface Planar Wall", "Planar wall surface alignment", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SurfAlignPlanarFloor, "Align Surface Planar Floor", "Planar floor surface alignment", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SurfAlignBox, "Align Surface Box", "Box surface alignment", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SurfAlignFit, "Align Surface Fit", "Best fit surface alignment", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ApplyMaterialToSurface, "Apply Material to Surface Selection", "Applies the selected material to the selected surfaces", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( CreateBoundingBoxVolume, "Create Bounding Box Blocking Volume From Mesh", "Create a bounding box blocking volume from the static mesh", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CreateHeavyConvexVolume, "Heavy Convex Blocking Volume From Mesh", "Creates a heavy convex blocking volume from the static mesh", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CreateNormalConvexVolume, "Normal Convex Blocking Volume From Mesh", "Creates a normal convex blocking volume from the static mesh", EUserInterfaceActionType::Button, FInputChord() ); 
	UI_COMMAND( CreateLightConvexVolume, "Light Convex Blocking Volume From Mesh", "Creates a light convex blocking volume from the static mesh", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CreateRoughConvexVolume, "Rough Convex Blocking Volume From Mesh", "Creates a rough convex blocking volume from the static mesh", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( KeepSimulationChanges, "Keep Simulation Changes", "Saves the changes made to this actor in Simulate mode to the actor's default state.", EUserInterfaceActionType::Button, FInputChord( EKeys::K ) );

	UI_COMMAND( MakeActorLevelCurrent, "Make Selected Actor's Level Current", "Makes the selected actor's level the current level", EUserInterfaceActionType::Button, FInputChord( EKeys::M ) );
#if PLATFORM_MAC
	UI_COMMAND( MoveSelectedToCurrentLevel, "Move Selection to Current Level", "Moves the selected actors to the current level", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Command, EKeys::M ) );
#else
	UI_COMMAND( MoveSelectedToCurrentLevel, "Move Selection to Current Level", "Moves the selected actors to the current level", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Control, EKeys::M ) );
#endif
	UI_COMMAND( FindActorLevelInContentBrowser, "Find Actor Level in Content Browser", "Finds the selected actors' level in the content browser", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( FindLevelsInLevelBrowser, "Find Levels in Level Browser", "Finds the selected actors' levels in the level browser", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AddLevelsToSelection, "Add Levels to Selection", "Adds the selected actors' levels to the current level browser selection", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( RemoveLevelsFromSelection, "Remove Levels from Selection", "Removes the selected actors' levels from the current level browser selection", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( FindActorInLevelScript, "Find in Level Blueprint", "Finds any references to the selected actor in its level's blueprint", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( WorldProperties, "World Settings", "Displays the world settings", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenPlaceActors, "Place Actors Panel", "Opens the Place Actors Panel", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenContentBrowser, "Open Content Browser", "Opens the Content Browser", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control|EModifierKey::Shift, EKeys::F) );
	UI_COMMAND( ImportContent, "Import Content...", "Import Content into a specified location", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND( ToggleVR, "Toggle VR", "Toggles VR (Virtual Reality) mode", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Shift, EKeys::V ) );

	UI_COMMAND( OpenLevelBlueprint, "Open Level Blueprint", "Edit the Level Blueprint for the current level", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CheckOutProjectSettingsConfig, "Check Out", "Checks out the project settings config file so the game mode can be set.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CreateBlankBlueprintClass, "New Empty Blueprint Class...", "Create a new Blueprint Class", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ConvertSelectionToBlueprint, "Convert Selection to Blueprint Class...", "Replace all of the selected actors with a new Blueprint Class", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( ShowTransformWidget, "Show Transform Gizmo", "Toggles the visibility of the transform widgets", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( AllowTranslucentSelection, "Allow Translucent Selection", "Allows translucent objects to be selected", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::T) );
	UI_COMMAND( AllowGroupSelection, "Allow Group Selection", "Allows actor groups to be selected", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control|EModifierKey::Shift, EKeys::G) );
	UI_COMMAND( StrictBoxSelect, "Strict Box Selection", "When enabled an object must be entirely encompassed by the selection box when marquee box selecting", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( TransparentBoxSelect, "Box Select Occluded Objects", "When enabled, marquee box select operations will also select objects that are occluded by other objects.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowSelectionSubcomponents, "Show Subcomponents", "Toggles the visibility of the subcomponents related to the current selection", EUserInterfaceActionType::ToggleButton, FInputChord() );
	
	UI_COMMAND( DrawBrushMarkerPolys, "Draw Brush Polys", "Draws semi-transparent polygons around a brush when selected", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( OnlyLoadVisibleInPIE, "Only Load Visible Levels in Game Preview", "If enabled, when game preview starts, only visible levels will be loaded", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleSocketSnapping, "Socket", "Enables or disables snapping to sockets", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleParticleSystemLOD, "Enable Particle System LOD Switching", "If enabled particle systems will use distance LOD switching in perspective viewports", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleFreezeParticleSimulation, "Freeze Particle Simulation", "If enabled particle systems will freeze their simulation state", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleParticleSystemHelpers, "Toggle Particle System Helpers", "Toggles showing particle system helper widgets in viewports", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleLODViewLocking, "Enable LOD View Locking", "If enabled viewports of the same type will use the same LOD", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( LevelStreamingVolumePrevis, "Enable Automatic Level Streaming", "If enabled, the viewport will stream in levels automatically when the camera is moved", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( EnableActorSnap, "Actor", "If enabled, actors will snap to the location of other actors when they are within distance", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::K) );
	UI_COMMAND( EnableVertexSnap, "Vertex", "If enabled, actors will snap to the location of the nearest vertex on another actor in the direction of movement", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleHideViewportUI, "Show Viewport UI", "Sets the visibility of all overlaid viewport UI widgets.", EUserInterfaceActionType::ToggleButton, FInputChord() );

	//if (FParse::Param( FCommandLine::Get(), TEXT( "editortoolbox" ) ))
	//{
	//	UI_COMMAND( BspMode, "Enable Bsp Mode", "Enables BSP mode", EUserInterfaceActionType::ToggleButton, FInputChord( EModifierKey::Shift, EKeys::One ) );
	//	UI_COMMAND( MeshPaintMode, "Enable Mesh Paint Mode", "Enables mesh paint mode", EUserInterfaceActionType::ToggleButton, FInputChord( EModifierKey::Shift, EKeys::Two ) );
	//	UI_COMMAND( LandscapeMode, "Enable Landscape Mode", "Enables landscape editing", EUserInterfaceActionType::ToggleButton, FInputChord( EModifierKey::Shift, EKeys::Three ) );
	//	UI_COMMAND( FoliageMode, "Enable Foliage Mode", "Enables foliage editing", EUserInterfaceActionType::ToggleButton, FInputChord( EModifierKey::Shift, EKeys::Four ) );
	//}

	UI_COMMAND( ShowSelectedDetails, "Show Actor Details", "Opens a details panel for the selected actors", EUserInterfaceActionType::Button, FInputChord( EKeys::F4 ) );

	UI_COMMAND( RecompileShaders, "Recompile Changed Shaders", "Recompiles shaders which are out of date", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift|EModifierKey::Control, EKeys::Period ) );
	UI_COMMAND( ProfileGPU, "Profile GPU", "Profiles the GPU for the next frame and opens a window with profiled data", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift|EModifierKey::Control, EKeys::Comma ) );
	UI_COMMAND( DumpGPU, "Dump GPU", "Dump the GPU intermediary resources for the next frame and opens explorer", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND( ResetAllParticleSystems, "Reset All Particle Systems", "Resets all particle system emitters (removes all active particles and restarts them)", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift, EKeys::Slash ) );
	UI_COMMAND( ResetSelectedParticleSystem, "Resets Selected Particle Systems" , "Resets selected particle system emitters (removes all active particles and restarts them)", EUserInterfaceActionType::Button, FInputChord( EKeys::Slash ) );

	UI_COMMAND( SelectActorsInLayers, "Select all actors in selected actor's layers", "Selects all actors belonging to the layers of the currently selected actors", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Control, EKeys::L ) );

	UI_COMMAND(MaterialQualityLevel_Low, "Low", "Sets material quality in the scene to low.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MaterialQualityLevel_Medium, "Medium", "Sets material quality in the scene to medium.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MaterialQualityLevel_High, "High", "Sets material quality in the scene to high.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MaterialQualityLevel_Epic, "Epic", "Sets material quality in the scene to Epic.", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(TogglePreviewPlatform, "Preview Mode Toggle", "Toggles the Preview Mode on or off for the currently selected Preview target", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(AllowArcballRotation, "Enable Arcball Rotation", "Allow arcball rotation with rotate widget", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AllowScreenspaceRotation, "Enable Screenspace Rotation", "Allow screen rotation with rotate widget", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnableViewportHoverFeedback, "Preselection Highlight", "Enables real-time hover feedback when mousing over objects in editor view ports", EUserInterfaceActionType::ToggleButton, FInputChord());

	// Camera Preferences
	UI_COMMAND(OrbitCameraAroundSelection, "Orbit Around Selection", "If enabled, the camera will orbit around the current selection in the viewport", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(LinkOrthographicViewports, "Link Ortho Camera Movement", "If checked all orthographic view ports are linked to the same position and move together.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(OrthoZoomToCursor, "Ortho Zoom to Cursor", "If checked, in orthographic viewports zooming will center on the mouse position.  If unchecked, the zoom is around the center of the viewport.", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	// Mouse Controls
	UI_COMMAND(InvertMiddleMousePan, "Invert Middle Mouse Pan", "Whether or not to invert the direction of middle mouse panning in viewports", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(InvertOrbitYAxis, "Invert Orbit Axis", "Whether or not to invert mouse on y axis in orbit mode", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(InvertRightMouseDollyYAxis, "Invert Right Mouse Dolly", "Whether or not to invert the direction of right mouse dolly on the Y axis in orbit mode", EUserInterfaceActionType::ToggleButton, FInputChord());

	// Add preview platforms
	TSet<FName> PreviewShaderPlatformNames;
	for (const FPreviewPlatformMenuItem& Item : FDataDrivenPlatformInfoRegistry::GetAllPreviewPlatformMenuItems())
	{
		FTextBuilder FriendlyNameBuilder;
		bool bIsDisablePreview = false;
		if (!IsRunningCommandlet() && !GUsingNullRHI)
		{
			if (FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(Item.ShaderPlatformToPreview) == GMaxRHIShaderPlatform)
			{
				bIsDisablePreview = true;
				FriendlyNameBuilder.AppendLine(NSLOCTEXT("PreviewPlatform", "PreviewMenuText_DisablePreview", "Disable Preview"));
			}
			else
			{
				EShaderPlatform ShaderPlatform = FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(Item.PreviewShaderPlatformName);
				if (ShaderPlatform == SP_NumPlatforms)
				{
					// if the shader platform isn't compiled in, we don't have a friendly name available, so use ugly name
					FriendlyNameBuilder.AppendLine(FText::FromName(Item.PreviewShaderPlatformName));
				}
				else if (!Item.OptionalFriendlyNameOverride.IsEmpty())
				{
					FriendlyNameBuilder.AppendLine(Item.OptionalFriendlyNameOverride);
				}
				else
				{
					FriendlyNameBuilder.AppendLine(FDataDrivenShaderPlatformInfo::GetFriendlyName(ShaderPlatform, Item.DeviceProfileName));
				}
			}
		}

		PreviewPlatformOverrides.Add(
			FUICommandInfoDecl(
				this->AsShared(),
				FName(*FString::Printf(TEXT("PreviewPlatformOverrides_%s_%s_%s"), *Item.PlatformName.ToString(), *Item.ShaderFormat.ToString(), *Item.DeviceProfileName.ToString())),
				FriendlyNameBuilder.ToText(),
				Item.MenuTooltip)
			.UserInterfaceType(EUserInterfaceActionType::Check)
			.DefaultChord(FInputChord())
		);
		
		FName SectionName = FDataDrivenShaderPlatformInfo::GetLanguage(FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(Item.ShaderPlatformToPreview));
		if (bIsDisablePreview)
		{
			DisablePlatformPreview = PreviewPlatformOverrides.Last();
		}
		else
		{
			FLevelEditorCommands::PreviewPlatformCommand PreviewPlatform;
			PreviewPlatform.CommandInfo = PreviewPlatformOverrides.Last();
			PreviewPlatform.SectionName = SectionName;
			PlatformToPreviewPlatformOverrides.FindOrAdd(Item.PlatformName).Add(PreviewPlatform);
		}

		FConfigCacheIni* PlatformEngineIni = FConfigCacheIni::ForPlatform(*Item.PlatformName.ToString());
		FString DeviceProfileSelectionModule;

		if (PlatformEngineIni && PlatformEngineIni->GetString(TEXT("DeviceProfileManager"), TEXT("PreviewDeviceProfileSelectionModule"), DeviceProfileSelectionModule, GEngineIni))
		{
			if (IDeviceProfileSelectorModule* DPSelectorModule = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>(*DeviceProfileSelectionModule))
			{
				EShaderPlatform ShaderPlatform = FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(Item.ShaderPlatformToPreview);

				TArray<PreviewPlatformCommand>* PlatformToPreviewJsonPlatformOverridesValue = PlatformToPreviewJsonPlatformOverrides.Find(Item.PlatformName);
				if (!PlatformToPreviewJsonPlatformOverridesValue)
				{
					FLevelEditorCommands::PreviewPlatformCommand GenerateJsonPlatform;
					GenerateJsonPlatform.CommandInfo = FUICommandInfoDecl(
						this->AsShared(),
						FName(*FString::Printf(TEXT("Generate Platform Json for %s"), *Item.PlatformName.ToString())),
						NSLOCTEXT("GeneratePlatformJson", "Generate Platform Json", "Generate Platform Json"),
						NSLOCTEXT("GeneratePlatformJsonDesc", "Generate Platform Json From Connected Devices", "Generate Platform Json From Connected Devices"))
						.UserInterfaceType(EUserInterfaceActionType::Button)
						.DefaultChord(FInputChord());
					GenerateJsonPlatform.bIsGeneratingJsonCommand = true;

					PlatformToPreviewJsonPlatformOverridesValue = &PlatformToPreviewJsonPlatformOverrides.Add(Item.PlatformName);
					PlatformToPreviewJsonPlatformOverridesValue->Add(GenerateJsonPlatform);

					FLevelEditorCommands::PreviewPlatformCommand PreviewJsonPlatform;
					PreviewJsonPlatform.CommandInfo = FUICommandInfoDecl(
						this->AsShared(),
						FName(*FString::Printf(TEXT("Preview %s via Json"), *Item.PlatformName.ToString())),
						NSLOCTEXT("PreviewviaJson", "Preview via Json", "Preview via Json..."),
						NSLOCTEXT("PreviewviaJsonDesc", "Preview via Json", "Preview via Json"))
						.UserInterfaceType(EUserInterfaceActionType::Button)
						.DefaultChord(FInputChord());
					PreviewJsonPlatform.bIsGeneratingJsonCommand = false;
					PreviewJsonPlatform.SectionName = SectionName;
					PlatformToPreviewJsonPlatformOverridesValue->Add(PreviewJsonPlatform);

					TMap<FString, TArray<FString>> DirectoryToJsonFiles;
					FString AbsoluteDebugInfoDirectory = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*(FPaths::ProjectSavedDir() / TEXT("PreviewJsonDevices") / Item.PlatformName.ToString()));
					IFileManager::Get().FindFiles(DirectoryToJsonFiles.Add(AbsoluteDebugInfoDirectory), *AbsoluteDebugInfoDirectory, TEXT(".json"));
					FString ProjectEditorJsonDir = FPaths::ProjectContentDir() / TEXT("Editor") / TEXT("PreviewJsonDevices") / Item.PlatformName.ToString();
					IFileManager::Get().FindFiles(DirectoryToJsonFiles.Add(ProjectEditorJsonDir), *ProjectEditorJsonDir, TEXT(".json"));

					TSet<FString> UniqueJsons;
					for (auto Iter = DirectoryToJsonFiles.CreateConstIterator(); Iter; ++Iter)
					{
						FString DirectoryName = Iter.Key();
						for (const FString& JsonFile : Iter.Value())
						{
							if (!UniqueJsons.Find(JsonFile))
							{
								FString FullFilePath = DirectoryName / JsonFile;
								if(DPSelectorModule->IsJsonCompatible(FullFilePath))
								{
									FLevelEditorCommands::PreviewPlatformCommand PreviewJsonFilePlatform;
									PreviewJsonFilePlatform.CommandInfo = FUICommandInfoDecl(
										this->AsShared(),
										FName(*FString::Printf(TEXT("Preview %s with Json"), *JsonFile)),
										FText::Format(NSLOCTEXT("PreviewJson", "Preview Json", "Preview {0}"), FText::FromString(FPaths::GetBaseFilename(JsonFile))),
										FText::Format(NSLOCTEXT("PreviewJsonDesc", "Preview using Platform Json", "Preview {0}"), FText::FromString(FPaths::GetBaseFilename(JsonFile))))
										.UserInterfaceType(EUserInterfaceActionType::Check)
										.DefaultChord(FInputChord());
									PreviewJsonFilePlatform.bIsGeneratingJsonCommand = false;
									PreviewJsonFilePlatform.FilePath = DirectoryName / JsonFile;
									PreviewJsonFilePlatform.SectionName = SectionName;
									PlatformToPreviewJsonPlatformOverridesValue->Add(PreviewJsonFilePlatform);
								}
								UniqueJsons.Add(JsonFile);
							}
						}
					}
				}
			}
		}
	}

	PlatformToPreviewPlatformOverrides.KeyStableSort([](FName lhs, FName rhs) {return lhs.Compare(rhs) < 0; });

	UI_COMMAND(OpenMergeActor, "Merge Actors", "Opens the Merge Actor panel", EUserInterfaceActionType::Button, FInputChord());
}

FORCENOINLINE const FLevelEditorCommands& FLevelEditorCommands::Get()
{
	return TCommands<FLevelEditorCommands>::Get();
}

UE_ENABLE_OPTIMIZATION_SHIP

void FLevelEditorActionCallbacks::FixupGroupActor_Clicked()
{
	if (UActorGroupingUtils::IsGroupingActive())
	{
		AGroupActor::FixupGroupActor();
	}
}

#undef LOCTEXT_NAMESPACE
