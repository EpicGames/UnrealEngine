// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorCameraEditorState.h"
#include "LevelEditorInternalTools.h"
#include "LevelEditorViewport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelEditorCameraEditorState)

#define LOCTEXT_NAMESPACE "LevelEditorCameraEditorState"

ULevelEditorCameraEditorState::ULevelEditorCameraEditorState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CameraFOVAngle(90.0f)
{
}

FText ULevelEditorCameraEditorState::GetCategoryText() const
{
	return FText(LOCTEXT("LevelEditorCameraEditorStateCategoryText", "Camera"));
}

const FVector& ULevelEditorCameraEditorState::GetCameraLocation() const
{
	return CameraLocation;
}

const FRotator& ULevelEditorCameraEditorState::GetCameraRotation() const
{
	return CameraRotation;
}

const float ULevelEditorCameraEditorState::GetCameraFOVAngle() const
{
	return CameraFOVAngle;
}

UEditorState::FOperationResult ULevelEditorCameraEditorState::CaptureState()
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetActiveLevelViewport();
	if (!LevelViewport.IsValid())
	{
		return FOperationResult(FOperationResult::Failure, LOCTEXT("CaptureStateFailure_NoActiveViewport", "No active viewport"));
	}

	const FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();
	CameraLocation = LevelViewportClient.GetViewLocation();
	CameraRotation = LevelViewportClient.GetViewRotation();
	CameraFOVAngle = LevelViewportClient.FOVAngle;

	return FOperationResult(FOperationResult::Success, FText::Format(LOCTEXT("CaptureStateSuccess", "Location=({0}) Rotation=({1}) FOV={2}"), FText::FromString(CameraLocation.ToString()), FText::FromString(CameraRotation.ToString()), CameraFOVAngle));
}

UEditorState::FOperationResult ULevelEditorCameraEditorState::RestoreState() const
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		RestoreCameraState(FirstLevelEditor);
	}
	else if (!OnLevelEditorCreatedDelegateHandle.IsValid())
	{
		// Camera state will be applied as soon as the level viewport is created
		OnLevelEditorCreatedDelegateHandle = LevelEditorModule.OnLevelEditorCreated().AddUObject(this, &ULevelEditorCameraEditorState::RestoreCameraState);
	}

	return FOperationResult(FOperationResult::Success, FText::Format(LOCTEXT("RestoreStateSuccess", "Location=({0}) Rotation=({1}) FOV={2}"), FText::FromString(CameraLocation.ToString()), FText::FromString(CameraRotation.ToString()), CameraFOVAngle));
}

void ULevelEditorCameraEditorState::RestoreCameraState(TSharedPtr<ILevelEditor> LevelEditor) const
{
	// We want the delegate to be called once, so unregister ourself on the first call.
	if (OnLevelEditorCreatedDelegateHandle.IsValid())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.OnLevelEditorCreated().Remove(OnLevelEditorCreatedDelegateHandle);
		OnLevelEditorCreatedDelegateHandle.Reset();
	}

	TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditor->GetActiveViewportInterface();
	FLevelEditorViewportClient& LevelViewportClient = ActiveLevelViewport->GetLevelViewportClient();
	LevelViewportClient.SetViewLocation(CameraLocation);
	if (!LevelViewportClient.IsOrtho())
	{
		LevelViewportClient.SetViewRotation(CameraRotation);
	}

	LevelViewportClient.FOVAngle = CameraFOVAngle;
	LevelViewportClient.ViewFOV = CameraFOVAngle;
	LevelViewportClient.Invalidate();

	FEditorDelegates::OnEditorCameraMoved.Broadcast(CameraLocation, CameraRotation, LevelViewportClient.ViewportType, LevelViewportClient.ViewIndex);
}

#undef LOCTEXT_NAMESPACE
