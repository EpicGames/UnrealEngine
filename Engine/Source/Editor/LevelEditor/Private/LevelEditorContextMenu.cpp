// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorContextMenu.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Misc/Paths.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Engine/HitResult.h"
#include "Engine/Selection.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "SLevelEditor.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor/GroupActor.h"
#include "LevelEditorViewport.h"
#include "EditorModes.h"
#include "LevelEditor.h"
#include "LevelEditorInternalTools.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "AssetSelection.h"
#include "LevelEditorActions.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"
#include "ActorTreeItem.h"
#include "Kismet2/DebuggerCommands.h"
#include "Styling/SlateIconFinder.h"
#include "EditorViewportCommands.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "LevelEditorCreateActorMenu.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "SourceCodeNavigation.h"
#include "EditorClassUtils.h"
#include "Framework/Commands/GenericCommands.h"
#include "LevelViewportActions.h"
#include "ActorGroupingUtils.h"
#include "IMergeActorsModule.h"
#include "IMergeActorsTool.h"
#include "SLevelEditor.h"
#include "SLevelViewport.h"
#include "PropertyEditorModule.h"
#include "Misc/AxisDisplayInfo.h"

#define LOCTEXT_NAMESPACE "LevelViewportContextMenu"

DEFINE_LOG_CATEGORY_STATIC(LogViewportMenu, Log, All);

class FLevelEditorContextMenuImpl
{
public:
	static FSelectedActorInfo SelectionInfo;
public:
	/**
	 * Fills in menu options for the actor visibility menu
	 *
	 * @param MenuBuilder	The menu to add items to
	 */
	static void FillActorVisibilityMenu(UToolMenu* Menu);

	/** Fills in menu options for typed element visibility menu */
	static void FillElementVisibilityMenu(UToolMenu* Menu);

	/**
	 * Fills in menu options for the actor level menu
	 *
	 * @param SharedLevel			The level shared between all selected actors.  If any actors are in a different level, this is NULL
	 * @param bAllInCurrentLevel	true if all selected actors are in the current level
	 * @param MenuBuilder			The menu to add items to
	 */
	static void FillActorLevelMenu(UToolMenu* Menu);

	/**
	 * Fills in menu options for the transform menu
	 *
	 * @param MenuBuilder	The menu to add items to
	 */
	static void FillTransformMenu(UToolMenu* Menu);

	/**
	 * Fills in menu options for the Fill Actor menu
	 *
	 * @param MenuBuilder	The menu to add items to
	 */
	static void FillActorMenu(UToolMenu* Menu);

	/**
	 * Fills in menu options for the snap menu
	 *
	 * @param MenuBuilder	The menu to add items to
	 */
	static void FillSnapAlignMenu(UToolMenu* Menu);

	/**
	 * Fills in menu options for the pivot menu
	 *
	 * @param MenuBuilder	The menu to add items to
	 */
	static void FillPivotMenu(UToolMenu* Menu);
	
	/**
	 * Fills in menu options for the group menu
	 *
	 * @param MenuBuilder	The menu to add items to
	 */
	static void FillGroupMenu( UToolMenu* Menu );

	/**
	 * Fills in menu options for the edit menu
	 *
	 * @param MenuBuilder	The menu to add items to
	 * @param ContextType	The context for this editor menu
	 */
	static void FillEditMenu(UToolMenu* Menu) { FillEditMenu(Menu, nullptr); }
	static void FillEditMenu(UToolMenu* Menu, FToolMenuSection* InSection);

	static void FillBulkEditComponentsMenu(UToolMenu* Menu);

	/**
	 * Fills in the menu options for the Asset Tools submenu
	 *
	 * @param Menu			The menu to add items to
	 */
	static void FillAssetToolsMenu(UToolMenu* Menu);

	/**
	 * Fills in menu options for the merge actors menu
	 * 
	 * @param MenuBuilder	The menu to add items to
	 */
	static void FillMergeActorsMenu(UToolMenu* Menu);

	/**
	 * Adds the Source Control SubMenu to the provided Section.
	 *
	 * @param Section	The menu to add items to
	 */
	static void AddSourceControlMenu(FToolMenuSection& Section);

	/**
	 * Fills in menu options for the source control menu
	 *
	 * @param MenuBuilder	The menu to add items to
	 */
	static void FillSourceControlMenu(UToolMenu* Menu);

	/**
	 * Adds the entry for Select Immediate Children depending on the current selection
	 * 
	 * @param Section			The section to add items to
	 * @param SelectedActors	Current list of selected actors
	 * @param Context			Menu context, used to determine whether entry is needed
	 */
	static void AddSelectChildrenEntry(FToolMenuSection& Section, const TArray<AActor*>& SelectedActors, ULevelEditorContextMenuContext* Context);
};

FSelectedActorInfo FLevelEditorContextMenuImpl::SelectionInfo;

struct FLevelScriptEventMenuHelper
{
	/**
	* Fills in menu options for events that can be associated with that actors's blueprint in the level script blueprint
	*
	* @param MenuBuilder	The menu to add items to
	*/
	static void FillLevelBlueprintEventsMenu(FToolMenuSection& Section, const TArray<AActor*>& SelectedActors);
};

void FLevelEditorContextMenu::RegisterComponentContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered("LevelEditor.ComponentContextMenu"))
	{
		return;
	}

	UToolMenu* Menu = ToolMenus->RegisterMenu("LevelEditor.ComponentContextMenu");
	Menu->AddDynamicSection("ComponentControlDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		ULevelEditorContextMenuContext* LevelEditorContext = InMenu->FindContext<ULevelEditorContextMenuContext>();
		if (!LevelEditorContext)
		{
			return;
		}

		TArray<UActorComponent*> SelectedComponents;
		for (FSelectedEditableComponentIterator It(GEditor->GetSelectedEditableComponentIterator()); It; ++It)
		{
			SelectedComponents.Add(CastChecked<UActorComponent>(*It));
		}

		{
			FToolMenuSection& Section = InMenu->AddSection("ComponentControl", LOCTEXT("ComponentControlHeading", "Component"));

			AActor* OwnerActor = GEditor->GetSelectedActors()->GetTop<AActor>();
			if(OwnerActor)
			{
				Section.AddMenuEntry(
					FLevelEditorCommands::Get().SelectComponentOwnerActor,
					FText::Format(LOCTEXT("SelectComponentOwner", "Select Owner [{0}]"), FText::FromString(OwnerActor->GetHumanReadableName())),
					TAttribute<FText>(),
					FSlateIconFinder::FindIconForClass(OwnerActor->GetClass())
				);
			}

			Section.AddMenuEntry(FEditorViewportCommands::Get().FocusViewportToSelection);

			const FVector* ClickLocation = &GEditor->ClickLocation;
			FUIAction GoHereAction;
			GoHereAction.ExecuteAction = FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::GoHere_Clicked, ClickLocation);

			Section.AddMenuEntry(FLevelEditorCommands::Get().GoHere);
			Section.AddMenuEntry(FLevelEditorCommands::Get().SnapCameraToObject);
			Section.AddMenuEntry(FLevelEditorCommands::Get().SnapObjectToCamera);
			AddPlayFromHereSubMenu(Section);						
			Section.AddMenuEntry(FLevelEditorCommands::Get().CopyActorFilePathtoClipboard);

			FLevelEditorContextMenuImpl::AddSourceControlMenu(Section);
		}

		FComponentEditorUtils::FillComponentContextMenuOptions(InMenu, SelectedComponents);
	}));
}

void FLevelEditorContextMenu::AddPlayFromHereSubMenu(FToolMenuSection& Section)
{
	if(FLevelEditorActionCallbacks::PlayFromHere_IsVisible())
	{
		Section.AddSubMenu("PlayFromHere", LOCTEXT("PlayFromHere", "Play From Here"), FText(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			FToolMenuSection& NewSection = InMenu->AddSection("Section");
			FUIAction PlayFromHere(FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::PlayFromHere_Clicked, false));
			FUIAction PlayFromHereFloating(FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::PlayFromHere_Clicked, true));

			NewSection.AddMenuEntry("PlayFromHereActiveViewport", LOCTEXT("PlayFromHereActiveViewport", "Selected Viewport"), LOCTEXT("PlayFromHereActiveViewportTooltip", "Play from this actor in the active level editor viewport"), FSlateIcon("EditorSTyle", "PlayWorld.PlayInViewport"), PlayFromHere);
			NewSection.AddMenuEntry("PlayFromHereFloatingWindow", LOCTEXT("PlayFromHereFloatingWindow", "New Editor Window (PIE)"), LOCTEXT("PlayFromHereFloatingWindowTooltip", "Play from this actor in a new editor window"), FSlateIcon("EditorSTyle", "PlayWorld.PlayInEditorFloating"), PlayFromHereFloating);
		}));
	}
}

void FLevelEditorContextMenu::RegisterActorContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered("LevelEditor.ActorContextMenu"))
	{
		return;
	}

	UToolMenu* Menu = ToolMenus->RegisterMenu("LevelEditor.ActorContextMenu");
	Menu->AddDynamicSection("ActorContextMenuDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		ULevelEditorContextMenuContext* LevelEditorContext = InMenu->FindContext<ULevelEditorContextMenuContext>();
		if (!LevelEditorContext || !LevelEditorContext->LevelEditor.IsValid())
		{
			return;
		}

		TWeakPtr<ILevelEditor> LevelEditor = LevelEditorContext->LevelEditor;

		// Generate information about our selection
		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

		FSelectedActorInfo& SelectionInfo = FLevelEditorContextMenuImpl::SelectionInfo;
		SelectionInfo = AssetSelectionUtils::BuildSelectedActorInfo(SelectedActors);

		int32 NumSelectedActors = SelectedActors.Num();

		{
			// General actions that apply to most actors and their underlying assets
			// In most cases, you DO NOT want to extend this section; look at ActorUETools or ActorTypeTools below
			FToolMenuSection& Section = InMenu->AddSection("ActorGeneral", LOCTEXT("AssetOptionsHeading", "Asset Options"));

			// Check if current selection has any referenced assets that can be edited
			TArray< UObject* > ReferencedAssets;
			GEditor->GetReferencedAssetsForEditorSelection(ReferencedAssets);

			TArray< FSoftObjectPath> SoftReferencedAssets;
			GEditor->GetSoftReferencedAssetsForEditorSelection(SoftReferencedAssets);

			// Asset type icon is used in multiple places below
			FSlateIcon AssetIcon = ReferencedAssets.Num() == 1 ? FSlateIconFinder::FindIconForClass(ReferencedAssets[0]->GetClass()) : FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Default");

			// Edit and Find entries (a) always appear in main menu, and (b) appear in right-click menu if referenced asset is available
			if (LevelEditorContext->ContextType == ELevelEditorMenuContext::MainMenu || ReferencedAssets.Num() > 0 || SoftReferencedAssets.Num() > 0 || SelectionInfo.bHaveBrowseOverride)
			{
				Section.AddMenuEntry(FGlobalEditorCommonCommands::Get().FindInContentBrowser);

				if (ReferencedAssets.Num() == 0)
				{
					Section.AddMenuEntry(
						FLevelEditorCommands::Get().EditAsset,
						TAttribute<FText>(), // use command's label
						TAttribute<FText>(), // use command's tooltip
						AssetIcon
					);
				}
				else if (ReferencedAssets.Num() == 1)
				{
					UObject*  Asset = ReferencedAssets[0];
					const FString AssetLabel = Cast<AActor>(Asset) ? Cast<AActor>(Asset)->GetActorNameOrLabel() : Asset->GetName();

					Section.AddMenuEntry(
						FLevelEditorCommands::Get().EditAsset,
						FText::Format(LOCTEXT("EditAssociatedAsset", "Edit {0}"), FText::FromString(AssetLabel)),
						TAttribute<FText>(), // use command's tooltip
						AssetIcon
					);
				}
				else if (ReferencedAssets.Num() > 1)
				{
					Section.AddMenuEntry(
						FLevelEditorCommands::Get().EditAssetNoConfirmMultiple,
						LOCTEXT("EditAssociatedAssetsMultiple", "Edit Multiple Assets"),
						TAttribute<FText>(), // use command's tooltip
						AssetIcon
					);
				}
			}

			if (LevelEditorContext->ContextType == ELevelEditorMenuContext::MainMenu)
			{
				Section.AddSubMenu(
					"AssetToolsSubMenu",
					LOCTEXT("AssetToolsSubMenu", "Asset Tools"),
					LOCTEXT("AssetToolsSubMenuToolTip", "Tools for the asset associated with the selected actor"),
					FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillAssetToolsMenu),
					/*bInOpenSubMenuOnClick*/ false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Adjust"));

				// This is an invisible entry used as an extension point for "Convert SomeActor To SomeType" entries
				FUIAction Action;
				Action.IsActionVisibleDelegate = FIsActionButtonVisible::CreateLambda([]() { return false; });
				Section.AddMenuEntry("ActorConvert", TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Convert"), Action, EUserInterfaceActionType::None);
			}

			Section.AddMenuEntry(
				FLevelEditorCommands::Get().CopyActorFilePathtoClipboard,
				TAttribute<FText>(), // use command's label
				TAttribute<FText>(), // use command's tooltip
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy")
			);

			Section.AddMenuEntry(
				FLevelEditorCommands::Get().OpenActorInReferenceViewer,
				TAttribute<FText>(), // use command's label
				TAttribute<FText>(), // use command's tooltip
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.ReferenceViewer")
			);

			Section.AddMenuEntry(
				FLevelEditorCommands::Get().SaveActor,
				TAttribute<FText>(), // use command's label
				TAttribute<FText>(), // use command's tooltip
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save")
			);

			LevelEditorCreateActorMenu::FillAddReplaceContextMenuSections(Section, LevelEditorContext);
		}
		
		if(FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" ).GetCanUsePropertyMatrix() 
			&& NumSelectedActors >= 2)
		{
			// Options that relate to bulk editing assets
			// In most cases, you DO NOT want to extend this section; look at ActorUETools or ActorTypeTools below
			
			// Figure out whether the selected actors are actually the same thing we would see in a detail panel,
			//  or whether they are actually stand-ins for some non-actors.
			bool bDetailsObjectsAreNotActors = InternalEditorLevelLibrary::DoesDetailSelectionContainNonActorsOrComponents(LevelEditorContext->CurrentSelection);

			if (!bDetailsObjectsAreNotActors || FPropertyEditorModule::CVarAllowPropertyMatrixForNonActors.GetValueOnAnyThread())
			{
				FToolMenuSection& Section = InMenu->AddSection("ActorBulkEdit", LOCTEXT("ActorBulkEditHeading", "Bulk Editing"));

				Section.AddMenuEntry(
					FLevelEditorCommands::Get().OpenSelectionInPropertyMatrix,
					TAttribute<FText>(), // use command's label
					TAttribute<FText>(), // use command's tooltip
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "DetailsView.EditRawProperties")
				);

				// The previous entry for opening the selection in the property matrix works even in cases where 
				//  the selected actors are stand-ins for something different that is editable (since the true 
				//  targets will be gotten via ITypedElementDetailsInterface). On the other hand, editing components
				//  of the selection is harder to generalize in a way that is overridable by plugins. So, we will
				//  not add this entry in that case. A plugin that uses stand-in actors can add its own "Edit Components"
				//  entry if it likes.
				if (!bDetailsObjectsAreNotActors)
				{
					Section.AddSubMenu(
						"BulkEditComponentsSubmenu",
						LOCTEXT("BulkEditComponentsSubmenuName", "Edit Components in the Property Matrix"),
						LOCTEXT("BulkEditComponentsSubmenuTooltip", "Bulk Edit any editable components that are common between the selected actors in the Property Matrix"),
						FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillBulkEditComponentsMenu),
						FUIAction(),
						EUserInterfaceActionType::Button,
						false, // default value
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "DetailsView.EditRawProperties")
					);
				}
			}
		}


		{
			// Options that affect the current viewport.
			// In most cases, you DO NOT want to extend this section; look at ActorUETools or ActorTypeTools below
			FToolMenuSection& Section = InMenu->AddSection("ActorViewOptions", LOCTEXT("ViewOptionsHeading", "View Options"));
			const FVector* ClickLocation = &GEditor->ClickLocation;

			Section.AddMenuEntry(
				FEditorViewportCommands::Get().FocusViewportToSelection,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FrameActor")
			);

			// This keys off the mouse position so can only appear in the viewport
			if (LevelEditorContext->ContextType == ELevelEditorMenuContext::Viewport)
			{
				FUIAction GoHereAction;
				GoHereAction.ExecuteAction = FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::GoHere_Clicked, ClickLocation);
				Section.AddMenuEntry(
					FLevelEditorCommands::Get().GoHere,
					TAttribute<FText>(),
					TAttribute<FText>(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.GoHere")
				);
			}

			Section.AddMenuEntry(
				FLevelEditorCommands::Get().SnapCameraToObject,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.SnapViewToObject")
			);

			Section.AddMenuEntry(
				FLevelEditorCommands::Get().SnapObjectToCamera,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.SnapObjectToView")
			);

			if (SelectedActors.Num() == 1)
			{
				const FLevelViewportCommands& Actions = FLevelViewportCommands::Get();

				auto Viewport = LevelEditor.Pin()->GetActiveViewportInterface();
				if (Viewport.IsValid())
				{
					auto& ViewportClient = Viewport->GetLevelViewportClient();

					if (ViewportClient.IsPerspective() && !ViewportClient.IsLockedToCinematic())
					{
						if (Viewport->IsSelectedActorLocked())
						{
							Section.AddMenuEntry(
								Actions.EjectActorPilot,
								FText::Format(LOCTEXT("PilotActor_Stop", "Stop piloting '{0}'"), FText::FromString(SelectedActors[0]->GetActorLabel()))
								);
						}
						else
						{
							Section.AddMenuEntry(
								Actions.PilotSelectedActor,
								FText::Format(LOCTEXT("PilotActor", "Pilot '{0}'"), FText::FromString(SelectedActors[0]->GetActorLabel()))
								);
						}

						// Only allowed on actor with bCanPlayFromHere = true
						FToolMenuEntry& PlayFromHereEntry = Section.AddMenuEntry(FLevelEditorCommands::Get().PlayFromHere);
						PlayFromHereEntry.Visibility = FIsActionButtonVisible::CreateStatic(&FLevelEditorActionCallbacks::PlayFromHere_IsVisible);
					}
				}
			}
		}

		{
			// Options for editing, transforming, and manipulating this actor
			// In most cases, you DO NOT want to extend this section; look at ActorUETools or ActorTypeTools below
			FToolMenuSection& Section = InMenu->AddSection("ActorOptions", LOCTEXT("ActorOptionsHeading", "Actor Options"));

			FLevelEditorContextMenuImpl::AddSelectChildrenEntry(Section, SelectedActors, LevelEditorContext);

			Section.AddSubMenu(
				"EditSubMenu",
				LOCTEXT("EditSubMenu", "Edit"),
				FText::GetEmpty(),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillEditMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"));

			Section.AddSubMenu(
				"VisibilitySubMenu",
				LOCTEXT("VisibilitySubMenu", "Visibility"),
				LOCTEXT("VisibilitySubMenu_ToolTip", "Selected actor visibility options"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillActorVisibilityMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Visibility"));

			Section.AddSubMenu(
				"TransformSubMenu",
				LOCTEXT("TransformSubMenu", "Transform"),
				LOCTEXT("TransformSubMenu_ToolTip", "Actor transform utils"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillTransformMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Transform"));

			Section.AddSubMenu(
				"SnapAlignSubMenu",
				LOCTEXT("SnapAlignSubMenu", "Snapping"),
				LOCTEXT("SnapAlignSubMenu_ToolTip", "Actor snap/align utils"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillSnapAlignMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Snap"));

			Section.AddSubMenu(
				"PivotSubMenu",
				LOCTEXT("PivotSubMenu", "Pivot"),
				LOCTEXT("PivotSubMenu_ToolTip", "Actor pivoting utils"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillPivotMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.SetShowPivot"));

			// Build the menu for grouping actors - this is either the Group item or Groups submenu
			BuildGroupMenu(Section, SelectionInfo);

			// Attach and detach
			Section.AddSubMenu(
				"ActorAttachToSubMenu",
				LOCTEXT("ActorAttachToSubMenu", "Attach To"),
				LOCTEXT("ActorAttachToSubMenu_ToolTip", "Attach Actor as child"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillActorMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.Attach"));

			Section.AddMenuEntry(
				FLevelEditorCommands::Get().DetachFromParent,
				TAttribute<FText>(), // Use command title
				TAttribute<FText>(), // Use command tooltip
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.Detach"));

			// Add/jump to event should go in main menu only
			if (LevelEditorContext->ContextType == ELevelEditorMenuContext::MainMenu)
			{
				FLevelScriptEventMenuHelper::FillLevelBlueprintEventsMenu(Section, SelectedActors);
			}
		}
			
		// General-purpose extension point for tools that apply to many types of actors
		// These should appear in the main menu context only, by design
		// For type-specific actions, consider adding them to "ActorTypeTools" below
		if (LevelEditorContext->ContextType == ELevelEditorMenuContext::MainMenu)
		{
			FToolMenuSection& Section = InMenu->AddSection("ActorUETools", LOCTEXT("UEToolsHeading", "UE Tools"));

			Section.AddSubMenu(
				"MergeActorsSubMenu",
				FText::Format(LOCTEXT("MergeActorsSubMenu", "Merge Actors ({0})"), FText::AsNumber(SelectedActors.Num())),
				LOCTEXT("MergeActorsSubMenu_ToolTip", "Merge actors utils"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillMergeActorsMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Merge"));
		}

		// General-purpose extension point for tools that only appear for certain types of actors
		// Generally, you should only use this for type-specific actions since this section appears in all contexts
		// For tools that apply to many types of actors, add them to "ActorUETools" above
		FToolMenuSection& ActorTypeToolsSection = InMenu->AddSection("ActorTypeTools");

		ActorTypeToolsSection.AddSubMenu(
			"LevelSubMenu",
			LOCTEXT("LevelSubMenu", "Level"),
			LOCTEXT("LevelSubMenu_ToolTip", "Options for interacting with this actor's level"),
			FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillActorLevelMenu),
			false, // default value
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Level"));
		
		// DEPRECATED SECTION NAMES -- DO NOT ADD NEW EXTENSIONS TO THESE POINTS -- THEY MAY BE REMOVED IN THE FUTURE
		// These sections are included here because they used to exist and may have been used as extension points
		// For new context menu entries, use "ActorUETools" or "ActorTypeTools" above
		InMenu->AddSection("ActorAsset");
		InMenu->AddSection("ActorControl");
		InMenu->AddSection("ActorSelectVisibilityLevels");
		InMenu->AddSection("ActorType");
		InMenu->AddSection("LevelViewportAttach");

		{
			// Play from here, keep simulation changes
			FToolMenuSection& Section = InMenu->AddSection("ActorPreview", LOCTEXT("PreviewHeading", "Preview"));

			if (LevelEditorContext->ContextType == ELevelEditorMenuContext::Viewport)
			{
				AddPlayFromHereSubMenu(Section);

				// Only extend if above PlayFromHere option isn't available
				if (!FLevelEditorActionCallbacks::PlayFromHere_IsVisible())
				{
					const FText PlayFromHereLabel = GEditor->OnlyLoadEditorVisibleLevelsInPIE() ? LOCTEXT("PlayFromHereVisible", "Play From Here (visible levels)") : LOCTEXT("PlayFromHere", "Play From Here");
					Section.AddMenuEntry(FLevelEditorCommands::Get().PlayFromHere, PlayFromHereLabel);
				}
			}

			if (GEditor->PlayWorld != NULL)
			{
				if (SelectionInfo.NumSelected > 0)
				{
					Section.AddMenuEntry(FLevelEditorCommands::Get().KeepSimulationChanges);
				}
			}
		}

	}));
}

void FLevelEditorContextMenu::RegisterElementContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered("LevelEditor.ElementContextMenu"))
	{
		return;
	}

	UToolMenu* Menu = ToolMenus->RegisterMenu("LevelEditor.ElementContextMenu");
	Menu->AddDynamicSection("ElementContextMenuDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		ULevelEditorContextMenuContext* LevelEditorContext = InMenu->FindContext<ULevelEditorContextMenuContext>();
		if (!LevelEditorContext || !LevelEditorContext->LevelEditor.IsValid())
		{
			return;
		}

		// This menu will mostly follow FLevelEditorContextMenu::RegisterActorContextMenu as more support for
		//  non-actors is added.

		{
			// Options that affect the current viewport.
			FToolMenuSection& Section = InMenu->AddSection("ElementViewOptions", LOCTEXT("ViewOptionsHeading", "View Options"));
			const FVector* ClickLocation = &GEditor->ClickLocation;

			Section.AddMenuEntry(
				FEditorViewportCommands::Get().FocusViewportToSelection,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FrameActor")
			);

			// This keys off the mouse position so can only appear in the viewport
			if (LevelEditorContext->ContextType == ELevelEditorMenuContext::Viewport)
			{
				FUIAction GoHereAction;
				GoHereAction.ExecuteAction = FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::GoHere_Clicked, ClickLocation);
				Section.AddMenuEntry(
					FLevelEditorCommands::Get().GoHere,
					TAttribute<FText>(),
					TAttribute<FText>(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.GoHere")
				);
			}

			Section.AddMenuEntry(
				FLevelEditorCommands::Get().SnapCameraToObject,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.SnapViewToObject")
			);

			Section.AddMenuEntry(
				FLevelEditorCommands::Get().SnapObjectToCamera,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.SnapObjectToView")
			);

			// TODO: Support piloting like we do for actors. This requires changes to viewport code.
		}

		{
			// Options for editing, transforming, and manipulating this element
			FToolMenuSection& Section = InMenu->AddSection("ElementOptions", LOCTEXT("ElementOptionsHeading", "Element Options"));

			// TODO: Duplicate and Rename in this submenu still need implementing. Duplicate requires implementing
			//  UEntityElementWorldInterface::DuplicateElements, whereas Rename requires support from the outliner,
			//  as the outliner requires something to be selected in it.
			Section.AddSubMenu(
				"EditSubMenu",
				LOCTEXT("EditSubMenu", "Edit"),
				FText::GetEmpty(),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillEditMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"));

			Section.AddSubMenu(
				"VisibilitySubMenu",
				LOCTEXT("VisibilityElementSubMenu", "Visibility"),
				LOCTEXT("VisibilityElementSubMenu_ToolTip", "Selected element visibility options"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillElementVisibilityMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Visibility"));

			Section.AddSubMenu(
				"TransformSubMenu",
				LOCTEXT("TransformElementSubMenu", "Transform"),
				LOCTEXT("TransformElementSubMenu_ToolTip", "Element transform utils"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillTransformMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Transform"));

			Section.AddSubMenu(
				"SnapAlignSubMenu",
				LOCTEXT("SnapAlignElementSubMenu", "Snapping"),
				LOCTEXT("SnapAlignElementSubMenu_ToolTip", "Element snap/align utils"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillSnapAlignMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Snap"));

			Section.AddSubMenu(
				"PivotSubMenu",
				LOCTEXT("PivotElementSubMenu", "Pivot"),
				LOCTEXT("PivotElementSubMenu_ToolTip", "Element pivoting utils"),
				FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillPivotMenu),
				false, // default value
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.SetShowPivot"));

			// TODO: Attach/detach requires additional support. Attach, in particular, requires ability to show
			//  elements to which attachment is possible.
			//Section.AddSubMenu(
			//	"ElementAttachToSubMenu",
			//	LOCTEXT("ElementAttachToSubMenu", "Attach To"),
			//	LOCTEXT("ElementAttachToSubMenu_ToolTip", "Attach Element as child"),
			//	FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillActorMenu),
			//	false, // default value
			//	FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.Attach"));
			//Section.AddMenuEntry(
			//	FLevelEditorCommands::Get().DetachFromParent,
			//	TAttribute<FText>(), // Use command title
			//	TAttribute<FText>(), // Use command tooltip
			//	FSlateIcon(FAppStyle::GetAppStyleSetName(), "Actors.Detach"));
		}
			
		// General-purpose extension point for tools that apply to many types of elements, analogous to ActorUETools
		if (LevelEditorContext->ContextType == ELevelEditorMenuContext::MainMenu)
		{
			FToolMenuSection& Section = InMenu->AddSection("ElementUETools", LOCTEXT("UEToolsHeading", "UE Tools"));
		}

		// General-purpose extension point for tools that only appear for certain types of elements, analogous to ActorTypeTools
		FToolMenuSection& ElementTypeToolsSection = InMenu->AddSection("ElementTypeTools");

		{
			// Play from here
			// TODO: May someday support custom play-from-here behavior the way that some actors do, and
			//  may someday support simulation changes
			FToolMenuSection& Section = InMenu->AddSection("ElementPreview", LOCTEXT("PreviewHeading", "Preview"));
			const FText PlayFromHereLabel = GEditor->OnlyLoadEditorVisibleLevelsInPIE() ? LOCTEXT("PlayFromHereVisible", "Play From Here (visible levels)") : LOCTEXT("PlayFromHere", "Play From Here");
			Section.AddMenuEntry(FLevelEditorCommands::Get().PlayFromHere, PlayFromHereLabel);
		}
	}));
}

void FLevelEditorContextMenu::RegisterSceneOutlinerContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered("LevelEditor.SceneOutlinerContextMenu"))
	{
		return;
	}

	UToolMenu* Menu = ToolMenus->RegisterMenu("LevelEditor.SceneOutlinerContextMenu");
	Menu->AddDynamicSection("SelectVisibilityLevels", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		if (ULevelEditorContextMenuContext* LevelEditorContext = InMenu->FindContext<ULevelEditorContextMenuContext>())
		{
			TWeakPtr<ISceneOutliner> SceneOutlinerPtr = LevelEditorContext->LevelEditor.Pin()->GetMostRecentlyUsedSceneOutliner();
			if (SceneOutlinerPtr.IsValid())
			{
				FToolMenuSection& Section = InMenu->AddSection("SelectVisibilityLevels");
				Section.AddSubMenu(
					"EditSubMenu",
					LOCTEXT("EditSubMenu", "Edit"),
					FText::GetEmpty(),
					FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillEditMenu)
				);
			}
		}
	}));
}

void FLevelEditorContextMenu::RegisterMenuBarEmptyContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered("LevelEditor.MenuBarEmptyContextMenu"))
	{
		return;
	}

	UToolMenu* Menu = ToolMenus->RegisterMenu("LevelEditor.MenuBarEmptyContextMenu");
	FToolMenuSection& Section = Menu->AddSection("MenuBarEmpty");

	const FText EmptySelectionInformationalMessage = LOCTEXT("EmptySelectionInformationalMessage", "Select an object to view actions.");

	Section.AddEntry(FToolMenuEntry::InitWidget(
		NAME_None,
		SNew(SBox)
		.Padding(FMargin(80.f, 8.f))
		[
			SNew(STextBlock)
			.Text(EmptySelectionInformationalMessage)
			.TextStyle(FAppStyle::Get(), "HintText")
		],
		FText::GetEmpty(), /*bNoIndent*/ true, /*bSearchable*/ false));
}

void FLevelEditorContextMenu::RegisterEmptySelectionContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered("LevelEditor.EmptySelectionContextMenu"))
	{
		return;
	}

	UToolMenu* Menu = ToolMenus->RegisterMenu("LevelEditor.EmptySelectionContextMenu");
	Menu->AddDynamicSection("PlaceActors", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		if (ULevelEditorContextMenuContext* LevelEditorContext = InMenu->FindContext<ULevelEditorContextMenuContext>())
		{
			{
				FToolMenuSection& Section = InMenu->AddSection("SelectActorGeneral", LOCTEXT("SelectAnyHeading", "General"));
				Section.AddMenuEntry(FGenericCommands::Get().SelectAll, TAttribute<FText>(), LOCTEXT("SelectAll_ToolTip", "Selects all actors"));
			}

			if (LevelEditorContext->ContextType == ELevelEditorMenuContext::Viewport)
			{
				FToolMenuSection& Section = InMenu->AddSection("ActorType");
				LevelEditorCreateActorMenu::FillAddReplaceContextMenuSections(Section, LevelEditorContext);
			}
		}
	}));
}

void FLevelEditorContextMenu::RegisterAsDerivedMenu(FName MenuNameToUse, FName BaseMenuName, const FToolMenuInsert& InsertLocation)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ensure(ToolMenus))
	{
		return;
	}

	if (!ToolMenus->IsMenuRegistered(MenuNameToUse))
	{
		// Inherit from the passed-in context
		UToolMenu* Menu = ToolMenus->RegisterMenu(MenuNameToUse, BaseMenuName);

		// Add everything that we would show in the level editor context menu
		FToolMenuSection& Section = Menu->AddDynamicSection("LevelEditorContextMenu", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			ULevelEditorContextMenuContext* LevelEditorContext = InMenu->FindContext<ULevelEditorContextMenuContext>();
			if (!LevelEditorContext)
			{
				return;
			}

			FName LevelContextMenuName = FLevelEditorContextMenu::GetContextMenuName(ELevelEditorMenuContext::SceneOutliner, LevelEditorContext->CurrentSelection);
			if (LevelContextMenuName != NAME_None)
			{
				UToolMenu* OtherMenu = UToolMenus::Get()->GenerateMenu(LevelContextMenuName, InMenu->Context);
				InMenu->Sections.Append(OtherMenu->Sections);
			}
		}));
		Section.InsertPosition = InsertLocation;
	}
}

FName FLevelEditorContextMenu::GetContextMenuName(ELevelEditorMenuContext ContextType, const UTypedElementSelectionSet* InSelectionSet)
{
	if (InSelectionSet)
	{
		if (InSelectionSet->HasSelectedObjects<UActorComponent>())
		{
			return "LevelEditor.ComponentContextMenu";
		}

		if (InSelectionSet->HasSelectedObjects<AActor>())
		{
			return "LevelEditor.ActorContextMenu";
		}

		if (InSelectionSet->GetNumSelectedElements())
		{
			return "LevelEditor.ElementContextMenu";
		}
	}

	if (ContextType == ELevelEditorMenuContext::SceneOutliner)
	{
		return "LevelEditor.SceneOutlinerContextMenu";
	}

	if (ContextType == ELevelEditorMenuContext::MainMenu)
	{
		return "LevelEditor.MenuBarEmptyContextMenu";
	}

	return "LevelEditor.EmptySelectionContextMenu";
}

FText FLevelEditorContextMenu::GetContextMenuTitle(ELevelEditorMenuContext ContextType, const UTypedElementSelectionSet* InSelectionSet)
{
	if (InSelectionSet)
	{
		if (InSelectionSet->HasSelectedObjects<UActorComponent>())
		{
			return LOCTEXT("ComponentContextMenuTitle", "Component");
		}

		if (InSelectionSet->HasSelectedObjects<AActor>())
		{
			return LOCTEXT("ActorContextMenuTitle", "Actor");
		}

		if (InSelectionSet->GetNumSelectedElements())
		{
			return LOCTEXT("ElementContextMenuTitle", "Element");
		}
	}

	// Show "Actor" label by default as title when nothing selected since most selections (currently) will be actors anyways
	return LOCTEXT("ActorContextMenuTitle", "Actor");
}

FText FLevelEditorContextMenu::GetContextMenuToolTip(ELevelEditorMenuContext ContextType, const UTypedElementSelectionSet* InSelectionSet)
{
	if (InSelectionSet)
	{
		const int32 ComponentCount = InSelectionSet->CountSelectedObjects<UActorComponent>();
		if (ComponentCount == 1)
		{
			UActorComponent* Component = InSelectionSet->GetTopSelectedObject<UActorComponent>();
			check(Component);
			return FText::Format(LOCTEXT("ComponentContextMenuToolTipSingle", "Show actions for component \"{0}\""), FText::FromString(Component->GetName()));
		}
		else if (ComponentCount > 1)
		{
			return FText::Format(LOCTEXT("ComponentContextMenuToolTipOther", "Show actions for {0} components"), FText::AsNumber(ComponentCount));
		}

		const int32 ActorCount = InSelectionSet->CountSelectedObjects<AActor>();
		if (ActorCount == 1)
		{
			AActor* Actor = InSelectionSet->GetTopSelectedObject<AActor>();
			check(Actor);
			return FText::Format(LOCTEXT("ActorContextMenuToolTipSingle", "Show actions for actor \"{0}\""), FText::FromString(Actor->GetActorLabel()));
		}
		else if (ActorCount > 1)
		{
			return FText::Format(LOCTEXT("ActorContextMenuToolTipOther", "Show actions for {0} actors"), FText::AsNumber(ActorCount));
		}

		const int32 ElementCount = InSelectionSet->GetNumSelectedElements();
		if (ElementCount)
		{
			return FText::Format(LOCTEXT("ElementContextMenuToolTip", "Show actions for {0} {0}|plural(one=element,other=elements)"), FText::AsNumber(ElementCount));
		}
	}

	return LOCTEXT("NothingSelectedToolTip", "Select an object to show actions");
}

FName FLevelEditorContextMenu::InitMenuContext(FToolMenuContext& Context, TWeakPtr<ILevelEditor> LevelEditor, ELevelEditorMenuContext ContextType, const FTypedElementHandle& HitProxyElement)
{
	RegisterComponentContextMenu();
	RegisterActorContextMenu();
	RegisterElementContextMenu();
	RegisterSceneOutlinerContextMenu();
	RegisterMenuBarEmptyContextMenu();
	RegisterEmptySelectionContextMenu();

	TSharedPtr<ILevelEditor> LevelEditorPtr = LevelEditor.Pin();
	check(LevelEditorPtr);

	TSharedPtr<FUICommandList> LevelEditorActionsList = LevelEditorPtr->GetLevelEditorActions();
	Context.AppendCommandList(LevelEditorActionsList);

	ULevelEditorContextMenuContext* ContextObject = NewObject<ULevelEditorContextMenuContext>();
	ContextObject->LevelEditor = LevelEditor;
	ContextObject->ContextType = ContextType;
	ContextObject->CurrentSelection = LevelEditorPtr->GetElementSelectionSet();
	ContextObject->HitProxyElement = HitProxyElement;
	{
		TTypedElement<ITypedElementObjectInterface> HitProxyObjectElement = ContextObject->CurrentSelection->GetElementList()->GetElement<ITypedElementObjectInterface>(ContextObject->HitProxyElement);
		ContextObject->HitProxyActor = HitProxyObjectElement ? Cast<AActor>(HitProxyObjectElement.GetObject()) : nullptr;
	}
	for (FSelectedEditableComponentIterator It(GEditor->GetSelectedEditableComponentIterator()); It; ++It)
	{
		ContextObject->SelectedComponents.Add(CastChecked<UActorComponent>(*It));
	}

	// obtain the world location of the cursor
	if (GCurrentLevelEditingViewportClient)
	{
		FHitResult HitResult;
		FViewportCursorLocation CursorLocation = GCurrentLevelEditingViewportClient->GetCursorWorldLocationFromMousePos();
		FCollisionQueryParams LineParams(SCENE_QUERY_STAT(FocusOnPoint), true);

		if (GCurrentLevelEditingViewportClient->GetWorld()->LineTraceSingleByObjectType(
			HitResult,
			CursorLocation.GetOrigin(),
			CursorLocation.GetOrigin() + CursorLocation.GetDirection() * HALF_WORLD_MAX,
			FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects),
			LineParams))
		{
			ContextObject->CursorWorldLocation = HitResult.ImpactPoint;
		}
	}
	
	Context.AddObject(ContextObject, [](UObject* InContext)
	{
		ULevelEditorContextMenuContext* CastContext = CastChecked<ULevelEditorContextMenuContext>(InContext);
		CastContext->CurrentSelection = nullptr;
		CastContext->HitProxyElement.Release();
	});

	if (GEditor->GetSelectedComponentCount() == 0 && GEditor->GetSelectedActorCount() > 0)
	{
		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

		// Get all menu extenders for this context menu from the level editor module
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TArray<FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors> MenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
		TArray<TSharedPtr<FExtender>> Extenders;
		for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
		{
			if (MenuExtenderDelegates[i].IsBound())
			{
				Extenders.Add(MenuExtenderDelegates[i].Execute(LevelEditorActionsList.ToSharedRef(), SelectedActors));
			}
		}

		if (Extenders.Num() > 0)
		{
			Context.AddExtender(FExtender::Combine(Extenders));
		}
	}

	return GetContextMenuName(ContextType, ContextObject->CurrentSelection);
}

UToolMenu* FLevelEditorContextMenu::GenerateMenu(TWeakPtr<ILevelEditor> LevelEditor, ELevelEditorMenuContext ContextType, TSharedPtr<FExtender> Extender, const FTypedElementHandle& HitProxyElement)
{
	FToolMenuContext Context;
	if (Extender.IsValid())
	{
		Context.AddExtender(Extender);
	}

	FName ContextMenuName = InitMenuContext(Context, LevelEditor, ContextType, HitProxyElement);
	return UToolMenus::Get()->GenerateMenu(ContextMenuName, Context);
}

// NOTE: We intentionally receive a WEAK pointer here because we want to be callable by a delegate whose
//       payload contains a weak reference to a level editor instance
TSharedRef< SWidget > FLevelEditorContextMenu::BuildMenuWidget(TWeakPtr< ILevelEditor > LevelEditor, ELevelEditorMenuContext ContextType, TSharedPtr<FExtender> Extender, const FTypedElementHandle& HitProxyElement)
{
	UToolMenu* Menu = GenerateMenu(LevelEditor, ContextType, Extender, HitProxyElement);
	return UToolMenus::Get()->GenerateWidget(Menu);
}

namespace EViewOptionType
{
	enum Type
	{
		Top,
		Bottom,
		Left,
		Right,
		Front,
		Back,
		Perspective
	};
}

TSharedPtr<SWidget> MakeViewOptionWidget(const TSharedRef< SLevelEditor >& LevelEditor, bool bShouldCloseWindowAfterMenuSelection, EViewOptionType::Type ViewOptionType)
{
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, LevelEditor->GetActiveViewport()->GetCommandList());

	if (ViewOptionType == EViewOptionType::Top)
	{
		MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Top);
	}
	else if (ViewOptionType == EViewOptionType::Bottom)
	{
		MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Bottom);
	}
	else if (ViewOptionType == EViewOptionType::Left)
	{
		MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Left);
	}
	else if (ViewOptionType == EViewOptionType::Right)
	{
		MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Right);
	}
	else if (ViewOptionType == EViewOptionType::Front)
	{
		MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Front);
	}
	else if (ViewOptionType == EViewOptionType::Back)
	{
		MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Back);
	}
	else if (ViewOptionType == EViewOptionType::Perspective)
	{
		MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Perspective);
	}
	else
	{
		return nullptr;
	}
	return MenuBuilder.MakeWidget();
}

void BuildViewOptionMenu(const TSharedRef< SLevelEditor >& LevelEditor, TSharedPtr<SWidget> InWidget, const FVector2D WidgetPosition)
{
	if (InWidget.IsValid())
	{
		FSlateApplication::Get().PushMenu(
			LevelEditor->GetActiveViewport().ToSharedRef(),
			FWidgetPath(),
			InWidget.ToSharedRef(),
			WidgetPosition,
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
	}
}

void FLevelEditorContextMenu::SummonViewOptionMenu( const TSharedRef< SLevelEditor >& LevelEditor, const ELevelViewportType ViewOption )
{
	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

	bool bShouldCloseWindowAfterMenuSelection = true;
	EViewOptionType::Type ViewOptionType = EViewOptionType::Perspective;

	switch (ViewOption)
	{
		case LVT_OrthoBottom:
			ViewOptionType = EViewOptionType::Bottom;
			break;
		case LVT_OrthoBack:
			ViewOptionType = EViewOptionType::Back;
			break;
		case LVT_OrthoRight:
			ViewOptionType = EViewOptionType::Right;
			break;
		case LVT_OrthoTop:
			ViewOptionType = EViewOptionType::Top;
			break;
		case LVT_OrthoFront:
			ViewOptionType = EViewOptionType::Front;
			break;
		case LVT_OrthoLeft:
			ViewOptionType = EViewOptionType::Left;
			break;
		case LVT_Perspective:
			ViewOptionType = EViewOptionType::Perspective;
			break;
	};
	// Build up menu
	BuildViewOptionMenu(LevelEditor, MakeViewOptionWidget(LevelEditor, bShouldCloseWindowAfterMenuSelection, ViewOptionType), MouseCursorLocation);
}

void FLevelEditorContextMenu::SummonMenu(const TSharedRef< SLevelEditor >& LevelEditor, ELevelEditorMenuContext ContextType, const FTypedElementHandle& HitProxyElement)
{
	// Create the context menu!
	TSharedPtr<SWidget> MenuWidget = BuildMenuWidget( LevelEditor, ContextType, nullptr, HitProxyElement );
	if ( MenuWidget.IsValid() )
	{
		// @todo: Should actually use the location from a click event instead!
		const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();
	
		FSlateApplication::Get().PushMenu(
			LevelEditor->GetActiveViewport().ToSharedRef(),
			FWidgetPath(),
			MenuWidget.ToSharedRef(),
			MouseCursorLocation,
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu ) );
	}
}

FSlateColor InvertOnHover( const TWeakPtr< SWidget > WidgetPtr )
{
	TSharedPtr< SWidget > Widget = WidgetPtr.Pin();
	if ( Widget.IsValid() && Widget->IsHovered() )
	{
		static const FName InvertedForegroundName("InvertedForeground");
		return FAppStyle::GetSlateColor(InvertedForegroundName);
	}

	return FSlateColor::UseForeground();
}

void FLevelEditorContextMenu::BuildGroupMenu(FToolMenuSection& Section, const FSelectedActorInfo& SelectedActorInfo)
{
	if( UActorGroupingUtils::IsGroupingActive() )
	{
		// Whether or not we added a grouping sub-menu
		bool bNeedGroupSubMenu = SelectedActorInfo.bHaveSelectedLockedGroup || SelectedActorInfo.bHaveSelectedUnlockedGroup;
		
		if( bNeedGroupSubMenu )
		{
			Section.AddSubMenu(
				"GroupMenu",
				LOCTEXT("GroupMenu", "Groups"),
				LOCTEXT("GroupMenu_ToolTip", "Opens the actor grouping menu"),
				FNewToolMenuDelegate::CreateStatic( &FLevelEditorContextMenuImpl::FillGroupMenu),
				false, // default
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.GroupActors"));
		}
		else
		{
			Section.AddMenuEntry(
				FLevelEditorCommands::Get().RegroupActors, FLevelEditorCommands::Get().GroupActors->GetLabel(), FLevelEditorCommands::Get().GroupActors->GetDescription(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.GroupActors"));
		}
	}
}

void FLevelEditorContextMenuImpl::FillElementVisibilityMenu(UToolMenu* Menu)
{
	{
		FToolMenuSection& Section = Menu->AddSection("VisibilitySelected", LOCTEXT("VisibilitySelectedHeader", "Selection"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleSelectedVisibility);
		Section.AddMenuEntry(FLevelEditorCommands::Get().HideSelected);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ShowSelected);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleSelectedHierarchyVisibility);
		Section.AddMenuEntry(FLevelEditorCommands::Get().HideSelectedHierarchy);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ShowSelectedHierarchy);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("VisibilityAll", LOCTEXT("VisibilityAllHeader", "World"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().ShowSelectedOnly);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ShowAll);
	}
}

void FLevelEditorContextMenuImpl::FillActorVisibilityMenu(UToolMenu* Menu)
{
	FillElementVisibilityMenu(Menu);

	{
		FToolMenuSection& Section = Menu->AddSection("VisibilityStartup", LOCTEXT("VisibilityStartupHeader", "Startup"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().ShowAllStartup);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ShowSelectedStartup);
		Section.AddMenuEntry(FLevelEditorCommands::Get().HideSelectedStartup);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ShowSelectedHierarchyStartup);
		Section.AddMenuEntry(FLevelEditorCommands::Get().HideSelectedHierarchyStartup);
	}
}

void FLevelEditorContextMenuImpl::FillActorLevelMenu(UToolMenu* Menu)
{
	ULevelEditorContextMenuContext* LevelEditorContext = Menu->FindContext<ULevelEditorContextMenuContext>();
	if (!LevelEditorContext || !LevelEditorContext->LevelEditor.IsValid())
	{
		return;
	}

	if (LevelEditorContext->ContextType == ELevelEditorMenuContext::MainMenu)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("ActorLevel", LOCTEXT("ActorLevel", "Actor Level"));
			if (SelectionInfo.SharedLevel && !SelectionInfo.SharedLevel->IsInstancedLevel() && SelectionInfo.SharedWorld && SelectionInfo.SharedWorld->GetCurrentLevel() != SelectionInfo.SharedLevel)
			{
				// All actors are in the same level and that level is not the current level 
				// so add a menu entry to make the shared level current

				FText MakeCurrentLevelText = FText::Format(LOCTEXT("MakeCurrentLevelMenu", "Make Current Level: {0}"), FText::FromString(SelectionInfo.SharedLevel->GetOutermost()->GetName()));
				Section.AddMenuEntry(FLevelEditorCommands::Get().MakeActorLevelCurrent, MakeCurrentLevelText);
			}

			if (!SelectionInfo.bAllSelectedActorsBelongToCurrentLevel)
			{
				// Only show this menu entry if any actors are not in the current level
				Section.AddMenuEntry(FLevelEditorCommands::Get().MoveSelectedToCurrentLevel);
			}

			Section.AddMenuEntry(FLevelEditorCommands::Get().FindActorLevelInContentBrowser);
		}

		{
			FToolMenuSection& Section = Menu->AddSection("LevelBlueprint", LOCTEXT("LevelBlueprint", "Level Blueprint"));
			Section.AddMenuEntry(FLevelEditorCommands::Get().FindActorInLevelScript);
		}

		{
			FToolMenuSection& Section = Menu->AddSection("LevelBrowser", LOCTEXT("LevelBrowser", "Level Browser"));
			Section.AddMenuEntry(FLevelEditorCommands::Get().FindLevelsInLevelBrowser);
			Section.AddMenuEntry(FLevelEditorCommands::Get().AddLevelsToSelection);
			Section.AddMenuEntry(FLevelEditorCommands::Get().RemoveLevelsFromSelection);
		}
	}
}


void FLevelEditorContextMenuImpl::FillTransformMenu(UToolMenu* Menu)
{
	if (ULevelEditorContextMenuContext* LevelEditorContext = Menu->FindContext<ULevelEditorContextMenuContext>())
	{
		if (!LevelEditorContext->CurrentSelection || LevelEditorContext->CurrentSelection->GetNumSelectedElements() == 0)
		{
			return;
		}

		{
			FToolMenuSection& Section = Menu->AddSection("DeltaTransformToActors");
			Section.AddMenuEntry(FLevelEditorCommands::Get().DeltaTransformToActors);
		}

		FToolMenuSection& Section = Menu->AddSection("MirrorLock");

		if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
		{
			Section.AddMenuEntry(FLevelEditorCommands::Get().MirrorActorY);
			Section.AddMenuEntry(FLevelEditorCommands::Get().MirrorActorZ);
			Section.AddMenuEntry(FLevelEditorCommands::Get().MirrorActorX);
		}
		else
		{
			Section.AddMenuEntry(FLevelEditorCommands::Get().MirrorActorX);
			Section.AddMenuEntry(FLevelEditorCommands::Get().MirrorActorY);
			Section.AddMenuEntry(FLevelEditorCommands::Get().MirrorActorZ);
		}

		Section.AddMenuEntry(FLevelEditorCommands::Get().LockActorMovement);
	}
}

// A box that will expand to match its content's desired size, but will never shrink.
// A helper class for the AttachToActor menu so that it does not constantly resize all the time,
// but also ensure that you're never in a state where you can't read the full actor name.
class SOnlyExpandsBox : public SBox
{
protected:
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
	{
		UE::Slate::FDeprecateVector2DResult RequestedDesiredSize = UE::Slate::CastToVector2f(SBox::ComputeDesiredSize(LayoutScaleMultiplier));
		if (RequestedDesiredSize.X > MaxPreviousWidth)
		{
			MaxPreviousWidth = RequestedDesiredSize.X;
			return RequestedDesiredSize;
		}
		else
		{
			return FVector2D(MaxPreviousWidth, RequestedDesiredSize.Y);
		}
	}
private:
	mutable float MaxPreviousWidth = 400.0f;
};

void FLevelEditorContextMenuImpl::FillActorMenu(UToolMenu* Menu)
{
	// The contents of this menu are added as a custom widget with its own search field so we
	// disable searching in this parent menu to avoid displaying two search fields to the user.
	Menu->bSearchable = false;

	struct Local
	{
		static FReply OnInteractiveActorPickerClicked()
		{
			FSlateApplication::Get().DismissAllMenus();
			FLevelEditorActionCallbacks::AttachActorIteractive();
			return FReply::Handled();
		}
	};

	FSceneOutlinerInitializationOptions InitOptions;
	{	
		InitOptions.bShowHeaderRow = false;
		InitOptions.bFocusSearchBoxWhenOpened = true;

		// Only display Actors that we can attach too
		InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateStatic(&FLevelEditorActionCallbacks::IsAttachableActor));
	}		

	FToolMenuSection& Section = Menu->AddSection("Actor");
	if(SelectionInfo.bHaveAttachedActor)
	{
		Section.AddMenuEntry(FLevelEditorCommands::Get().DetachFromParent, LOCTEXT("None", "None"));
	}

	// Actor selector to allow the user to choose a parent actor
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>( "SceneOutliner" );

	TSharedRef< SWidget > MenuWidget = 
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.MaxHeight(400.0f)
			[
				SNew(SOnlyExpandsBox)
				[
					SceneOutlinerModule.CreateActorPicker(
						InitOptions,
						FOnActorPicked::CreateStatic( &FLevelEditorActionCallbacks::AttachToActor )
						)
				]
			]
		]
	
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoWidth()
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.ToolTipText( LOCTEXT( "PickButtonLabel", "Pick a parent actor to attach to") )
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(FOnClicked::CreateStatic(&Local::OnInteractiveActorPickerClicked))
				.ContentPadding(4.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.EyeDropper"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];

	Section.AddEntry(FToolMenuEntry::InitWidget("PickParentActor", MenuWidget, FText::GetEmpty(), false));
}

void FLevelEditorContextMenuImpl::FillSnapAlignMenu(UToolMenu* Menu)
{
	FToolMenuSection& Section = Menu->AddSection("SnapAlign");
	Section.AddMenuEntry( FLevelEditorCommands::Get().SnapOriginToGrid );
	Section.AddMenuEntry( FLevelEditorCommands::Get().SnapOriginToGridPerActor );
	Section.AddMenuEntry( FLevelEditorCommands::Get().AlignOriginToGrid );
	Section.AddMenuEntry( FLevelEditorCommands::Get().SnapTo2DLayer );
	Section.AddMenuEntry( FLevelEditorCommands::Get().SnapToFloor );
	Section.AddMenuEntry( FLevelEditorCommands::Get().AlignToFloor );
	Section.AddMenuEntry( FLevelEditorCommands::Get().SnapPivotToFloor );
	Section.AddMenuEntry( FLevelEditorCommands::Get().AlignPivotToFloor );
	Section.AddMenuEntry( FLevelEditorCommands::Get().SnapBottomCenterBoundsToFloor );
	Section.AddMenuEntry( FLevelEditorCommands::Get().AlignBottomCenterBoundsToFloor );
/*
	Section.AddSeparator();
	AActor* Actor = GEditor->GetSelectedActors()->GetBottom<AActor>();
	if( Actor && FLevelEditorActionCallbacks::ActorsSelected_CanExecute())
	{
		const FString Label = Actor->GetActorLabel();	// Update the options to show the actors label
		
		TSharedPtr< FUICommandInfo > SnapOriginToActor = FLevelEditorCommands::Get().SnapOriginToActor;
		TSharedPtr< FUICommandInfo > AlignOriginToActor = FLevelEditorCommands::Get().AlignOriginToActor;
		TSharedPtr< FUICommandInfo > SnapToActor = FLevelEditorCommands::Get().SnapToActor;
		TSharedPtr< FUICommandInfo > AlignToActor = FLevelEditorCommands::Get().AlignToActor;
		TSharedPtr< FUICommandInfo > SnapPivotToActor = FLevelEditorCommands::Get().SnapPivotToActor;
		TSharedPtr< FUICommandInfo > AlignPivotToActor = FLevelEditorCommands::Get().AlignPivotToActor;
		TSharedPtr< FUICommandInfo > SnapBottomCenterBoundsToActor = FLevelEditorCommands::Get().SnapBottomCenterBoundsToActor;
		TSharedPtr< FUICommandInfo > AlignBottomCenterBoundsToActor = FLevelEditorCommands::Get().AlignBottomCenterBoundsToActor;

		SnapOriginToActor->Label = FString::Printf( *LOCTEXT("Snap Origin To", "Snap Origin to %s"), *Label);
		AlignOriginToActor->Label = FString::Printf( *LOCTEXT("Align Origin To", "Align Origin to %s"), *Label);
		SnapToActor->Label = FString::Printf( *LOCTEXT("Snap To", "Snap to %s"), *Label);
		AlignToActor->Label = FString::Printf( *LOCTEXT("Align To", "Align to %s"), *Label);
		SnapPivotToActor->Label = FString::Printf( *LOCTEXT("Snap Pivot To", "Snap Pivot to %s"), *Label);
		AlignPivotToActor->Label = FString::Printf( *LOCTEXT("Align Pivot To", "Align Pivot to %s"), *Label);
		SnapBottomCenterBoundsToActor->Label = FString::Printf( *LOCTEXT("Snap Bottom Center Bounds To", "Snap Bottom Center Bounds to %s"), *Label);
		AlignBottomCenterBoundsToActor->Label = FString::Printf( *LOCTEXT("Align Bottom Center Bounds To", "Align Bottom Center Bounds to %s"), *Label);

		Section.AddMenuEntry( SnapOriginToActor );
		Section.AddMenuEntry( AlignOriginToActor );
		Section.AddMenuEntry( SnapToActor );
		Section.AddMenuEntry( AlignToActor );
		Section.AddMenuEntry( SnapPivotToActor );
		Section.AddMenuEntry( AlignPivotToActor );
		Section.AddMenuEntry( SnapBottomCenterBoundsToActor );
		Section.AddMenuEntry( AlignBottomCenterBoundsToActor );
	}
	else
	{
		Section.AddMenuEntry( FLevelEditorCommands::Get().SnapOriginToActor );
		Section.AddMenuEntry( FLevelEditorCommands::Get().AlignOriginToActor );
		Section.AddMenuEntry( FLevelEditorCommands::Get().SnapToActor );
		Section.AddMenuEntry( FLevelEditorCommands::Get().AlignToActor );
		Section.AddMenuEntry( FLevelEditorCommands::Get().SnapPivotToActor );
		Section.AddMenuEntry( FLevelEditorCommands::Get().AlignPivotToActor );
		Section.AddMenuEntry( FLevelEditorCommands::Get().SnapBottomCenterBoundsToActor );
		Section.AddMenuEntry( FLevelEditorCommands::Get().AlignBottomCenterBoundsToActor );
	}
*/
}

void FLevelEditorContextMenuImpl::FillPivotMenu( UToolMenu* Menu )
{
	{
		FToolMenuSection& Section = Menu->AddSection("SaveResetPivot");
		Section.AddMenuEntry(FLevelEditorCommands::Get().SavePivotToPrePivot);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ResetPrePivot);

		if (ULevelEditorContextMenuContext* LevelEditorContext = Menu->FindContext<ULevelEditorContextMenuContext>())
		{
			if (LevelEditorContext->ContextType == ELevelEditorMenuContext::Viewport)
			{
				Section.AddMenuEntry(FLevelEditorCommands::Get().MovePivotHere);
				Section.AddMenuEntry(FLevelEditorCommands::Get().MovePivotHereSnapped);
			}
		}
	}

	{
		FToolMenuSection& Section = Menu->AddSection("MovePivot");
		Section.AddMenuEntry(FLevelEditorCommands::Get().MovePivotToCenter);
	}
}

void FLevelEditorContextMenuImpl::FillGroupMenu( UToolMenu* Menu )
{
	FToolMenuSection& Section = Menu->AddSection("Group");

	if( SelectionInfo.NumSelectedUngroupedActors > 1 )
	{
		// Only show this menu item if we have more than one actor.
		Section.AddMenuEntry( FLevelEditorCommands::Get().GroupActors );
	}

	if( SelectionInfo.bHaveSelectedLockedGroup || SelectionInfo.bHaveSelectedUnlockedGroup )
	{
		const int32 NumActiveGroups = AGroupActor::NumActiveGroups(true);

		// Regroup will clear any existing groups and create a new one from the selection
		// Only allow regrouping if multiple groups are selected, or a group and ungrouped actors are selected
		if( NumActiveGroups > 1 || (NumActiveGroups && SelectionInfo.NumSelectedUngroupedActors) )
		{
			Section.AddMenuEntry( FLevelEditorCommands::Get().RegroupActors );
		}

		Section.AddMenuEntry( FLevelEditorCommands::Get().UngroupActors );

		if( SelectionInfo.bHaveSelectedUnlockedGroup )
		{
			// Only allow removal of loose actors or locked subgroups
			if( !SelectionInfo.bHaveSelectedLockedGroup || ( SelectionInfo.bHaveSelectedLockedGroup && SelectionInfo.bHaveSelectedSubGroup ) )
			{
				Section.AddMenuEntry( FLevelEditorCommands::Get().RemoveActorsFromGroup );
			}
			Section.AddMenuEntry( FLevelEditorCommands::Get().LockGroup );
		}

		if( SelectionInfo.bHaveSelectedLockedGroup )
		{
			Section.AddMenuEntry( FLevelEditorCommands::Get().UnlockGroup );
		}

		// Only allow group adds if a single group is selected in addition to ungrouped actors
		if( AGroupActor::NumActiveGroups(true, false) == 1 && SelectionInfo.NumSelectedUngroupedActors )
		{ 
			Section.AddMenuEntry( FLevelEditorCommands::Get().AddActorsToGroup );
		}

		if (AGroupActor::SelectedGroupNeedsFixup())
		{
			Section.AddMenuEntry(FLevelEditorCommands::Get().FixupGroupActor);
		}
	}
}

void FLevelEditorContextMenuImpl::FillEditMenu( UToolMenu* Menu, FToolMenuSection* InSection )
{
	FToolMenuSection& Section = InSection ? *InSection : Menu->AddSection("Section");

	Section.AddMenuEntry( FGenericCommands::Get().Cut );
	Section.AddMenuEntry( FGenericCommands::Get().Copy );
	Section.AddMenuEntry( FGenericCommands::Get().Paste );
	if (ULevelEditorContextMenuContext* LevelEditorContext = Menu->FindContext<ULevelEditorContextMenuContext>())
	{
		if (LevelEditorContext->ContextType == ELevelEditorMenuContext::Viewport)
		{
			Section.AddMenuEntry(FLevelEditorCommands::Get().PasteHere);
		}
	}

	Section.AddMenuEntry( FGenericCommands::Get().Duplicate );
	Section.AddMenuEntry( FGenericCommands::Get().Delete );
	Section.AddMenuEntry( FGenericCommands::Get().Rename );
}

void FLevelEditorContextMenuImpl::FillBulkEditComponentsMenu( UToolMenu* Menu )
{
	/** This is how the logic for selecting components to bulk edit works at a high level:
	 *  We try to find the common types of components that are shared between all the actors in the selection, and then
	 *  add a menu entry for each of the common component types which opens them in the Property Matrix
	 *  To achieve this, we pick the components of the first actor in the selection as a base, and then remove any
	 *  that are not shared with every subsequent actor in the selection, early exiting if there are no common components.
	 */
	
	FToolMenuSection& Section = Menu->AddSection("Section");

	ULevelEditorContextMenuContext* LevelEditorContext = Menu->FindContext<ULevelEditorContextMenuContext>();
	
	if (!LevelEditorContext)
	{
		return;
	}

	TObjectPtr<const UTypedElementSelectionSet> Selection = LevelEditorContext->CurrentSelection;

	if(!Selection)
	{
		return;
	}
	
	TArray<AActor*> SelectedActors = Selection->GetSelectedObjects<AActor>();

	if(SelectedActors.Num() == 0)
	{
		return;
	}
	
	/* Map to keep track of components that are common between the current selection
	 * The key represents the type of the component, while the value contains the actual components from each object in the selection
	 */
	TMap<TSubclassOf<UActorComponent>, TArray< UObject* >> CommonComponents;

	// We start by filling in CommonComponents with all the components of the first actor in the selection
	const AActor* FirstActor = SelectedActors[0];
	for( UActorComponent* Component : FirstActor->GetComponents())
	{
		// Make sure the component can be edited
		if(FComponentEditorUtils::CanEditComponentInstance(Component, Cast<USceneComponent>(Component), false))
		{
			// If this type of component already exists, simply add it to the array
			if(TArray< UObject* >* FoundCommonComponent = CommonComponents.Find(Component->GetClass()))
			{
				FoundCommonComponent->Add(Component);
			}
			// Otherwise add a new entry for this component type
			else
			{
				TArray< UObject* > CurrentActorComponents;
				CurrentActorComponents.Add(Component);
				CommonComponents.Add(Component->GetClass(), CurrentActorComponents);
			}
			
		}
	}

	// We start iterating from the second object since we've already used the first as a base
	TArray<AActor*>::TIterator ActorIt = SelectedActors.CreateIterator();
	++ActorIt;
	
	for(; ActorIt; ++ActorIt)
	{
		// Iterate through each common component
		for(TMap<TSubclassOf<UActorComponent>, TArray< UObject* >>::TIterator ComponentIt = CommonComponents.CreateIterator(); ComponentIt; ++ComponentIt)
		{
			// Get all components of the current component type from this actor
			TArray<UActorComponent*> ComponentsOfType;
			(*ActorIt)->GetComponents(ComponentIt.Key(), ComponentsOfType);

			bool bHasComponentOfCurrentType = false;

			/* For each component the current actor has, we make sure it is EXACTLY the same type as the common component
			 * because GetComponents also checks SubClasses. i.e if CommonComponent is SceneComponent and CurrentComponent
			 * is StaticMeshComponent, we do not want to multi edit them currently
			 */
			for(UActorComponent* CurrentComponent : ComponentsOfType)
			{
				if(ComponentIt.Key() == CurrentComponent->GetClass() && FComponentEditorUtils::CanEditComponentInstance(CurrentComponent, Cast<USceneComponent>(CurrentComponent), false))
				{
					ComponentIt.Value().Add(CurrentComponent);
					bHasComponentOfCurrentType = true;
				}
			}

			// If this actor has no components of this type, discard this type
			if(!bHasComponentOfCurrentType)
			{
				ComponentIt.RemoveCurrent();
			}
		}

		// Early out if there are no common components between all actors in the selection
		if(CommonComponents.IsEmpty())
		{
			break;
		}
		
	}

	// If we have no common components, show some text indicating so
	if(CommonComponents.IsEmpty())
	{
		Section.AddEntry(FToolMenuEntry::InitWidget(
		NAME_None,
		SNew(SBox)
		.Padding(FMargin(6.f, 0.f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NoCommonComponents", "The selected actors have no common editable components."))
			.TextStyle(FAppStyle::Get(), "HintText")
		],
		FText::GetEmpty(), /*bNoIndent*/ true, /*bSearchable*/ false, false,
		LOCTEXT("NoCommonComponentsTooltip", "There are no editable components of the same type between the currently selected actors.")));
		
		return;
	}

	// Add a menu entry to open each set of components in the property matrix
	for(auto it = CommonComponents.CreateIterator(); it; ++it)
	{
		TArray<UObject*>& Components = it.Value();
		
		FUIAction BulkEditAction(
		FExecuteAction::CreateLambda([Components]()
		{
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
			PropertyEditorModule.CreatePropertyEditorToolkit(TSharedPtr<IToolkitHost>(), Components );
		}));
		
		Section.AddMenuEntry(
			NAME_None,
			FText::FromName(it.Key()->GetFName()),
			FText::GetEmpty(),
			FSlateIcon(),
			BulkEditAction);
	}
}



void FLevelEditorContextMenuImpl::FillAssetToolsMenu(UToolMenu* Menu)
{
	{
		FToolMenuSection& Section = Menu->AddSection("AssetTools");

		// Go to C++ Code
		// Create custom label and tooltip with the header's filename if possible,
		// otherwise the unset TAttribute's will cause the menu item to use the command's label/tooltip.
		TAttribute<FText> GoToCodeForActorLabel;
		TAttribute<FText> GoToCodeForActorToolTip;
		if (SelectionInfo.SelectionClass != NULL)
		{
			if (FSourceCodeNavigation::IsCompilerAvailable())
			{
				FString ClassHeaderPath;
				if (FSourceCodeNavigation::FindClassHeaderPath(SelectionInfo.SelectionClass, ClassHeaderPath) && IFileManager::Get().FileSize(*ClassHeaderPath) != INDEX_NONE)
				{
					const FString CodeFileName = FPaths::GetCleanFilename(*ClassHeaderPath);
					GoToCodeForActorLabel = FText::Format(LOCTEXT("GoToCodeForActor", "Open {0}"), FText::FromString(CodeFileName));
					GoToCodeForActorToolTip = FText::Format(LOCTEXT("GoToCodeForActor_ToolTip", "Opens the header file for this actor ({0}) in a code editing program"), FText::FromString(CodeFileName));
				}
			}
		}

		Section.AddMenuEntry(FLevelEditorCommands::Get().GoToCodeForActor,
			GoToCodeForActorLabel,
			GoToCodeForActorToolTip,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.C++"));
	}

	{

		FToolMenuSection& Section = Menu->AddSection("SourceControl");
		FLevelEditorContextMenuImpl::AddSourceControlMenu(Section);
	}
}

void FLevelEditorContextMenuImpl::FillMergeActorsMenu(UToolMenu* Menu)
{
	FToolMenuSection& Section = Menu->AddSection("OpenPanel");

	IMergeActorsModule& MergeActorsModule = IMergeActorsModule::Get();
	TArray<IMergeActorsTool*> MergeActorTools;
	MergeActorsModule.Get().GetRegisteredMergeActorsTools(MergeActorTools);

	for (IMergeActorsTool* Tool : MergeActorTools)
	{
		FUIAction MergeActorToolAction(
			FExecuteAction::CreateLambda([Tool]() { Tool->RunMergeFromSelection(); }),
			FCanExecuteAction::CreateLambda([Tool]() { return Tool->CanMergeFromSelection(); }));

		Section.AddMenuEntry(
			NAME_None,
			Tool->GetToolNameText(),
			Tool->GetTooltipText(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), Tool->GetIconName()),
			MergeActorToolAction);
	}

	if (MergeActorTools.Num() > 0)
	{
		Section.AddSeparator(NAME_None);
	}

	Section.AddMenuEntry(
		FLevelEditorCommands::Get().OpenMergeActor,
		LOCTEXT("OpenMergeActor", "Merge Actors Settings..."),
		LOCTEXT("OpenMergeActor_ToolTip", "Click to open the Merge Actor panel"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings"));
}

void FLevelEditorContextMenuImpl::AddSourceControlMenu(FToolMenuSection& Section)
{
	Section.AddSubMenu(TEXT("SourceControlSubMenu"), LOCTEXT("SourceControlSubMenu", "Revision Control"),
					   LOCTEXT("SourceControlSubMenu_ToolTip", "Opens the Revision Control sub menu"),
					   FNewToolMenuDelegate::CreateStatic(&FLevelEditorContextMenuImpl::FillSourceControlMenu),
					   false, // default value
					   FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.ConnectToSourceControl"));
}

void FLevelEditorContextMenuImpl::FillSourceControlMenu(UToolMenu* Menu)
{
	FToolMenuSection& Section = Menu->AddSection(TEXT("Revision Control"));

	Section.AddMenuEntry(FLevelEditorCommands::Get().ShowActorHistory);
}

void FLevelEditorContextMenuImpl::AddSelectChildrenEntry(FToolMenuSection& Section, const TArray<AActor*>& SelectedActors, ULevelEditorContextMenuContext* Context)
{
	// Don't show in main Actor menu because it's already available from the Select pulldown menu.
	if (Context->ContextType == ELevelEditorMenuContext::MainMenu)
	{
		return;
	}

	bool bHasAttachedChildren = false;
	for (AActor* Actor : SelectedActors)
	{
		TArray<AActor*> AttachedActors;
		Actor->GetAttachedActors(AttachedActors);
		if (!AttachedActors.IsEmpty())
		{
			bHasAttachedChildren = true;
			break;
		}
	}

	// Only show if there are attached children to prevent crowding the context menu all the time.
	if (bHasAttachedChildren)
	{
		Section.AddMenuEntry(FLevelEditorCommands::Get().SelectImmediateChildren);
	}
}

void FLevelScriptEventMenuHelper::FillLevelBlueprintEventsMenu(FToolMenuSection& Section, const TArray<AActor*>& SelectedActors)
{
	AActor* SelectedActor = (1 == SelectedActors.Num()) ? SelectedActors[0] : NULL;
	TWeakObjectPtr<AActor> ActorPtr(SelectedActor);

	const bool bAnyEventExists = FKismetEditorUtilities::AnyBoundLevelScriptEventForActor(SelectedActor, false);
	const bool bAnyEventCanBeAdded = FKismetEditorUtilities::AnyBoundLevelScriptEventForActor(SelectedActor, true);

	Section.AddSubMenu(
		"AddEventSubMenu",
		LOCTEXT("AddEventSubMenu", "Add Event"),
		FText::GetEmpty(),
		FNewToolMenuDelegate::CreateStatic(&FKismetEditorUtilities::AddLevelScriptEventOptionsForActor,
			ActorPtr, false, true, true),
		FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([bAnyEventCanBeAdded]()
			{
				return bAnyEventCanBeAdded;
			})),
		EUserInterfaceActionType::Button,
		false, // default value
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Event")
	);

	Section.AddSubMenu(
		"JumpEventSubMenu",
		LOCTEXT("JumpEventSubMenu", "Jump to Event"),
		FText::GetEmpty(),
		FNewToolMenuDelegate::CreateStatic(&FKismetEditorUtilities::AddLevelScriptEventOptionsForActor,
			ActorPtr, true, false, true),
		FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([bAnyEventExists]()
			{
				return bAnyEventExists;
			})),
		EUserInterfaceActionType::Button,
		false, // default value
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.JumpToEvent")
				);
}


#undef LOCTEXT_NAMESPACE
