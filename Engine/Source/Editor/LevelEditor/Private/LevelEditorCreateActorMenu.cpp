// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorCreateActorMenu.h"
#include "Engine/Blueprint.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "ToolMenus.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactories/ActorFactoryBoxVolume.h"
#include "ActorFactories/ActorFactoryCameraActor.h"
#include "ActorFactories/ActorFactoryCylinderVolume.h"
#include "ActorFactories/ActorFactoryDirectionalLight.h"
#include "ActorFactories/ActorFactoryPlayerStart.h"
#include "ActorFactories/ActorFactoryPointLight.h"
#include "ActorFactories/ActorFactorySpotLight.h"
#include "ActorFactories/ActorFactoryRectLight.h"
#include "ActorFactories/ActorFactorySphereVolume.h"
#include "ActorFactories/ActorFactoryTriggerBox.h"
#include "ActorFactories/ActorFactoryTriggerCapsule.h"
#include "ActorFactories/ActorFactoryTriggerSphere.h"
#include "GameFramework/Volume.h"
#include "Engine/BlockingVolume.h"
#include "AssetRegistry/AssetData.h"
#include "AssetThumbnail.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"
#include "AssetSelection.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Styling/SlateIconFinder.h"
#include "ClassIconFinder.h"
#include "LevelEditorActions.h"
#include "ActorPlacementInfo.h"
#include "IPlacementModeModule.h"
#include "Engine/TriggerBase.h"
#include "LevelEditorMenuContext.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Engine/Selection.h"
#include "ActorEditorUtils.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Widgets/Layout/SBorder.h"

class SMenuThumbnail : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SMenuThumbnail ) 
		: _Width(32)
		, _Height(32)
	{}
		SLATE_ARGUMENT( uint32, Width )
		SLATE_ARGUMENT( uint32, Height )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const FAssetData& InAsset )
	{
		Asset = InAsset;

		Thumbnail = MakeShareable( new FAssetThumbnail(Asset, InArgs._Width, InArgs._Height, UThumbnailManager::Get().GetSharedThumbnailPool()));

		FAssetThumbnailConfig ThumbnailConfig;
		ThumbnailConfig.ColorStripOrientation = EThumbnailColorStripOrientation::VerticalRightEdge;
		ThumbnailConfig.BorderPadding = FMargin(2.0f); // Prevents overlap with rounded corners; this matches what the Content Browser tiles do

		ChildSlot
		[
			Thumbnail->MakeThumbnailWidget(ThumbnailConfig)
		];
	}

private:

	FAssetData Asset;
	TSharedPtr< FAssetThumbnail > Thumbnail;
};

static void GetMenuEntryText(const FAssetData& Asset, const TArray<FActorFactoryAssetProxy::FMenuItem>& AssetMenuOptions, FText& OutAssetDisplayName, FText& OutActorTypeDisplayName)
{
	const bool IsClass = Asset.GetClass() == UClass::StaticClass();
	const bool IsVolume = IsClass ? Cast<UClass>(Asset.GetAsset())->IsChildOf(AVolume::StaticClass()) : false;

	if (IsClass)
	{
		OutAssetDisplayName = Asset.GetClass()->GetDisplayNameText();
	}
	else
	{
		OutAssetDisplayName = FText::FromName(Asset.AssetName);
	}

	if (AssetMenuOptions.Num() == 1)
	{
		const FActorFactoryAssetProxy::FMenuItem& MenuItem = AssetMenuOptions[0];
		if (IsClass)
		{
			UClass* MenuItemClass = Cast<UClass>(MenuItem.AssetData.GetAsset());
			if (MenuItemClass && MenuItemClass->IsChildOf(AActor::StaticClass()))
			{
				AActor* DefaultActor = Cast<AActor>(Cast<UClass>(MenuItem.AssetData.GetAsset())->GetDefaultObject(false));
				OutActorTypeDisplayName = DefaultActor->GetClass()->GetDisplayNameText();
			}
		}

		// If the class type name wasn't set above, then use the factory's display name
		if (OutActorTypeDisplayName.IsEmpty() && MenuItem.FactoryToUse != nullptr)
		{
			OutActorTypeDisplayName = MenuItem.FactoryToUse->GetDisplayName();
		}

		// For non-volume classes, use the type display name as the primary label (in place of the actor display name)
		if (IsClass && !IsVolume && !OutActorTypeDisplayName.IsEmpty())
		{
			OutAssetDisplayName = OutActorTypeDisplayName;
			OutActorTypeDisplayName = FText::GetEmpty();
		}
	}
}

class SAssetMenuEntry : public SCompoundWidget
{
	public:

	SLATE_BEGIN_ARGS( SAssetMenuEntry ){}
		SLATE_ARGUMENT( FText, LabelOverride )
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The layer the row widget is supposed to represent
	 * @param	InOwnerTableView	The owner of the row widget
	 */
	void Construct( const FArguments& InArgs, const FAssetData& Asset, const TArray< FActorFactoryAssetProxy::FMenuItem >& AssetMenuOptions )
	{
		FText AssetDisplayName;
		FText ActorTypeDisplayName;
		GetMenuEntryText(Asset, AssetMenuOptions, AssetDisplayName, ActorTypeDisplayName);

		if ( !InArgs._LabelOverride.IsEmpty() )
		{
			AssetDisplayName = InArgs._LabelOverride;
		}

		TSharedRef<SWidget> ActorType =
			ActorTypeDisplayName.IsEmpty()
			? SNullWidget::NullWidget
			: SNew(SBox)
				.Padding(FMargin(0.f, 8.f, 0.f, 0.f))
				[
					SNew(STextBlock)
					.Text(ActorTypeDisplayName)
					.TextStyle(FAppStyle::Get(), "LevelViewportContextMenu.ActorType.Text")
					.TransformPolicy(ETextTransformPolicy::ToUpper)
				];

		ChildSlot
		.Padding( FMargin(0.f, 0.f, 8.f, 0.f) )
		[
			SNew( SBorder )
			.Padding( 0.f )
			.BorderImage( FAppStyle::Get().GetBrush("LevelViewportContextMenu.AssetTileItem.NameAreaBackground") )
			[
				SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.Padding( 0.f, 0.f, 0.f, 0.f)
				.VAlign( VAlign_Center )
				.AutoWidth()
				[
					SNew( SBorder )
					//.Padding( FMargin(4.f, 0.f, 0.f, 0.f) ) // prevent contents from overlapping the rounded corners
					.Padding(0.f)
					.BorderImage( FAppStyle::Get().GetBrush("LevelViewportContextMenu.AssetTileItem.ThumbnailAreaBackground") )
					[
						SNew( SBox )
						.WidthOverride( FOptionalSize(48.f) )
						.HeightOverride( FOptionalSize(48.f) )
						[
							SNew( SMenuThumbnail, Asset )
							.Width(48)
							.Height(48)
						]
					]
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(5.f, 0.f, 10.f, 0.f)
				[
					SNew( SVerticalBox )
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.TextStyle( FAppStyle::Get(), "LevelViewportContextMenu.AssetLabel.Text" )
						.Text( AssetDisplayName )
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						ActorType
					]
				]
			]
		];
	}
};

static bool CanReplaceActors()
{
	bool bCanReplace = false;
	FText OutReason;
	for (FSelectionIterator SelectionIter = GEditor->GetSelectedActorIterator(); SelectionIter; ++SelectionIter)
	{
		if (AActor* Actor = Cast<AActor>(*SelectionIter))
		{ 
			bCanReplace = true;
			
			if(!Actor->CanReplaceSelectedActor(OutReason))
			{
				bCanReplace = false;
				break;
			}
		}
	}

	return bCanReplace;
}

/**
 * Helper function to get menu options for selected asset data
 * @param	TargetAssetData		Asset data to be used
 * @param	AssetMenuOptions	Output menu options
 */
static void GetContentBrowserSelectionFactoryMenuEntries( FAssetData& TargetAssetData, TArray< FActorFactoryAssetProxy::FMenuItem >& AssetMenuOptions )
{
	TArray<FAssetData> SelectedAssets;
	AssetSelectionUtils::GetSelectedAssets( SelectedAssets );

	bool bPlaceable = true;

	if ( SelectedAssets.Num() > 0 )
	{
		TargetAssetData = SelectedAssets.Top();
	}

	UClass* AssetClass = TargetAssetData.GetClass();
	if (AssetClass == UClass::StaticClass() )
	{
		UClass* Class = Cast<UClass>( TargetAssetData.GetAsset() );

		bPlaceable = AssetSelectionUtils::IsClassPlaceable( Class );
	}
	else if (AssetClass && AssetClass->IsChildOf<UBlueprint>())
	{
		// For blueprints, attempt to determine placeability from its tag information

		FString TagValue;

		if ( TargetAssetData.GetTagValue( FBlueprintTags::NativeParentClassPath, TagValue ) && !TagValue.IsEmpty() )
		{
			// If the native parent class can't be placed, neither can the blueprint
			UClass* NativeParentClass = UClass::TryFindTypeSlow<UClass>(FPackageName::ExportTextPathToObjectPath(TagValue));

			bPlaceable = AssetSelectionUtils::IsChildBlueprintPlaceable( NativeParentClass );
		}
		
		if ( bPlaceable && TargetAssetData.GetTagValue( FBlueprintTags::ClassFlags, TagValue ) && !TagValue.IsEmpty() )
		{
			// Check to see if this class is placeable from its class flags

			const int32 NotPlaceableFlags = CLASS_NotPlaceable | CLASS_Deprecated | CLASS_Abstract;
			uint32 ClassFlags = FCString::Atoi( *TagValue );

			bPlaceable = ( ClassFlags & NotPlaceableFlags ) == CLASS_None;
		}
	}

	if ( bPlaceable )
	{
		FActorFactoryAssetProxy::GenerateActorFactoryMenuItems( TargetAssetData, &AssetMenuOptions, true );
	}
}


/**
 * Helper function for FillAddReplaceActorMenu & FillAddReplaceContextMenuSections. Builds a menu for an asset & options.
 * @param	MenuBuilder			The menu builder used to generate the context menu
 * @param	Asset				Asset data to use
 * @param	AssetMenuOptions	Menu options to use
 * @param	CreateMode			The creation mode to use
 */
static void FillAssetAddReplaceActorMenu(UToolMenu* Menu, const FAssetData Asset, const TArray< FActorFactoryAssetProxy::FMenuItem > AssetMenuOptions, EActorCreateMode::Type CreateMode)
{
	FToolMenuSection& Section = Menu->AddSection("Section");
	for( int32 ItemIndex = 0; ItemIndex < AssetMenuOptions.Num(); ++ItemIndex )
	{
		const FActorFactoryAssetProxy::FMenuItem& MenuItem = AssetMenuOptions[ItemIndex];
		AActor* DefaultActor = MenuItem.FactoryToUse->GetDefaultActor( MenuItem.AssetData );

		FText Label = MenuItem.FactoryToUse->DisplayName;
		FText ToolTip = MenuItem.FactoryToUse->DisplayName;

		FSlateIcon Icon = FSlateIconFinder::FindIcon(*FString::Printf(TEXT("ClassIcon.%s"), *MenuItem.FactoryToUse->GetClass()->GetName()));
		if ( !Icon.IsSet() )
		{
			Icon = FClassIconFinder::FindSlateIconForActor(DefaultActor);
		}

		FUIAction Action;
		if ( CreateMode == EActorCreateMode::Replace )
		{
			Action = FUIAction( FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ReplaceActors_Clicked, MenuItem.FactoryToUse,  MenuItem.AssetData ) );
		}
		else
		{
			Action = FUIAction( FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::AddActor_Clicked, MenuItem.FactoryToUse,  MenuItem.AssetData) );
		}

		Section.AddMenuEntry(NAME_None, Label, ToolTip, Icon, Action);
	}
}

/**
 * Helper function for FillAddReplaceActorMenu & FillAddReplaceContextMenuSections. Builds a single menu option.
 * @param	MenuBuilder			The menu builder used to generate the context menu
 * @param	Asset				Asset data to use
 * @param	AssetMenuOptions	Menu options to use
 * @param	CreateMode			The creation mode to use
 * @param	LabelOverride		The lable to use, if any.
 */
static void BuildSingleAssetAddReplaceActorMenu(FToolMenuSection& Section, const FAssetData& Asset, const TArray< FActorFactoryAssetProxy::FMenuItem >& AssetMenuOptions, EActorCreateMode::Type CreateMode, const FText& LabelOverride = FText::GetEmpty(), bool bUseAssetTile=false)
{
	if ( !Asset.IsValid() || AssetMenuOptions.Num() == 0 )
	{
		return;
	}

	if ( AssetMenuOptions.Num() == 1 )
	{
		const FActorFactoryAssetProxy::FMenuItem& MenuItem = AssetMenuOptions[0];

		FUIAction Action;
		if ( CreateMode == EActorCreateMode::Replace )
		{
			Action = FUIAction( FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ReplaceActors_Clicked, MenuItem.FactoryToUse,  MenuItem.AssetData ) );
		}
		else
		{
			Action = FUIAction( FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::AddActor_Clicked, MenuItem.FactoryToUse,  MenuItem.AssetData) );
		}

		// For a single option that doesn't open a submenu, we have an option of the custom tile widget (used for recents, selection) and a regular menu entry.
		if (bUseAssetTile)
		{
			FString EntryName = LabelOverride.BuildSourceString();
			EntryName.RemoveSpacesInline();
			FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(*EntryName, Action, SNew(SAssetMenuEntry, Asset, AssetMenuOptions).LabelOverride(LabelOverride));
			Section.AddEntry(Entry);
		}
		else
		{
			FText AssetDisplayName;
			if (LabelOverride.IsEmpty())
			{
				FText UnusedActorTypeDisplayName;
				GetMenuEntryText(Asset, AssetMenuOptions, AssetDisplayName, UnusedActorTypeDisplayName);
			}
			else
			{
				AssetDisplayName = LabelOverride;
			}

			FString EntryName = AssetDisplayName.BuildSourceString();
			EntryName.RemoveSpacesInline();
			FSlateIcon Icon = FSlateIconFinder::FindIconForClass(FClassIconFinder::GetIconClassForAssetData(Asset));
			Section.AddMenuEntry(*EntryName, AssetDisplayName, TAttribute<FText>(), Icon, Action, EUserInterfaceActionType::Button);
		}
	}
	else
	{
		// If this opens a submenu for multiple options, always use a regular menu entry and never the custom tile widget.
		FText AssetDisplayName;
		if (LabelOverride.IsEmpty())
		{
			FText UnusedActorTypeDisplayName;
			GetMenuEntryText(Asset, AssetMenuOptions, AssetDisplayName, UnusedActorTypeDisplayName);
		}
		else
		{
			AssetDisplayName = LabelOverride;
		}

		FString SubMenuName = AssetDisplayName.BuildSourceString();
		SubMenuName.RemoveSpacesInline();
		FSlateIcon Icon = FSlateIconFinder::FindIconForClass(FClassIconFinder::GetIconClassForAssetData(Asset));
		Section.AddSubMenu(
			*SubMenuName, AssetDisplayName, TAttribute<FText>(), FNewToolMenuDelegate::CreateStatic(&FillAssetAddReplaceActorMenu, Asset, AssetMenuOptions, CreateMode),
			FToolUIActionChoice(), EUserInterfaceActionType::Button, /*bInOpenSubMenuOnClick*/ false, Icon);
	}
}

void LevelEditorCreateActorMenu::FillAddReplaceContextMenuSections(FToolMenuSection& Section, ULevelEditorContextMenuContext* LevelEditorMenuContext)
{
	FAssetData TargetAssetData;
	TArray< FActorFactoryAssetProxy::FMenuItem > AssetMenuOptions;
	GetContentBrowserSelectionFactoryMenuEntries( /*OUT*/TargetAssetData, /*OUT*/AssetMenuOptions );

	const bool bCanPlaceActor = true;
	const bool bCanReplaceActors = CanReplaceActors();

	if (bCanPlaceActor)
	{
		Section.AddSubMenu(
			"AddActor",
			NSLOCTEXT("LevelViewportContextMenu", "AddActorHeading", "Place Actor") , 
			NSLOCTEXT("LevelViewportContextMenu", "AddActorMenu_ToolTip", "Templates for adding a new actor to the world"),
			FNewToolMenuDelegate::CreateStatic(&LevelEditorCreateActorMenu::FillAddReplaceActorMenu, EActorCreateMode::Add),
			false, // default value
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.PlaceActors"));
	}

	if (bCanReplaceActors)
	{
		Section.AddSubMenu(
			"ReplaceActor",
			NSLOCTEXT("LevelViewportContextMenu", "ReplaceActorHeading", "Replace Selected Actors with") , 
			NSLOCTEXT("LevelViewportContextMenu", "ReplaceActorMenu_ToolTip", "Templates for replacing selected with new actors in the world"),
			FNewToolMenuDelegate::CreateStatic(&LevelEditorCreateActorMenu::FillAddReplaceActorMenu, EActorCreateMode::Replace),
			false, // default value
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ReplaceActor"));
	}
}

bool GReplaceSelectedActorsWithSelectedClassCopyProperties = false;
void LevelEditorCreateActorMenu::FillAddReplaceActorMenu(UToolMenu* Menu, EActorCreateMode::Type CreateMode)
{
	if ( CreateMode == EActorCreateMode::Replace )
	{
		FToolMenuSection& Section = Menu->AddSection("Options", NSLOCTEXT("LevelViewportContextMenu", "Options", "Options"));

		GReplaceSelectedActorsWithSelectedClassCopyProperties = GetDefault<ULevelEditorMiscSettings>()->bReplaceCopyPropertiesDefault;

		FToolMenuEntry ToolMenuEntry = FToolMenuEntry::InitMenuEntry(
			"CopyProperties",
			NSLOCTEXT("LevelViewportContextMenu", "CopyProperties", "Copy Properties"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]() { GReplaceSelectedActorsWithSelectedClassCopyProperties = !GReplaceSelectedActorsWithSelectedClassCopyProperties; }),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([] { return GReplaceSelectedActorsWithSelectedClassCopyProperties ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
			),
			EUserInterfaceActionType::ToggleButton
		);
		ToolMenuEntry.bShouldCloseWindowAfterMenuSelection = false;

		Section.AddEntry(ToolMenuEntry);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("ContentBrowserActor", NSLOCTEXT("LevelViewportContextMenu", "AssetSelectionSection", "Selection"));
		FAssetData TargetAssetData;
		TArray< FActorFactoryAssetProxy::FMenuItem > AssetMenuOptions;
		GetContentBrowserSelectionFactoryMenuEntries( /*OUT*/TargetAssetData, /*OUT*/AssetMenuOptions );

		BuildSingleAssetAddReplaceActorMenu( Section, TargetAssetData, AssetMenuOptions, CreateMode, FText::GetEmpty(), /*bUseAssetTile*/ true );
	}

	{
		FToolMenuSection& Section = Menu->AddSection("RecentlyPlaced", NSLOCTEXT("LevelViewportContextMenu", "RecentlyPlacedSection", "Recently Placed"));
		if ( IPlacementModeModule::IsAvailable() )
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			const TArray< FActorPlacementInfo > RecentlyPlaced = IPlacementModeModule::Get().GetRecentlyPlaced();
			for (int Index = 0; Index < RecentlyPlaced.Num() && Index < 3; Index++)
			{
				FAssetData Asset = AssetRegistryModule.Get().GetAssetByObjectPath( FSoftObjectPath(RecentlyPlaced[Index].ObjectPath) );

				if ( Asset.IsValid() )
				{
					TArray< FActorFactoryAssetProxy::FMenuItem > AssetMenuOptions;
					UActorFactory* Factory = FindObject<UActorFactory>( NULL, *RecentlyPlaced[Index].Factory );

					if ( Factory != NULL )
					{
						AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, Asset ) );
					}
					else
					{
						FActorFactoryAssetProxy::GenerateActorFactoryMenuItems( Asset, &AssetMenuOptions, true );
						while( AssetMenuOptions.Num() > 1 )
						{
							AssetMenuOptions.Pop();
						}
					}

					BuildSingleAssetAddReplaceActorMenu(Section, Asset, AssetMenuOptions, CreateMode, FText::GetEmpty(), /*bUseAssetTile*/ true );
				}
			}
		}
	}

	{
		FToolMenuSection& Section = Menu->AddSection("Lights", NSLOCTEXT("LevelViewportContextMenu", "LightsSection", "Lights"));
		TArray< FActorFactoryAssetProxy::FMenuItem > AssetMenuOptions;

		{
			AssetMenuOptions.Empty();
			UActorFactory* Factory = GEditor->FindActorFactoryByClass( UActorFactoryDirectionalLight::StaticClass() );
			FAssetData AssetData = FAssetData( Factory->GetDefaultActorClass( FAssetData() ) );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );
			BuildSingleAssetAddReplaceActorMenu(Section, AssetData, AssetMenuOptions, CreateMode);
		}

		{
			AssetMenuOptions.Empty();
			UActorFactory* Factory = GEditor->FindActorFactoryByClass( UActorFactorySpotLight::StaticClass() );
			FAssetData AssetData = FAssetData( Factory->GetDefaultActorClass( FAssetData() ) );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );
			BuildSingleAssetAddReplaceActorMenu(Section, AssetData, AssetMenuOptions, CreateMode);
		}

		{
			AssetMenuOptions.Empty();
			UActorFactory* Factory = GEditor->FindActorFactoryByClass( UActorFactoryPointLight::StaticClass() );
			FAssetData AssetData = FAssetData( Factory->GetDefaultActorClass( FAssetData() ) );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );
			BuildSingleAssetAddReplaceActorMenu(Section, AssetData, AssetMenuOptions, CreateMode);
		}

		{
			AssetMenuOptions.Empty();
			UActorFactory* Factory = GEditor->FindActorFactoryByClass( UActorFactoryRectLight::StaticClass() );
			FAssetData AssetData = FAssetData( Factory->GetDefaultActorClass( FAssetData() ) );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );
			BuildSingleAssetAddReplaceActorMenu(Section, AssetData, AssetMenuOptions, CreateMode);
		}
	}

	{
		FToolMenuSection& Section = Menu->AddSection("Primitives", NSLOCTEXT("LevelViewportContextMenu", "PrimitivesSection", "Primitives"));
		TArray< FActorFactoryAssetProxy::FMenuItem > AssetMenuOptions;
		{
			AssetMenuOptions.Empty();
			UActorFactory* Factory = GEditor->FindActorFactoryByClass( UActorFactoryCameraActor::StaticClass() );
			FAssetData AssetData = FAssetData( Factory->GetDefaultActorClass( FAssetData() ) );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );
			BuildSingleAssetAddReplaceActorMenu(Section, AssetData, AssetMenuOptions, CreateMode);
		}

		{
			AssetMenuOptions.Empty();
			UActorFactory* Factory = GEditor->FindActorFactoryByClass( UActorFactoryPlayerStart::StaticClass() );
			FAssetData AssetData = FAssetData( Factory->GetDefaultActorClass( FAssetData() ) );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );
			BuildSingleAssetAddReplaceActorMenu(Section, AssetData, AssetMenuOptions, CreateMode);
		}

		{
			const UClass* BlockingVolumeClass = ABlockingVolume::StaticClass();
			FAssetData AssetData = FAssetData( BlockingVolumeClass );

			AssetMenuOptions.Empty();
			UActorFactory* Factory = GEditor->FindActorFactoryByClassForActorClass( UActorFactorySphereVolume::StaticClass(), BlockingVolumeClass );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );

			Factory = GEditor->FindActorFactoryByClassForActorClass( UActorFactoryBoxVolume::StaticClass(), BlockingVolumeClass );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );

			Factory = GEditor->FindActorFactoryByClassForActorClass( UActorFactoryCylinderVolume::StaticClass(), BlockingVolumeClass );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );
			BuildSingleAssetAddReplaceActorMenu(Section, AssetData, AssetMenuOptions, CreateMode);
		}

		{
			AssetMenuOptions.Empty();
			UActorFactory* Factory = GEditor->FindActorFactoryByClass( UActorFactoryTriggerBox::StaticClass() );
			FAssetData AssetData = FAssetData( Factory->GetDefaultActorClass( FAssetData() ) );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );

			Factory = GEditor->FindActorFactoryByClass( UActorFactoryTriggerSphere::StaticClass() );
			AssetData = FAssetData( Factory->GetDefaultActorClass( FAssetData() ) );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );

			Factory = GEditor->FindActorFactoryByClass( UActorFactoryTriggerCapsule::StaticClass() );
			AssetData = FAssetData( Factory->GetDefaultActorClass( FAssetData() ) );
			AssetMenuOptions.Add( FActorFactoryAssetProxy::FMenuItem( Factory, AssetData ) );

			BuildSingleAssetAddReplaceActorMenu(Section, FAssetData(ATriggerBase::StaticClass()), AssetMenuOptions, CreateMode, NSLOCTEXT("LevelViewportContextMenu", "TriggersGroup", "Trigger"));
		}
	}

	{
		FToolMenuSection& Section = Menu->AddSection("Custom", NSLOCTEXT("LevelViewportContextMenu", "CustomSection", "Custom Actors"));
		TArray< FActorFactoryAssetProxy::FMenuItem > AssetMenuOptions;
		FText UnusedErrorMessage;
		const FAssetData NoAssetData {};
		for ( int32 FactoryIdx = 0; FactoryIdx < GEditor->ActorFactories.Num(); FactoryIdx++ )
		{
			AssetMenuOptions.Empty();
			UActorFactory* Factory = GEditor->ActorFactories[FactoryIdx];

			if (Factory->bShowInEditorQuickMenu)
			{
				FAssetData AssetData(Factory->GetDefaultActorClass(FAssetData()));
				const bool bFactoryWorksWithoutAsset = Factory->CanCreateActorFrom(NoAssetData, UnusedErrorMessage);
				const bool bFactoryWorksWithAsset = AssetData.IsValid() && Factory->CanCreateActorFrom(AssetData, UnusedErrorMessage);
				const bool bFactoryWorks = bFactoryWorksWithAsset || bFactoryWorksWithoutAsset;

				if (bFactoryWorks)
				{
					AssetMenuOptions.Add(FActorFactoryAssetProxy::FMenuItem(Factory, bFactoryWorksWithAsset ? AssetData : NoAssetData));
					BuildSingleAssetAddReplaceActorMenu(Section, AssetData, AssetMenuOptions, CreateMode);
				}
			}
		}
	}
}
