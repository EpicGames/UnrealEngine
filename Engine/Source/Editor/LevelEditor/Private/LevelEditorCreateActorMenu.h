// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FToolMenuSection;
class UToolMenu;
class ULevelEditorContextMenuContext;

/**
 * The mode to use when creating an actor
 */
namespace EActorCreateMode
{
	enum Type
	{
		/** Add the actor at the last click location */
		Add,

		/** Replace the actor that was last clicked on */
		Replace,

	};
}

namespace LevelEditorCreateActorMenu
{
	/**
	 * Fill the context menu section(s) for adding or replacing an actor in the viewport
	 * @param	Section		The tool menu section to which entries are added
	 */
	void FillAddReplaceContextMenuSections(FToolMenuSection& Section, ULevelEditorContextMenuContext* LevelEditorMenuContext);

	/**
	 * Fill the context menu for adding or replacing an actor. Used for in-viewport and level editor toolbar menus.
	 * @param	MenuBuilder		The menu builder used to generate the context menu
	 */
	void FillAddReplaceActorMenu(UToolMenu* Menu, EActorCreateMode::Type CreateMode);
};
