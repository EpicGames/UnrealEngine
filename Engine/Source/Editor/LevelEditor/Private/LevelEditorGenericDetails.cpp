// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorGenericDetails.h"
#include "Modules/ModuleManager.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Engine/Level.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "LevelEditor.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "LevelEditorActions.h"
#include "AssetSelection.h"
#include "SSurfaceProperties.h"
#include "SurfaceIterators.h"
#include "ScopedTransaction.h"
#include "FunctionalTest.h"
#include "MaterialList.h"

#define LOCTEXT_NAMESPACE "FLevelEditorGenericDetails"

/** Makes a new instance of this detail layout class for a specific detail view requesting it */
TSharedRef<IDetailCustomization> FLevelEditorGenericDetails::MakeInstance()
{
	return MakeShareable( new FLevelEditorGenericDetails );
}

void FLevelEditorGenericDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	FSelectedActorInfo SelectionInfo = AssetSelectionUtils::GetSelectedActorInfo();	
	if( AssetSelectionUtils::IsAnySurfaceSelected( SelectionInfo.SharedWorld ) )
	{
		AddSurfaceDetails( DetailLayout );
	}

#if 0
	TArray< TWeakObjectPtr<UObject> > CustomizedObjects;
	DetailLayout.GetObjectsBeingCustomized(CustomizedObjects);

	TArray< AFunctionalTest* > FunctionalTests;
	for ( auto& CustomizedObject : CustomizedObjects )
	{
		AFunctionalTest* FunctionalTest = Cast<AFunctionalTest>(CustomizedObject.Get());
		if ( FunctionalTest )
		{
			FunctionalTests.Add(FunctionalTest);
		}
	}
	
	if ( FunctionalTests.Num() > 0 )
	{
		FString TestNames;

		for ( int32 TestIndex = 0; TestIndex < FunctionalTests.Num(); TestIndex++ )
		{
			AFunctionalTest* FunctionalTest = FunctionalTests[TestIndex];

			FString TestName = FunctionalTest->GetName();
			FString MapName = FunctionalTest->GetLevel()->GetWorld()->GetMapName();

			// Project.Maps.Client Functional Testing
			TestNames += TEXT("Project.Maps.Functional Tests.");
			TestNames += MapName;
			TestNames += TEXT(".");
			TestNames += TestName;

			// If we're not at the end, add the test name delimiter.
			if ( TestIndex < ( FunctionalTests.Num() - 1 ) )
			{
				TestNames += TEXT("+");
			}
		}

		IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Run", LOCTEXT("RunFunctionalTestingTitle", "Run"));
		Category.AddCustomRow(FText::GetEmpty())
		[
			SNew(SButton)
			.OnClicked_Lambda([TestNames] {
				FString Command = TEXT("Automation RunTests Now ");
				Command += TestNames;

				GEditor->Exec(GWorld, *Command);

				return FReply::Handled();
			})
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Run Test", "Run Selected Tests"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}
#endif
}

void FLevelEditorGenericDetails::GetSelectedSurfaceMaterials(IMaterialListBuilder& MaterialList) const
{
	FSelectedActorInfo SelectionInfo = AssetSelectionUtils::GetSelectedActorInfo();	

	for (TSelectedSurfaceIterator<> SurfaceIt(SelectionInfo.SharedWorld); SurfaceIt; ++SurfaceIt)
	{
        FBspSurf* SelectedSurface = *SurfaceIt;

		const bool bCanBeReplaced = true;
		MaterialList.AddMaterial( 0, SelectedSurface->Material, bCanBeReplaced );
	}
}

void FLevelEditorGenericDetails::OnMaterialChanged( UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 SlotIndex, bool bReplaceAll )
{
	bool bModelDirtied = false;
	{
		FSelectedActorInfo SelectionInfo = AssetSelectionUtils::GetSelectedActorInfo();	

		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "PolySetMaterial", "Set Material") );
		GEditor->FlagModifyAllSelectedSurfacesInLevels( SelectionInfo.SharedWorld );

		for ( TSelectedSurfaceIterator<> It(SelectionInfo.SharedWorld) ; It ; ++It )
		{
			UModel* Model = It.GetModel();
			const int32 SurfaceIndex = It.GetSurfaceIndex();

			Model->Surfs[SurfaceIndex].Material = NewMaterial;
			const bool bUpdateTexCoords = false;
			const bool bOnlyUpdateSurfaceMaterials = true;
			GEditor->polyUpdateBrush(Model, SurfaceIndex, bUpdateTexCoords, bOnlyUpdateSurfaceMaterials);
			Model->MarkPackageDirty();

			bModelDirtied = true;
		}
	}
	GEditor->RedrawLevelEditingViewports();
	if ( bModelDirtied )
	{
		ULevel::LevelDirtiedEvent.Broadcast();
	}
}

void FLevelEditorGenericDetails::AddSurfaceDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	TSharedPtr< const FUICommandList > CommandBindings = LevelEditorModule.GetGlobalLevelEditorActions();

	/** Level editor commands for use with the selection detail view */
	const FLevelEditorCommands& Commands = FLevelEditorCommands::Get();

	FMenuBuilder SelectionBuilder( true, CommandBindings );
	{
		SelectionBuilder.BeginSection("LevelEditorGenericDetailsSurface");
		{
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectAllMatchingBrush );
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectAllMatchingTexture );
		}
		SelectionBuilder.EndSection();

		SelectionBuilder.BeginSection("LevelEditorGenericDetailsSurface2");
		{
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectAllAdjacents );
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectAllAdjacentCoplanars );
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectAllAdjacentWalls );
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectAllAdjacentSlants );
		}
		SelectionBuilder.EndSection();

		SelectionBuilder.BeginSection("LevelEditorGenericDetailsSurface3");
		{
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectReverse );
		}
		SelectionBuilder.EndSection();

		SelectionBuilder.BeginSection("LevelEditorGenericDetailsSurface4");
		{
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectMemorize );
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectRecall );
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectOr );
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectAnd );
			SelectionBuilder.AddMenuEntry( Commands.SurfSelectXor );
		}
		SelectionBuilder.EndSection();
	}

	FMenuBuilder AlignmentBuilder( true, CommandBindings );
	{
		AlignmentBuilder.AddMenuEntry( Commands.SurfUnalign );
		AlignmentBuilder.AddMenuEntry( Commands.SurfAlignPlanarAuto );
		AlignmentBuilder.AddMenuEntry( Commands.SurfAlignPlanarWall );
		AlignmentBuilder.AddMenuEntry( Commands.SurfAlignPlanarFloor );
		AlignmentBuilder.AddMenuEntry( Commands.SurfAlignBox );
		AlignmentBuilder.AddMenuEntry( Commands.SurfAlignFit );
	}


	struct Local
	{
		static FReply ExecuteExecCommand(FString InCommand)
		{
			FSelectedActorInfo SelectionInfo = AssetSelectionUtils::GetSelectedActorInfo();	

			GUnrealEd->Exec( SelectionInfo.SharedWorld, *InCommand );
			return FReply::Handled();
		}
	};

	const FSlateFontInfo& FontInfo = IDetailLayoutBuilder::GetDetailFont();

	IDetailCategoryBuilder& MaterialsCategory = DetailBuilder.EditCategory( "Surface Materials", LOCTEXT("BSPSurfaceMaterials", "Surface Materials") );
	{
		FMaterialListDelegates MaterialListDelegates;
		MaterialListDelegates.OnGetMaterials.BindSP(this, &FLevelEditorGenericDetails::GetSelectedSurfaceMaterials);
		MaterialListDelegates.OnMaterialChanged.BindSP(this, &FLevelEditorGenericDetails::OnMaterialChanged);

		//Pass an empty material list owner (owner can be use by the asset picker filter. In this case we do not need it)
		TArray<FAssetData> MaterialListOwner;

		TSharedRef<FMaterialList> MaterialList = MakeShareable(new FMaterialList(DetailBuilder, MaterialListDelegates, MaterialListOwner));

		MaterialsCategory.AddCustomBuilder( MaterialList );
	}

	// Add a new section for static meshes
	IDetailCategoryBuilder& BSPCategory = DetailBuilder.EditCategory( "Geometry", LOCTEXT("BSPSurfacesTitle", "Geometry") );
	BSPCategory.AddCustomRow( FText::GetEmpty() )
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 3.0f, 1.0f )
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 2.0f )
			[
				SNew( SComboButton )
				.ButtonContent()
				[
					SNew( STextBlock )
					.Text( LOCTEXT("SelectSurfacesMenu", "Select") ) 
					.Font( FontInfo )
				]
				.MenuContent()
				[
					SelectionBuilder.MakeWidget()
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 2.0f )
			[
				SNew( SComboButton )
				.ButtonContent()
				[
					SNew( STextBlock )
					.Text( LOCTEXT("AlignSurfaceTexMenu", "Alignment") ) 
					.Font( FontInfo )
				]
				.MenuContent()
				[
					AlignmentBuilder.MakeWidget()
				]
			]
		]
	];

	BSPCategory.AddCustomRow( LOCTEXT("CleanBSPMaterials", "Clean Geometry Materials"), true )
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 2.0f )
		[
			SNew( SButton )
			.ToolTipText( LOCTEXT("CleanBSPMaterials_Tooltip", "Cleans Geometry Materials") )
			.OnClicked( FOnClicked::CreateStatic( &Local::ExecuteExecCommand, FString( TEXT("CLEANBSPMATERIALS") ) ) )
			.HAlign( HAlign_Center )
			[
				SNew( STextBlock )
				.Text( LOCTEXT("CleanBSPMaterials", "Clean Geometry Materials") )
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		]
	];

	IDetailCategoryBuilder& SurfaceCategory = DetailBuilder.EditCategory( "Surface Properties", LOCTEXT("BSPSurfaceProperties", "Surface Properties") );
	SurfaceCategory.AddCustomRow( LOCTEXT("BSPSurfaceProperties", "Surface Properties") )
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 3.0f, 1.0f )
		[
			SNew(SSurfaceProperties)
		]
	];
}

#undef LOCTEXT_NAMESPACE
