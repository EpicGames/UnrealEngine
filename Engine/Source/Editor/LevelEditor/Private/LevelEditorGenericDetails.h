// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

class IMaterialListBuilder;
class UMaterialInterface;

/**
 * Creates details for the level editor details view that are not specific to any selected actor type
 */
class FLevelEditorGenericDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( class IDetailLayoutBuilder& DetailLayout ) override;

private:
	/**
	 * Adds a section for selected surface details
	 */
	void AddSurfaceDetails( class IDetailLayoutBuilder& DetailLayout );

	/**
	 * Populate the specified material list with the materials used on the currently selected BSP surfaces
	 */
	void GetSelectedSurfaceMaterials(IMaterialListBuilder& MaterialList) const;

	/**
	 * Called when the material should be changed on all selected BSP surfaces
	 */
	void OnMaterialChanged( UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 SlotIndex, bool bReplaceAll );
};

