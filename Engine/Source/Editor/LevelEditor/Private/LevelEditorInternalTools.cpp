// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorInternalTools.h"

#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementDetailsInterface.h"

namespace InternalEditorLevelLibrary
{
	TSharedPtr<SLevelViewport> GetLevelViewport(const FName& ViewportConfigKey)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

		TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
		if (LevelEditor.IsValid())
		{
			if (ViewportConfigKey != NAME_None)
			{
				for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
				{
					if (LevelViewport.IsValid() && LevelViewport->GetConfigKey() == ViewportConfigKey)
					{
						return LevelViewport;
					}
				}
			}

			TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditor->GetActiveViewportInterface();
			return ActiveLevelViewport;
		}

		return nullptr;
	}

	TSharedPtr<SLevelViewport> GetActiveLevelViewport()
	{
		return GetLevelViewport(NAME_None);
	}

	bool IsEditingLevelInstanceCurrentLevel(UWorld* InWorld)
	{
		if (InWorld)
		{
			if (ULevelInstanceSubsystem* LevelInstanceSubsystem = InWorld->GetSubsystem<ULevelInstanceSubsystem>())
			{
				if (ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetEditingLevelInstance())
				{
					return LevelInstanceSubsystem->GetLevelInstanceLevel(LevelInstance) == InWorld->GetCurrentLevel();
				}
			}
		}

		return false;
	}

	AActor* GetEditingLevelInstance(UWorld* InWorld)
	{
		ULevelInstanceSubsystem* LevelInstanceSubsystem = InWorld ? InWorld->GetSubsystem<ULevelInstanceSubsystem>() : nullptr;
		return LevelInstanceSubsystem ? Cast<AActor>(LevelInstanceSubsystem->GetEditingLevelInstance()) : nullptr;
	}

	bool IsActorEditorContextVisible(UWorld* InWorld)
	{
		return InWorld && (InWorld->GetCurrentLevel()->OwningWorld->GetLevels().Num() > 1 && (!InWorld->IsPartitionedWorld() || GetEditingLevelInstance(InWorld) != nullptr));
	}

	bool DoesDetailSelectionContainNonActorsOrComponents(const UTypedElementSelectionSet* SelectionSet)
	{
		if (!ensure(SelectionSet))
		{
			return false;
		}
	
		bool bFoundNonActorOrComponent = false;
		SelectionSet->ForEachSelectedElement<ITypedElementDetailsInterface>([&bFoundNonActorOrComponent](const TTypedElement<ITypedElementDetailsInterface>& InDetailsElement)
		{
			if (TUniquePtr<ITypedElementDetailsObject> ElementDetailsObject = InDetailsElement.GetDetailsObject())
			{
				UObject* Object = ElementDetailsObject->GetObject();
				if (!Object || (!Object->IsA<AActor>() && !Object->IsA<UActorComponent>()))
				{
					bFoundNonActorOrComponent = true;
					return false;
				}
			}
			return true;
		});

		return bFoundNonActorOrComponent;
	}
}
