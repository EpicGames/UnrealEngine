// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "SLevelViewport.h"

#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"

class UTypedElementSelectionSet;

namespace InternalEditorLevelLibrary
{
	TSharedPtr<SLevelViewport> GetLevelViewport(const FName& ViewportConfigKey);
	TSharedPtr<SLevelViewport> GetActiveLevelViewport();
	bool IsEditingLevelInstanceCurrentLevel(UWorld* InWorld);
	AActor* GetEditingLevelInstance(UWorld* InWorld);
	bool IsActorEditorContextVisible(UWorld* InWorld);
	/**
	 * When converted to objects that would be given to the detail panel, does the selection contain anything that
	 *  is not an actor or component?
	 */
	bool DoesDetailSelectionContainNonActorsOrComponents(const UTypedElementSelectionSet* SelectionSet);
}
