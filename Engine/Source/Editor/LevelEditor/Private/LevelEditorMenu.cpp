// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorMenu.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/VersePath.h"
#include "Engine/World.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "LevelEditorContextMenu.h"
#include "Interfaces/IMainFrameModule.h"
#include "MRUFavoritesList.h"
#include "Framework/Commands/GenericCommands.h"
#include "IAssetTools.h"
#include "IDocumentation.h"
#include "ToolMenus.h"
#include "LevelEditorMenuContext.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Settings/EditorExperimentalSettings.h"
#include "ITranslationEditor.h"
#include "ILocalizationDashboardModule.h"
#include "AssetSelection.h"
#include "EditorBuildUtils.h"
#include "EditorViewportCommands.h"
#include "LevelViewportActions.h"
#include "Styling/SlateIconFinder.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Interfaces/ITurnkeySupportModule.h"

#define LOCTEXT_NAMESPACE "LevelEditorMenu"

void FLevelEditorMenu::RegisterLevelEditorMenus()
{
	struct Local
	{
		static void RegisterFileLoadAndSaveItems()
		{
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.File");

			FToolMenuSection& OpenSection = Menu->FindOrAddSection("FileOpen");

			FToolMenuInsert InsertPos(NAME_None, EToolMenuInsertType::First);

			// New Level
			OpenSection.AddMenuEntry( FLevelEditorCommands::Get().NewLevel ).InsertPosition = InsertPos;

			// Open Level
			OpenSection.AddMenuEntry( FGlobalEditorCommonCommands::Get().OpenLevel ).InsertPosition = InsertPos;

			FToolMenuSection& AssetSection = Menu->FindOrAddSection("FileAsset");
			
			AssetSection.AddSeparator("FileAssetSeparator").InsertPosition = FToolMenuInsert(NAME_None, EToolMenuInsertType::First);

			// Open Asset
			//@TODO: Doesn't work when summoned from here: Section.AddMenuEntry( FGlobalEditorCommonCommands::Get().SummonOpenAssetDialog );

			FToolMenuSection& SaveSection = Menu->FindOrAddSection("FileSave");

			// Save
			SaveSection.AddMenuEntry( FLevelEditorCommands::Get().Save ).InsertPosition = InsertPos;
	
			// Save As
			SaveSection.AddMenuEntry( FLevelEditorCommands::Get().SaveAs ).InsertPosition = InsertPos;
		}

		static FText GetLevelPath(const FString& PackageName)
		{
			if (FAssetToolsModule::GetModule().Get().ShowingContentVersePath())
			{
				if (const IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
				{
					// Minic logic in UEditorEngine::Map_Load which finds the first UWorld in the package.
					TArray<FAssetData> AssetDatas;
					if (AssetRegistry->GetAssetsByPackageName(FName(PackageName), AssetDatas))
					{
						for (const FAssetData& AssetData : AssetDatas)
						{
							if (AssetData.AssetClassPath == UWorld::StaticClass()->GetClassPathName())
							{
								UE::Core::FVersePath VersePath = AssetData.GetVersePath();
								if (VersePath.IsValid())
								{
									return FText::FromString(MoveTemp(VersePath).ToString());
								}
								break;
							}
						}
					}
				}
			}

			return FText::FromString(PackageName);
		}

		static void FillFileRecentAndFavoriteFileItems()
		{
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.File");
			FToolMenuInsert SectionInsertPos("FileSave", EToolMenuInsertType::After);

			// Import/Export
			{
				FToolMenuSection& Section = Menu->AddSection("FileActors", LOCTEXT("ImportExportHeading", "Import/Export"), SectionInsertPos);
				{
					// Import Into Level
					Section.AddMenuEntry(FLevelEditorCommands::Get().ImportScene);

					// Export All
					Section.AddMenuEntry( FLevelEditorCommands::Get().ExportAll );

					// Export Selected
					Section.AddMenuEntry( FLevelEditorCommands::Get().ExportSelected );
				}
			}


			// Favorite Menus
			{
				struct FFavoriteLevelMenu
				{
					// Add a button to add/remove the currently loaded map as a favorite
					struct Local
					{
						static FText GetToggleFavoriteLabelText()
						{
							const FText LevelName = FText::FromString(FPackageName::GetShortName(GWorld->GetOutermost()->GetFName()));
							if (!FLevelEditorActionCallbacks::ToggleFavorite_IsChecked())
							{
								return FText::Format(LOCTEXT("ToggleFavorite_Add", "Add {0} to Favorites"), LevelName);
							}
							return FText::Format(LOCTEXT("ToggleFavorite_Remove", "Remove {0} from Favorites"), LevelName);
						}
					};

					static void MakeFavoriteLevelMenu(UToolMenu* InMenu)
					{
						// Add a button to add/remove the currently loaded map as a favorite
						if (FLevelEditorActionCallbacks::ToggleFavorite_CanExecute())
						{
							FToolMenuSection& Section = InMenu->AddSection("LevelEditorToggleFavorite");
							{
								TAttribute<FText> ToggleFavoriteLabel;
								ToggleFavoriteLabel.BindStatic(&Local::GetToggleFavoriteLabelText);
								Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleFavorite, ToggleFavoriteLabel);
							}
							Section.AddSeparator("LevelEditorToggleFavorite");
						}
						const FMainMRUFavoritesList& MRUFavorites = *FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame").GetMRUFavoritesList();
						const int32 NumFavorites = MRUFavorites.GetNumFavorites();
						
						const bool bNoIndent = false;
						const int32 AllowedFavorites = FMath::Min(NumFavorites, FLevelEditorCommands::Get().OpenFavoriteFileCommands.Num());
						for (int32 CurFavoriteIndex = 0; CurFavoriteIndex < AllowedFavorites; ++CurFavoriteIndex)
						{
							TSharedPtr< FUICommandInfo > OpenFavoriteFile = FLevelEditorCommands::Get().OpenFavoriteFileCommands[CurFavoriteIndex];
							const FString CurFavorite = MRUFavorites.GetFavoritesItem(CurFavoriteIndex);
							const FText ToolTip = FText::Format(LOCTEXT("FavoriteLevelToolTip", "Opens favorite level: {0}"), GetLevelPath(CurFavorite));
							const FText Label = FText::FromString(FPackageName::GetShortName(CurFavorite));

							InMenu->FindOrAddSection("Favorite").AddMenuEntry(OpenFavoriteFile, Label, ToolTip).Name = NAME_None;
						}
					}
				};

				FToolMenuSection& Section = Menu->FindOrAddSection("FileOpen");

				Section.AddDynamicEntry("FileFavoriteLevels", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
					const FMainMRUFavoritesList& RecentsAndFavorites = *MainFrameModule.GetMRUFavoritesList();

					// Only show the Favorite Levels menu if either 1) the current level could be favorited (it's saved)
					// or 2) there are 1 or more favorite levels.
					if (FLevelEditorActionCallbacks::ToggleFavorite_CanExecute()
						|| RecentsAndFavorites.GetNumFavorites() > 0)
					{
						InSection.AddSubMenu(
							"FavoriteLevelsSubMenu",
							LOCTEXT("FavoriteLevelsSubMenu", "Favorite Levels"),
							LOCTEXT("RecentLevelsSubMenu_ToolTip", "Select a level to load"),
							FNewToolMenuDelegate::CreateStatic(&FFavoriteLevelMenu::MakeFavoriteLevelMenu),
							false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.FavoriteLevels")
						);
					}
				}));
			}

			// Recent files
			{
				struct FRecentLevelMenu
				{
					static void MakeRecentLevelMenu(UToolMenu* InMenu)
					{
						const FMainMRUFavoritesList& MRUFavorites = *FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame").GetMRUFavoritesList();
						const int32 NumRecents = MRUFavorites.GetNumItems();

						FToolMenuSection& Section = InMenu->FindOrAddSection("Recent");

						const int32 AllowedRecents = FMath::Min(NumRecents, FLevelEditorCommands::Get().OpenRecentFileCommands.Num());
						for (int32 CurRecentIndex = 0; CurRecentIndex < AllowedRecents; ++CurRecentIndex)
						{
							TSharedPtr< FUICommandInfo > OpenRecentFile = FLevelEditorCommands::Get().OpenRecentFileCommands[CurRecentIndex];

							if (!MRUFavorites.MRUItemPassesCurrentFilter(CurRecentIndex))
							{
								continue;
							}

							const FString& CurRecent = MRUFavorites.GetMRUItem(CurRecentIndex);

							const FText ToolTip = FText::Format(LOCTEXT("RecentLevelToolTip", "Opens recent level: {0}"), GetLevelPath(CurRecent));
							const FText Label = FText::FromString(FPackageName::GetShortName(CurRecent));

							Section.AddMenuEntry(OpenRecentFile, Label, ToolTip).Name = NAME_None;
						}

						Section.AddSeparator("AfterRecentLevels");

						Section.AddMenuEntry("ClearRecentLevels", FLevelEditorCommands::Get().ClearRecentFiles);
					}
				};

				FToolMenuSection& Section = Menu->FindOrAddSection("FileOpen");
				Section.AddDynamicEntry("FileRecentLevels", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
					const FMainMRUFavoritesList& RecentsAndFavorites = *MainFrameModule.GetMRUFavoritesList();
					if (RecentsAndFavorites.GetNumItems() > 0)
					{
						InSection.AddSubMenu(
							"RecentLevelsSubMenu",
							LOCTEXT("RecentLevelsSubMenu", "Recent Levels"),
							LOCTEXT("RecentLevelsSubMenu_ToolTip", "Select a level to load"),
							FNewToolMenuDelegate::CreateStatic(&FRecentLevelMenu::MakeRecentLevelMenu),
							false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.RecentLevels")
						);
					}
				}));
			}
		}

		static void ExtendEditMenu()
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("LevelEditor.MainMenu.Edit", "MainFrame.MainMenu.Edit", EMultiBoxType::Menu, /*bWarnIfAlreadyRegistered*/false);
			{
				// Edit Actor
				{
					FToolMenuSection& Section = Menu->AddSection("EditMain", LOCTEXT("MainHeading", "Edit"), FToolMenuInsert("EditHistory", EToolMenuInsertType::After));

					Section.AddMenuEntry(FGenericCommands::Get().Cut);
					Section.AddMenuEntry(FGenericCommands::Get().Copy);
					Section.AddMenuEntry(FGenericCommands::Get().Paste);

					Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
					Section.AddMenuEntry(FGenericCommands::Get().Delete);
				}
			}

		}

		static void ExtendHelpMenu()
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("LevelEditor.MainMenu.Help", "MainFrame.MainMenu.Help", EMultiBoxType::Menu, /*bWarnIfAlreadyRegistered*/false);
			FToolMenuSection& Section = Menu->AddSection("HelpResources", NSLOCTEXT("MainHelpMenu", "LevelEditorHelpResources", "Level Editor Resources"), FToolMenuInsert("Learn", EToolMenuInsertType::First));
			{
				Section.AddMenuEntry( FLevelEditorCommands::Get().BrowseDocumentation );

				Section.AddMenuEntry( FLevelEditorCommands::Get().BrowseViewportControls );
			}
		}

		static void ExtendWindowMenu()
		{
			if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Window"))
			{
				FToolMenuSection& LayoutSection = Menu->FindOrAddSection("WindowLayout");

				// This entry needs to be placed after the fullscreen separator in the Window menu.
				// Making sure the separator exist - the one in Window menu might be missing if not on a Win build.
				// See FMainMenu::RegisterWindowMenu().
				LayoutSection.AddSeparator("FullscreenSeparator");
				
				FToolMenuInsert Insert("ToggleFullscreen", EToolMenuInsertType::Before);
				Insert.Fallback = EToolMenuInsertFallback::Insert;

				FToolMenuEntry& ToggleViewportToolbar = LayoutSection.AddMenuEntry(FLevelViewportCommands::Get().ToggleViewportToolbar);
				ToggleViewportToolbar.InsertPosition = Insert;
				
				FToolMenuEntry& ToggleUIEntry = LayoutSection.AddMenuEntry(FLevelEditorCommands::Get().ToggleHideViewportUI);
				ToggleUIEntry.InsertPosition = Insert;
			}
		}

		static void ExtendMenuBar()
		{
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu");

			FToolMenuSection& Section = Menu->FindOrAddSection(NAME_None);


			FToolMenuEntry& BuildEntry =
				Section.AddSubMenu(
					"Build",
					LOCTEXT("BuildMenu", "Build"),
					LOCTEXT("BuildMenu_ToolTip", "Level Build Options"),
					FNewToolMenuChoice()
				);

			BuildEntry.InsertPosition = FToolMenuInsert("Help", EToolMenuInsertType::Before);

			FToolMenuEntry& SelectEntry =
				Section.AddSubMenu(
					"Select",
					LOCTEXT("SelectMenu", "Select"),
					LOCTEXT("SelectMenu_ToolTip", "Level Selection"),
					FNewToolMenuChoice()
				);

			SelectEntry.InsertPosition = FToolMenuInsert("Help", EToolMenuInsertType::Before);

			const FName LevelEditorName("LevelEditor");
			FToolMenuEntry& ActionsEntry =
				Section.AddSubMenu(
					"Actions",
					TAttribute<FText>::CreateLambda([LevelEditorName]()
					{
						FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(LevelEditorName);
						TSharedPtr<ILevelEditor> LevelEditorInstancePtr = LevelEditorModule.GetLevelEditorInstance().Pin();
						return LevelEditorInstancePtr ? LevelEditorInstancePtr->GetLevelViewportContextMenuTitle() : FText::GetEmpty();
					}),
					TAttribute<FText>::CreateLambda([LevelEditorName]()
					{
						FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(LevelEditorName);
						TSharedPtr<ILevelEditor> LevelEditorInstancePtr = LevelEditorModule.GetLevelEditorInstance().Pin();
						return LevelEditorInstancePtr ? FLevelEditorContextMenu::GetContextMenuToolTip(ELevelEditorMenuContext::MainMenu, LevelEditorInstancePtr->GetElementSelectionSet()) : FText::GetEmpty();
					}),
					FOnGetContent::CreateLambda([LevelEditorName]()
					{
						// Generate the context menu completely separate from the main menu hierarchy for consistency with the right-click context menu.
						// This means that extenders/UToolMenu extensions registered for the viewport context menu apply here (since they'll take effect when generating the menu widget below),
						// and NOT any extenders registered for the main menu bar.
						// I have verified that this works properly with the global Mac menu bar.
						FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(LevelEditorName);
						TSharedPtr<ILevelEditor> LevelEditorInstancePtr = LevelEditorModule.GetLevelEditorInstance().Pin();
						return LevelEditorInstancePtr ? FLevelEditorContextMenu::BuildMenuWidget(LevelEditorInstancePtr, ELevelEditorMenuContext::MainMenu, nullptr, FTypedElementHandle()) : SNullWidget::NullWidget;
					}));
			
			ActionsEntry.InsertPosition = FToolMenuInsert("Help", EToolMenuInsertType::Before);

#if UE_WITH_TURNKEY_SUPPORT
			ITurnkeySupportModule::Get().MakeTurnkeyMenu(Section, FToolMenuInsert("Select", EToolMenuInsertType::Before));
#endif
		}

	};

	UToolMenus* ToolMenus = UToolMenus::Get();
	const bool bWarnIfAlreadyRegistered = false;
	ToolMenus->RegisterMenu("LevelEditor.MainMenu", "MainFrame.MainMenu", EMultiBoxType::MenuBar, bWarnIfAlreadyRegistered);
	ToolMenus->RegisterMenu("LevelEditor.MainMenu.File", "MainFrame.MainTabMenu.File", EMultiBoxType::Menu, bWarnIfAlreadyRegistered);
	ToolMenus->RegisterMenu("LevelEditor.MainMenu.Window", "MainFrame.MainMenu.Window", EMultiBoxType::Menu, bWarnIfAlreadyRegistered);
	ToolMenus->RegisterMenu("LevelEditor.MainMenu.Tools", "MainFrame.MainMenu.Tools", EMultiBoxType::Menu, bWarnIfAlreadyRegistered);

	// Add other top level menus
	Local::ExtendMenuBar();

	Local::RegisterFileLoadAndSaveItems();
	Local::FillFileRecentAndFavoriteFileItems();
	Local::ExtendEditMenu();
	Local::ExtendHelpMenu();
	Local::ExtendWindowMenu();

	RegisterBuildMenu();
	RegisterSelectMenu();
}

TSharedRef< SWidget > FLevelEditorMenu::MakeLevelEditorMenu( const TSharedPtr<FUICommandList>& CommandList, TSharedPtr<class SLevelEditor> LevelEditor )
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FExtender> Extenders = LevelEditorModule.GetMenuExtensibilityManager()->GetAllExtenders();
	FToolMenuContext ToolMenuContext(CommandList, Extenders.ToSharedRef());

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
	TSharedRef< SWidget > MenuBarWidget = MainFrameModule.MakeMainMenu( LevelEditor->GetTabManager(), "LevelEditor.MainMenu", ToolMenuContext );

	return MenuBarWidget;
}

void FLevelEditorMenu::RegisterBuildMenu()
{
	static const FName BaseMenuName = "LevelEditor.MainMenu.Build";
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(BaseMenuName, NAME_None, EMultiBoxType::Menu, /*bWarnIfAlreadyRegistered*/false);

	struct FLightingMenus
	{
	public:

		static void RegisterMenus(const FName InBaseMenuName)
		{
			FLightingMenus::RegisterLightingQualityMenu(InBaseMenuName);
			FLightingMenus::RegisterLightingInfoMenu(InBaseMenuName);
		}

	private:

		/** Generates a lighting quality sub-menu */
		static void RegisterLightingQualityMenu(const FName InBaseMenuName)
		{
			UToolMenu* SubMenu = UToolMenus::Get()->RegisterMenu(UToolMenus::JoinMenuPaths(InBaseMenuName, "LightingQuality"), NAME_None, EMultiBoxType::Menu, /*bWarnIfAlreadyRegistered*/false);

			{
				FToolMenuSection& Section = SubMenu->AddSection("LevelEditorBuildLightingQuality", LOCTEXT("LightingQualityHeading", "Quality Level"));
				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingQuality_Production);
				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingQuality_High);
				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingQuality_Medium);
				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingQuality_Preview);
			}
		}

		/** Generates a lighting density sub-menu */
		static void RegisterLightingDensityMenu(const FName InBaseMenuName)
		{
			UToolMenu* SubMenu = UToolMenus::Get()->RegisterMenu(UToolMenus::JoinMenuPaths(InBaseMenuName, "LightingDensity"), NAME_None, EMultiBoxType::Menu, /*bWarnIfAlreadyRegistered*/false);

			{
				FToolMenuSection& Section = SubMenu->AddSection("LevelEditorBuildLightingDensity", LOCTEXT("LightingDensityHeading", "Density Rendering"));
				TSharedRef<SWidget> Ideal = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FMargin(27.0f, 0.0f, 0.0f, 0.0f))
					.FillWidth(1.0f)
					[
						SNew(SSpinBox<float>)
						.MinValue(0.f)
						.MaxValue(100.f)
						.Value(FLevelEditorActionCallbacks::GetLightingDensityIdeal())
						.OnValueChanged_Static(&FLevelEditorActionCallbacks::SetLightingDensityIdeal)
					];

				Section.AddEntry(FToolMenuEntry::InitWidget("Ideal", Ideal, LOCTEXT("LightingDensity_Ideal", "Ideal Density")));

				TSharedRef<SWidget> Maximum = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SSpinBox<float>)
						.MinValue(0.01f)
						.MaxValue(100.01f)
						.Value(FLevelEditorActionCallbacks::GetLightingDensityMaximum())
						.OnValueChanged_Static(&FLevelEditorActionCallbacks::SetLightingDensityMaximum)
					];

				Section.AddEntry(FToolMenuEntry::InitWidget("Maximum", Maximum, LOCTEXT("LightingDensity_Maximum", "Maximum Density")));

				TSharedRef<SWidget> ClrScale = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FMargin(35.0f, 0.0f, 0.0f, 0.0f))
					.FillWidth(1.0f)
					[
						SNew(SSpinBox<float>)
						.MinValue(0.f)
						.MaxValue(10.f)
						.Value(FLevelEditorActionCallbacks::GetLightingDensityColorScale())
						.OnValueChanged_Static(&FLevelEditorActionCallbacks::SetLightingDensityColorScale)
					];

				Section.AddEntry(FToolMenuEntry::InitWidget("ColorScale", ClrScale, LOCTEXT("LightingDensity_ColorScale", "Color Scale")));

				TSharedRef<SWidget> GrayScale = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FMargin(11.0f, 0.0f, 0.0f, 0.0f))
					.FillWidth(1.0f)
					[
						SNew(SSpinBox<float>)
						.MinValue(0.f)
						.MaxValue(10.f)
						.Value(FLevelEditorActionCallbacks::GetLightingDensityGrayscaleScale())
						.OnValueChanged_Static(&FLevelEditorActionCallbacks::SetLightingDensityGrayscaleScale)
					];

				Section.AddEntry(FToolMenuEntry::InitWidget("GrayscaleScale", GrayScale, LOCTEXT("LightingDensity_GrayscaleScale", "Grayscale Scale")));

				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingDensity_RenderGrayscale);
			}
		}

		/** Generates a lighting resolution sub-menu */
		static void RegisterLightingResolutionMenu(const FName InBaseMenuName)
		{
			UToolMenu* SubMenu = UToolMenus::Get()->RegisterMenu(UToolMenus::JoinMenuPaths(InBaseMenuName, "LightingResolution"), NAME_None, EMultiBoxType::Menu, /*bWarnIfAlreadyRegistered*/false);

			{
				FToolMenuSection& Section = SubMenu->AddSection("LevelEditorBuildLightingResolution1", LOCTEXT("LightingResolutionHeading1", "Primitive Types"));
				TSharedRef<SWidget> Meshes = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SCheckBox)
						.Style(FAppStyle::Get(), "Menu.CheckBox")
						.ToolTipText(LOCTEXT("StaticMeshesToolTip", "Static Meshes will be adjusted if checked."))
						.IsChecked_Static(&FLevelEditorActionCallbacks::IsLightingResolutionStaticMeshesChecked)
						.OnCheckStateChanged_Static(&FLevelEditorActionCallbacks::SetLightingResolutionStaticMeshes)
						.Content()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("StaticMeshes", "Static Meshes"))
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(4.0f, 0.0f, 11.0f, 0.0f))
					[
						SNew(SSpinBox<float>)
						.MinValue(4.f)
						.MaxValue(4096.f)
						.ToolTipText(LOCTEXT("LightingResolutionStaticMeshesMinToolTip", "The minimum lightmap resolution for static mesh adjustments. Anything outside of Min/Max range will not be touched when adjusting."))
						.Value(FLevelEditorActionCallbacks::GetLightingResolutionMinSMs())
						.OnValueChanged_Static(&FLevelEditorActionCallbacks::SetLightingResolutionMinSMs)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpinBox<float>)
						.MinValue(4.f)
					.MaxValue(4096.f)
					.ToolTipText(LOCTEXT("LightingResolutionStaticMeshesMaxToolTip", "The maximum lightmap resolution for static mesh adjustments. Anything outside of Min/Max range will not be touched when adjusting."))
					.Value(FLevelEditorActionCallbacks::GetLightingResolutionMaxSMs())
					.OnValueChanged_Static(&FLevelEditorActionCallbacks::SetLightingResolutionMaxSMs)
					];
				Section.AddEntry(FToolMenuEntry::InitWidget("Meshes", Meshes, FText::GetEmpty(), true));

				TSharedRef<SWidget> BSPs = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "Menu.CheckBox")
					.ToolTipText(LOCTEXT("BSPSurfacesToolTip", "BSP Surfaces will be adjusted if checked."))
					.IsChecked_Static(&FLevelEditorActionCallbacks::IsLightingResolutionBSPSurfacesChecked)
					.OnCheckStateChanged_Static(&FLevelEditorActionCallbacks::SetLightingResolutionBSPSurfaces)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("BSPSurfaces", "BSP Surfaces"))
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(6.0f, 0.0f, 4.0f, 0.0f))
				[
					SNew(SSpinBox<float>)
					.MinValue(1.f)
					.MaxValue(63556.f)
					.ToolTipText(LOCTEXT("LightingResolutionBSPsMinToolTip", "The minimum lightmap resolution of a BSP surface to adjust. When outside of the Min/Max range, the BSP surface will no be altered."))
					.Value(FLevelEditorActionCallbacks::GetLightingResolutionMinBSPs())
					.OnValueChanged_Static(&FLevelEditorActionCallbacks::SetLightingResolutionMinBSPs)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpinBox<float>)
					.MinValue(1.f)
					.MaxValue(63556.f)
					.ToolTipText(LOCTEXT("LightingResolutionBSPsMaxToolTip", "The maximum lightmap resolution of a BSP surface to adjust. When outside of the Min/Max range, the BSP surface will no be altered."))
					.Value(FLevelEditorActionCallbacks::GetLightingResolutionMaxBSPs())
					.OnValueChanged_Static(&FLevelEditorActionCallbacks::SetLightingResolutionMaxBSPs)
				];
				Section.AddEntry(FToolMenuEntry::InitWidget("BSPs", BSPs, FText::GetEmpty(), true));
			}

			{
				FToolMenuSection& Section = SubMenu->AddSection("LevelEditorBuildLightingResolution2", LOCTEXT("LightingResolutionHeading2", "Select Options"));
				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingResolution_CurrentLevel);
				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingResolution_SelectedLevels);
				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingResolution_AllLoadedLevels);
				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingResolution_SelectedObjectsOnly);
			}

			{
				FToolMenuSection& Section = SubMenu->AddSection("LevelEditorBuildLightingResolution3", LOCTEXT("LightingResolutionHeading3", "Ratio"));
				TSharedRef<SWidget> Ratio = SNew(SSpinBox<int32>)
					.MinValue(0)
					.MaxValue(400)
					.ToolTipText(LOCTEXT("LightingResolutionRatioToolTip", "Ratio to apply (New Resolution = Ratio / 100.0f * CurrentResolution)."))
					.Value(FLevelEditorActionCallbacks::GetLightingResolutionRatio())
					.OnEndSliderMovement_Static(&FLevelEditorActionCallbacks::SetLightingResolutionRatio)
					.OnValueCommitted_Static(&FLevelEditorActionCallbacks::SetLightingResolutionRatioCommit);
				Section.AddEntry(FToolMenuEntry::InitWidget("Ratio", Ratio, LOCTEXT("LightingResolutionRatio", "Ratio")));
			}
		}

		/** Generates a lighting info dialogs sub-menu */
		static void RegisterLightingInfoMenu(const FName InBaseMenuName)
		{
			RegisterLightingDensityMenu(UToolMenus::JoinMenuPaths(InBaseMenuName, "LightingInfo"));
			RegisterLightingResolutionMenu(UToolMenus::JoinMenuPaths(InBaseMenuName, "LightingInfo"));

			UToolMenu* SubMenu = UToolMenus::Get()->RegisterMenu(UToolMenus::JoinMenuPaths(InBaseMenuName, "LightingInfo"), NAME_None, EMultiBoxType::Menu, /*bWarnIfAlreadyRegistered*/false);

			{
				FToolMenuSection& Section = SubMenu->AddSection("LevelEditorBuildLightingInfo", LOCTEXT("LightingInfoHeading", "Lighting Info Dialogs"));
				Section.AddSubMenu(
					"LightingDensity",
					LOCTEXT("LightingDensityRenderingSubMenu", "LightMap Density Rendering Options"),
					LOCTEXT("LightingDensityRenderingSubMenu_ToolTip", "Shows the LightMap Density Rendering viewmode options."),
					FNewToolMenuChoice());

				Section.AddSubMenu(
					"LightingResolution",
					LOCTEXT("LightingResolutionAdjustmentSubMenu", "LightMap Resolution Adjustment"),
					LOCTEXT("LightingResolutionAdjustmentSubMenu_ToolTip", "Shows the LightMap Resolution Adjustment options."),
					FNewToolMenuChoice());

				Section.AddMenuEntry(FLevelEditorCommands::Get().LightingStaticMeshInfo, LOCTEXT("BuildLightingInfo_LightingStaticMeshInfo", "Lighting StaticMesh Info..."));
			}
		}
	};

	{
		FToolMenuSection& Section = Menu->AddSection("Level", LOCTEXT("LevelHeading", "Level"));

		Section.AddMenuEntry(FLevelEditorCommands::Get().Build, LOCTEXT("Build", "Build All Levels"));
	}

	FLightingMenus::RegisterMenus(BaseMenuName);

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorLighting", LOCTEXT("LightingHeading", "Lighting"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().BuildLightingOnly, LOCTEXT("BuildLightingOnlyHeading", "Build Lighting Only"));

		Section.AddSubMenu(
			"LightingQuality",
			LOCTEXT("LightingQualitySubMenu", "Lighting Quality"),
			LOCTEXT("LightingQualitySubMenu_ToolTip", "Allows you to select the quality level for precomputed lighting"),
			FNewToolMenuChoice());

		Section.AddSubMenu(
			"LightingInfo",
			LOCTEXT("BuildLightingInfoSubMenu", "Lighting Info"),
			LOCTEXT("BuildLightingInfoSubMenu_ToolTip", "Access the lighting info dialogs"),
			FNewToolMenuChoice());

		Section.AddMenuEntry(FLevelEditorCommands::Get().LightingBuildOptions_UseErrorColoring);
		Section.AddMenuEntry(FLevelEditorCommands::Get().LightingBuildOptions_ShowLightingStats);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorReflections", LOCTEXT("ReflectionHeading", "Reflections"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().BuildReflectionCapturesOnly);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorVisibility", LOCTEXT("VisibilityHeading", "Visibility"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().BuildLightingOnly_VisibilityOnly);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorGeometry", LOCTEXT("GeometryHeading", "Geometry"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().BuildGeometryOnly);
		Section.AddMenuEntry(FLevelEditorCommands::Get().BuildGeometryOnly_OnlyCurrentLevel);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorNavigation", LOCTEXT("NavigationHeading", "Navigation"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().BuildPathsOnly);
	}

	// Add section for external build types
	{
		Menu->AddDynamicSection("ExternalBuildTypes", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			TArray<FName> BuildTypeNames;
			FEditorBuildUtils::GetBuildTypes(BuildTypeNames);
			const int32 NumTypes = BuildTypeNames.Num();
			const int32 AllowedTypes = FMath::Min(NumTypes, FLevelEditorCommands::Get().ExternalBuildTypeCommands.Num());
			
			if (AllowedTypes > 0)
			{
				TArray<FText> BuildTypeLocalizedNames;
				TArray<FText> BuildTypeLocalizedSubmenuNames;
				FEditorBuildUtils::GetBuildTypesLocalizedLabels(BuildTypeLocalizedNames, BuildTypeLocalizedSubmenuNames);
				check(BuildTypeNames.Num() == BuildTypeLocalizedNames.Num());
				check(BuildTypeNames.Num() == BuildTypeLocalizedSubmenuNames.Num());

				static FName ExternalBuildTypeSectionName("LevelEditorExternalBuildTypes");
				FToolMenuSection* BuildTypeSection = nullptr;
				FToolMenuSection* ExternalBuildTypeSection = nullptr;
				
				for (int32 Index = 0; Index < AllowedTypes; ++Index)
				{
					const FText& LocalizedSectionName = BuildTypeLocalizedSubmenuNames[Index];
					// Create dedicated section if provided
					if (!LocalizedSectionName.IsEmpty())
					{
						const FName SectionName(LocalizedSectionName.ToString()); 
						BuildTypeSection = InMenu->FindSection(SectionName);
						if (BuildTypeSection == nullptr)
						{
							BuildTypeSection = &InMenu->AddSection(SectionName, LocalizedSectionName);	
						}
					}
					// Otherwise create generic section for all external types if not already created
					else if (ExternalBuildTypeSection == nullptr)
					{
						ExternalBuildTypeSection = InMenu->FindSection(ExternalBuildTypeSectionName);
						if (ExternalBuildTypeSection == nullptr)
						{
							ExternalBuildTypeSection = &InMenu->AddSection(ExternalBuildTypeSectionName, LOCTEXT("ExternalBuildTypesHeading", "External Types")); 
						}
						BuildTypeSection = ExternalBuildTypeSection;
					}
					// Reused existing generic section
					else
					{
						BuildTypeSection = ExternalBuildTypeSection;
					}
					
					const TSharedPtr<FUICommandInfo>& CommandInfo = FLevelEditorCommands::Get().ExternalBuildTypeCommands[Index];

					// Create tooltip from localized entry label if provided, use generic version otherwise
					const FText ToolTip = BuildTypeLocalizedNames[Index].IsEmpty()
						? FText::Format(LOCTEXT("ExternalBuildTypeToolTip", "Builds external type: {0}"), FText::FromName(BuildTypeNames[Index]))
						: BuildTypeLocalizedNames[Index];

					// Create label from localized entry label if provided, use generic version otherwise
					const FText Label = BuildTypeLocalizedNames[Index].IsEmpty()
						? FText::Format(LOCTEXT("ExternalBuildTypeLabel", "Build {0}"), FText::FromName(BuildTypeNames[Index]))
						: BuildTypeLocalizedNames[Index];

					check(BuildTypeSection != nullptr);
					BuildTypeSection->AddMenuEntry(CommandInfo, Label, ToolTip).Name = BuildTypeNames[Index];
				}
			}
		}));
	}

	{
		Menu->AddDynamicSection("WorldPartitionBuildTypes", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if (GWorld != nullptr && GWorld->GetWorldPartition() != nullptr)
			{
				FToolMenuSection& Section = InMenu->AddSection("LevelEditorWorldPartition", LOCTEXT("WorldPartitionHeading", "World Partition"));
				Section.AddMenuEntry(FLevelEditorCommands::Get().BuildHLODs);
				Section.AddMenuEntry(FLevelEditorCommands::Get().BuildHLODs_SelectedActors);
				Section.AddMenuEntry(FLevelEditorCommands::Get().BuildMinimap);
				Section.AddMenuEntry(FLevelEditorCommands::Get().BuildLandscapeSplineMeshes);
			}
		}));
	}

	{
		// The day we only support World Partitioned worlds, we can remove this section.
		Menu->AddDynamicSection("NonWorldPartitionBuildTypes", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if (GWorld != nullptr && GWorld->GetWorldPartition() == nullptr)
			{
				FToolMenuSection& Section = InMenu->AddSection("LevelEditorLOD", LOCTEXT("LODHeading", "Hierarchical LOD"));
				Section.AddMenuEntry(FLevelEditorCommands::Get().BuildHLODs);
			}
		}));
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorTextureStreaming", LOCTEXT("TextureStreamingHeading", "Texture Streaming"));
		Section.AddDynamicEntry("BuildTextureStreamingOnly", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (CVarStreamingUseNewMetrics.GetValueOnAnyThread() != 0) // There is no point of in building texture streaming data with the old system.
				{
					InSection.AddMenuEntry(FLevelEditorCommands::Get().BuildTextureStreamingOnly);
				}
			}));
		Section.AddMenuEntry(FLevelEditorCommands::Get().BuildVirtualTextureOnly);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorLandscape", LOCTEXT("LandscapeHeading", "Landscape"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().BuildAllLandscape);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorAutomation", LOCTEXT("AutomationHeading", "Automation"));
		Section.AddMenuEntry(
			FLevelEditorCommands::Get().BuildAndSubmitToSourceControl,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.BuildAndSubmit")
		);
	}

	// Map Check
	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorVerification", LOCTEXT("VerificationHeading", "Verification"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().MapCheck, LOCTEXT("OpenMapCheck", "Map Check"));
	}

}

void FLevelEditorMenu::RegisterSelectMenu()
{
	static const FName BaseMenuName = "LevelEditor.MainMenu.Select";
	UToolMenu* Menu =
		UToolMenus::Get()->RegisterMenu(BaseMenuName, NAME_None, EMultiBoxType::Menu, /*bWarnIfAlreadyRegistered*/ false);

	// Main section
	{
		FToolMenuSection& UnnamedSection = Menu->FindOrAddSection(NAME_None);

		UnnamedSection.AddMenuEntry(FGenericCommands::Get().SelectAll);
		UnnamedSection.AddMenuEntry(FLevelEditorCommands::Get().SelectNone);
		UnnamedSection.AddMenuEntry(FLevelEditorCommands::Get().InvertSelection);

		// Hierarchy based selection
		{
			UnnamedSection.AddSubMenu(
				"Hierarchy",
				LOCTEXT("HierarchyLabel", "Hierarchy"),
				LOCTEXT("HierarchyTooltip", "Hierarchy selection tools"),
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* HierarchyMenu)
					{
						FToolMenuSection& HierarchySection = HierarchyMenu->FindOrAddSection(
							"SelectAllHierarchy", LOCTEXT("SelectAllHierarchyLabel", "Hierarchy")
						);

						HierarchySection.AddMenuEntry(FLevelEditorCommands::Get().SelectImmediateChildren);
						HierarchySection.AddMenuEntry(FLevelEditorCommands::Get().SelectAllDescendants);
					}
				),
				false,
				FSlateIconFinder::FindIcon("BTEditor.SwitchToBehaviorTreeMode")
			);
		}

		UnnamedSection.AddSeparator("Advanced");

		UnnamedSection.AddMenuEntry(
			FLevelEditorCommands::Get().SelectAllActorsOfSameClass,
			LOCTEXT("AdvancedSelectAllActorsOfSameClassLabel", "All of Same Class"),
			FLevelEditorCommands::Get().SelectAllActorsOfSameClass->GetDescription(),
			FSlateIconFinder::FindIcon("PlacementBrowser.Icons.All")
		);
	}

	// By Type section
	{
		FToolMenuSection& ByTypeSection =
			Menu->FindOrAddSection("ByTypeSection", LOCTEXT("ByTypeSectionLabel", "By Type"));

		ByTypeSection.AddSubMenu(
			"BSP",
			LOCTEXT("BspLabel", "BSP"),
			LOCTEXT("BspTooltip", "BSP-related tools"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* BspMenu)
				{
					FToolMenuSection& SelectAllSection =
						BspMenu->FindOrAddSection("SelectAllBSP", LOCTEXT("SelectAllBSPLabel", "Select All BSP"));

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectAllAddditiveBrushes,
						LOCTEXT("BSPSelectAllAdditiveBrushesLabel", "Addditive Brushes")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectAllSubtractiveBrushes,
						LOCTEXT("BSPSelectAllSubtractiveBrushesLabel", "Subtractive Brushes")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectAllSurfaces, LOCTEXT("BSPSelectAllAllSurfacesLabel", "Surfaces")
					);
				}
			),
			false,
			FSlateIconFinder::FindIcon("ShowFlagsMenu.BSP")
		);

		ByTypeSection.AddSubMenu(
			"Emitters",
			LOCTEXT("EmittersLabel", "Emitters"),
			LOCTEXT("EmittersTooltip", "Emitters-related tools"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* EmittersMenu)
				{
					FToolMenuSection& SelectAllSection = EmittersMenu->FindOrAddSection(
						"SelectAllEmitters", LOCTEXT("SelectAllEmittersLabel", "Select All Emitters")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectMatchingEmitter,
						LOCTEXT("EmittersSelectMatchingEmitterLabel", "Matching Emitters")
					);
				}
			),
			false,
			FSlateIconFinder::FindIcon("ClassIcon.Emitter")
		);

		ByTypeSection.AddSubMenu(
			"GeometryCollections",
			LOCTEXT("GeometryCollectionsLabel", "Geometry Collections"),
			LOCTEXT("GeometryCollectionsTooltip", "GeometryCollections-related tools"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* GeometryCollectionsMenu)
				{
					// This one will be filled by extensions from GeometryCollectionEditorPlugin
					// Hook is "SelectGeometryCollections"
					FToolMenuSection& SelectAllSection = GeometryCollectionsMenu->FindOrAddSection(
						"SelectGeometryCollections", LOCTEXT("SelectGeometryCollectionsLabel", "Geometry Collections")
					);
				}
			),
			false,
			FSlateIconFinder::FindIcon("ClassIcon.GeometryCollection")
		);

		ByTypeSection.AddSubMenu(
			"HLOD",
			LOCTEXT("HLODLabel", "HLOD"),
			LOCTEXT("HLODTooltip", "HLOD-related tools"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* HLODMenu)
				{
					FToolMenuSection& SelectAllSection =
						HLODMenu->FindOrAddSection("SelectAllHLOD", LOCTEXT("SelectAllHLODLabel", "Select All HLOD"));

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectOwningHierarchicalLODCluster,
						LOCTEXT("HLODSelectOwningHierarchicalLODClusterLabel", "Owning HLOD Cluster")
					);
				}
			),
			false,
			FSlateIconFinder::FindIcon("WorldPartition.ShowHLODActors")
		);

		ByTypeSection.AddSubMenu(
			"Lights",
			LOCTEXT("LightsLabel", "Lights"),
			LOCTEXT("LightsTooltip", "Lights-related tools"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* LightsMenu)
				{
					FToolMenuSection& SelectAllSection = LightsMenu->FindOrAddSection(
						"SelectAllLights", LOCTEXT("SelectAllLightsLabel", "Select All Lights")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectAllLights, LOCTEXT("LightsSelectAllLightsLabel", "All Lights")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectRelevantLights,
						LOCTEXT("LightsSelectRelevantLightsLabel", "Relevant Lights")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectStationaryLightsExceedingOverlap,
						LOCTEXT("LightsSelectStationaryLightsExceedingOverlapLabel", "Stationary Lights Exceeding Overlap")
					);
				}
			),
			false,
			FSlateIconFinder::FindIcon("PlacementBrowser.Icons.Lights")
		);

		ByTypeSection.AddSubMenu(
			"Material",
			LOCTEXT("MaterialLabel", "Material"),
			LOCTEXT("MaterialTooltip", "Material-related tools"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* MaterialMenu)
				{
					FToolMenuSection& SelectAllSection = MaterialMenu->FindOrAddSection(
						"SelectAllMaterial", LOCTEXT("SelectAllMaterialLabel", "Select All Material")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectAllWithSameMaterial,
						LOCTEXT("MaterialSelectAllWithSameMaterialLabel", "With Same Material")
					);
				}
			),
			false,
			FSlateIconFinder::FindIcon("ClassIcon.Material")
		);

		ByTypeSection.AddSubMenu(
			"SkeletalMeshes",
			LOCTEXT("SkeletalMeshesLabel", "Skeletal Meshes"),
			LOCTEXT("SkeletalMeshesTooltip", "SkeletalMeshes-related tools"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* SkeletalMeshesMenu)
				{
					FToolMenuSection& SelectAllSection = SkeletalMeshesMenu->FindOrAddSection(
						"SelectAllSkeletalMeshes", LOCTEXT("SelectAllSkeletalMeshesLabel", "Select All SkeletalMeshes")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectSkeletalMeshesOfSameClass,
						LOCTEXT("SkeletalMeshesSelectSkeletalMeshesOfSameClassLabel", "Using Selected Skeletal Meshes (Selected Actor Types)")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectSkeletalMeshesAllClasses,
						LOCTEXT("SkeletalMeshesSelectSkeletalMeshesAllClassesLabel", "Using Selected Skeletal Meshes (All Actor Types)")
					);
				}
			),
			false,
			FSlateIconFinder::FindIcon("SkeletonTree.Bone")
		);

		ByTypeSection.AddSubMenu(
			"StaticMeshes",
			LOCTEXT("StaticMeshesLabel", "Static Meshes"),
			LOCTEXT("StaticMeshesTooltip", "StaticMeshes-related tools"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* StaticMeshesMenu)
				{
					FToolMenuSection& SelectAllSection = StaticMeshesMenu->FindOrAddSection(
						"SelectAllStaticMeshes", LOCTEXT("SelectAllStaticMeshesLabel", "Select All StaticMeshes")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectStaticMeshesOfSameClass,
						LOCTEXT("StaticMeshesSelectStaticMeshesOfSameClassLabel", "Matching Selected Class")
					);

					SelectAllSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectStaticMeshesAllClasses,
						LOCTEXT("StaticMeshesSelectStaticMeshesAllClassesLabel", "Matching All Classes")
					);
				}
			),
			false,
			FSlateIconFinder::FindIcon("ShowFlagsMenu.StaticMeshes")
		);
	}
}

#undef LOCTEXT_NAMESPACE
