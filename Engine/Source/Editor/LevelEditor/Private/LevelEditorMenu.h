// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once


#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "SLevelEditor.h"

/**
 * Level editor menu
 */
class FLevelEditorMenu
{

public:

	static void RegisterLevelEditorMenus();

	/**
	 * Static: Creates a widget for the level editor's menu
	 *
	 * @return	New widget
	 */
	static TSharedRef< SWidget > MakeLevelEditorMenu( const TSharedPtr<FUICommandList>& CommandList, TSharedPtr<class SLevelEditor> LevelEditor );
private:
	static void RegisterBuildMenu();
	static void RegisterSelectMenu();
};
