// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorMenuContext.h"

#include "Elements/Framework/TypedElementRegistry.h"
#include "SLevelEditor.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelEditorMenuContext)

TWeakPtr<ILevelEditor> ULevelEditorMenuContext::GetLevelEditor() const
{
	return LevelEditor;
}

FScriptTypedElementHandle ULevelEditorContextMenuContext::GetScriptHitProxyElement()
{
	return UTypedElementRegistry::GetInstance()->CreateScriptHandle(HitProxyElement.GetId());
}

TSharedPtr<SLevelViewport> ULevelViewportContext::GetLevelViewport(const UToolMenu* Menu)
{
	if (Menu)
	{
		return GetLevelViewport(Menu->Context);
	}
	return nullptr;
}

TSharedPtr<SLevelViewport> ULevelViewportContext::GetLevelViewport(const FToolMenuSection& Section)
{
	return GetLevelViewport(Section.Context);
}

TSharedPtr<SLevelViewport> ULevelViewportContext::GetLevelViewport(const FToolMenuContext& Context)
{
	if (const ULevelViewportContext* ViewportContext = Context.FindContext<ULevelViewportContext>())
	{
		return ViewportContext->LevelViewport.Pin();
	}
	return nullptr;
}

FLevelEditorViewportClient* ULevelViewportContext::GetLevelViewportClient() const
{
	if (TSharedPtr<SLevelViewport> Viewport = LevelViewport.Pin())
	{
		return &Viewport->GetLevelViewportClient();
	}
	return nullptr;
}
