// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorModesActions.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

DEFINE_LOG_CATEGORY_STATIC(LevelEditorModesActions, Log, All);

#define LOCTEXT_NAMESPACE "LevelEditorModesActions"

/** UI_COMMAND takes long for the compile to optimize */
UE_DISABLE_OPTIMIZATION_SHIP
void FLevelEditorModesCommands::RegisterCommands()
{
	EditorModeCommands.Empty();

	int editorMode = 0;
	static const TArray<FKey, TInlineAllocator<9>> EdModeKeys = { EKeys::One, EKeys::Two, EKeys::Three, EKeys::Four, EKeys::Five, EKeys::Six, EKeys::Seven, EKeys::Eight, EKeys::Nine };

	for ( const FEditorModeInfo& Mode : GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetEditorModeInfoOrderedByPriority())
	{
		FName EditorModeCommandName = FName(*(FString("EditorMode.") + Mode.ID.ToString()));

		TSharedPtr<FUICommandInfo> EditorModeCommand = 
			FInputBindingManager::Get().FindCommandInContext(GetContextName(), EditorModeCommandName);

		// If a command isn't yet registered for this mode, we need to register one.
		if (!EditorModeCommand.IsValid() && Mode.IsVisible())
		{
			FFormatNamedArguments Args;
			FText ModeName = Mode.Name;
			if (ModeName.IsEmpty())
			{
				ModeName = FText::FromName(Mode.ID);
			}
			Args.Add(TEXT("Mode"), ModeName);
			const FText Tooltip = FText::Format( NSLOCTEXT("LevelEditor", "ModeTooltipF", "Activate Mode: {Mode}"), Args );

			FInputChord DefaultKeyBinding;
			if (Mode.IsVisible() && editorMode < EdModeKeys.Num())
			{
				DefaultKeyBinding = FInputChord(EModifierKey::Shift, EdModeKeys[editorMode]);
				++editorMode;
			}

			FSlateIcon ModeIcon = (Mode.IconBrush != FSlateIcon()) ? Mode.IconBrush : FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.EditorModes");
			FUICommandInfo::MakeCommandInfo(
				this->AsShared(),
				EditorModeCommand,
				EditorModeCommandName,
				ModeName,
				Tooltip,
				ModeIcon,
				EUserInterfaceActionType::CollapsedButton,
				DefaultKeyBinding);

			EditorModeCommands.Add(EditorModeCommand);
		}
	}
}

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
