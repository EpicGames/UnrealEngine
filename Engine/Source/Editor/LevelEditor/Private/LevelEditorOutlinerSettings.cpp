// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorOutlinerSettings.h"

#include "Filters/FilterBase.h"
#include "Filters/CustomClassFilterData.h"
#include "Filters/GenericFilter.h"

#include "LevelEditor.h"
#include "IPlacementModeModule.h"

#include "UnsavedAssetsTrackerModule.h"
#include "UncontrolledChangelistsModule.h"

#include "SceneOutlinerPublicTypes.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerHelpers.h"

#include "Animation/SkeletalMeshActor.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/SkyAtmosphereComponent.h"
#include "Components/VolumetricCloudComponent.h"
#include "Engine/LocalFogVolume.h"
#include "Engine/Blueprint.h"
#include "Engine/Brush.h"
#include "Engine/ExponentialHeightFog.h"
#include "Engine/PostProcessVolume.h"
#include "Engine/StaticMeshActor.h"
#include "InstancedFoliageActor.h"
#include "Landscape.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "Sound/AmbientSound.h"
#include "SourceControlHelpers.h"

#define LOCTEXT_NAMESPACE "LevelEditorOutlinerSettings"

const FString FLevelEditorOutlinerSettings::UnsavedAssetsFilterName("UnsavedAssetsFilter");
const FString FLevelEditorOutlinerSettings::UncontrolledAssetsFilterName("UncontrolledAssetsFilter");

FLevelEditorOutlinerSettings::~FLevelEditorOutlinerSettings()
{
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnFilesLoaded().RemoveAll(this);
	}
}

void FLevelEditorOutlinerSettings::Initialize()
{
	if (!GIsEditor) // to avoid crashes in server builds as CreateDefaultFilters will force loading the PlacementMode Module
	{
		return;
	} 
	
	FDelayedAutoRegisterHelper EndOfEngineInit(EDelayedRegisterRunPhase::EndOfEngineInit, [WeakThis = AsWeak()]()
	{
		if (const TSharedPtr<FLevelEditorOutlinerSettings> This = WeakThis.Pin())
		{
			// Ensure all the CustomClassFilters are up to date by rerunning CreateDefaultFilters when the asset registry is loaded
			// This is to ensure that FPlacementModeModule can access all placeable classes during the call to CreateDefaultFilters
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
			AssetRegistry.OnFilesLoaded().AddSPLambda(This.Get(), [WeakThis]()
			{
				if (const TSharedPtr<FLevelEditorOutlinerSettings> This = WeakThis.Pin())
				{
					IAssetRegistry::GetChecked().OnFilesLoaded().RemoveAll(This.Get());
					This->CreateDefaultFilters();
				}
			});
		}
	});
}

void FLevelEditorOutlinerSettings::AddCustomFilter(TSharedRef<FFilterBase<const ISceneOutlinerTreeItem&>> InCustomFilter)
{
	CustomFilters.Add(InCustomFilter);
}

void FLevelEditorOutlinerSettings::AddCustomFilter(FOutlinerFilterFactory InCreateCustomFilter)
{
	check (InCreateCustomFilter.IsBound())
	
	CustomFilterDelegates.Add(InCreateCustomFilter);
}

void FLevelEditorOutlinerSettings::AddCustomClassFilter(TSharedRef<FCustomClassFilterData> InCustomClassFilterData)
{
	TSharedRef<FCustomClassFilterData>* FoundFilter = CustomClassFilters.FindByPredicate([&InCustomClassFilterData](const TSharedPtr<FCustomClassFilterData> FilterData)
	{
		return FilterData->GetClassPathName() == InCustomClassFilterData->GetClassPathName();
	});

	// If a filter for the class already exists, just add the new categories to it
	// TODO: Not perfect, what if the user wants to use InCustomClassFilterData after if there is already a dupe?
	if (FoundFilter)
	{
		TArray<TSharedPtr<FFilterCategory>> Categories = InCustomClassFilterData->GetCategories();

		for(const TSharedPtr<FFilterCategory>& Category : Categories)
		{
			(*FoundFilter)->AddCategory(Category);
		}
	}
	else
	{
		CustomClassFilters.Add(InCustomClassFilterData);
	}
}

TSharedPtr<FFilterCategory> FLevelEditorOutlinerSettings::GetFilterCategory(const FName& CategoryName)
{
	if (TSharedPtr<FFilterCategory>* FoundCategory = FilterBarCategories.Find(CategoryName))
	{
		return *FoundCategory;
	}

	return nullptr;
}

void FLevelEditorOutlinerSettings::SetupBuiltInCategories()
{
	// First setup our unique built in categories
	TSharedPtr<FFilterCategory> CommonFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("CommonFiltersCategory", "Common"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::Common(), CommonFiltersCategory);

	TSharedPtr<FFilterCategory> AudioFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("AudioFiltersCategory", "Audio"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::Audio(), AudioFiltersCategory);

	TSharedPtr<FFilterCategory> AnimationFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("AnimationFiltersCategory", "Animation"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::Animation(), AnimationFiltersCategory);
	
	TSharedPtr<FFilterCategory> GeometryFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("GeometryFiltersCategory", "Geometry"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::Geometry(), GeometryFiltersCategory);

	TSharedPtr<FFilterCategory> EnvironmentFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("EnvironmentFiltersCategory", "Environment"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::Environment(), EnvironmentFiltersCategory);
	
	TSharedPtr<FFilterCategory> VPFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("VPFiltersCategory", "Virtual Production"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::VirtualProduction(), VPFiltersCategory);

	TSharedPtr<FFilterCategory> SCCFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("SCCFiltersCategory", "Revision Control"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::SourceControl(), SCCFiltersCategory);
	
	// Now convert some of the built in placement mode categories we want to filter categories and add them

	PlacementToFilterCategoryMap.Add(FBuiltInPlacementCategories::Basic(), FLevelEditorOutlinerBuiltInCategories::Basic());
	TSharedPtr<FFilterCategory> BasicFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("BasicFilterCategory", "Basic"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::Basic(), BasicFilterCategory);

	PlacementToFilterCategoryMap.Add(FBuiltInPlacementCategories::Lights(), FLevelEditorOutlinerBuiltInCategories::Lights());
	TSharedPtr<FFilterCategory> LightsFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("LightsFilterCategory", "Lights"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::Lights(), LightsFilterCategory);

	PlacementToFilterCategoryMap.Add(FBuiltInPlacementCategories::Visual(), FLevelEditorOutlinerBuiltInCategories::Visual());
	TSharedPtr<FFilterCategory> VisualFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("VisualFilterCategory", "Visual"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::Visual(), VisualFilterCategory);

	PlacementToFilterCategoryMap.Add(FBuiltInPlacementCategories::Volumes(), FLevelEditorOutlinerBuiltInCategories::Volumes());
	TSharedPtr<FFilterCategory> VolumesFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("VolumnesFilterCategory", "Volumes"), FText::GetEmpty());
	FilterBarCategories.Add(FLevelEditorOutlinerBuiltInCategories::Volumes(), VolumesFilterCategory);
}

void FLevelEditorOutlinerSettings::RefreshOutlinersWithActiveFilter(bool bFullRefresh, const FString& InFilterName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLevelEditorOutlinerSettings::RefreshOutlinersWithActiveFilter);

	TWeakPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
	if (LevelEditor.IsValid())
	{
		TArray<TWeakPtr<ISceneOutliner>> SceneOutlinerPtrs = LevelEditor.Pin()->GetAllSceneOutliners();

		for (TWeakPtr<ISceneOutliner> SceneOutlinerPtr : SceneOutlinerPtrs)
		{
			if (TSharedPtr<ISceneOutliner> SceneOutlinerPin = SceneOutlinerPtr.Pin())
			{
				// If this outliner has the input filter active, refresh it
				if (SceneOutlinerPin->IsFilterActive(InFilterName))
				{
					if (bFullRefresh)
					{
						SceneOutlinerPin->FullRefresh();
					}
					else
					{
						SceneOutlinerPin->Refresh();
					}
				}
			}
		}
	}
}

void FLevelEditorOutlinerSettings::CreateDefaultFilters()
{
	// First we will add all items registered to the place actors panel as filters
	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	
	// Get all the Categories
	TArray<FPlacementCategoryInfo> Categories;
	PlacementModeModule.GetSortedCategories(Categories);

	// Remove the Recently Placed, Shapes (don't make sense) and All Classes (too much bloat) categories from the filters
	Categories.RemoveAll([](const FPlacementCategoryInfo& Category)
	{
		return Category.UniqueHandle == FBuiltInPlacementCategories::RecentlyPlaced() || Category.UniqueHandle == FBuiltInPlacementCategories::AllClasses()
			|| Category.UniqueHandle == FBuiltInPlacementCategories::Shapes();
	});

	auto FindOrAddClassFilter = [this](UClass* Class, TSharedPtr<FFilterCategory> FilterCategory)
	{
		// If the underlying class already exists, just add this category to it
		TSharedRef<FCustomClassFilterData>* ExistingClassData = CustomClassFilters.FindByPredicate([Class](const TSharedPtr<FCustomClassFilterData> FilterData)
		{
			return FilterData->GetClassPathName() == Class->GetClassPathName();
		});
		
		if (ExistingClassData)
		{
			(*ExistingClassData)->AddCategory(FilterCategory);
		}
		else
		{
			TSharedRef<FCustomClassFilterData> NewClassData = MakeShared<FCustomClassFilterData>(Class, FilterCategory, FLinearColor::White);
			CustomClassFilters.Add(NewClassData);
		}
	};
	
	for (const FPlacementCategoryInfo& Category : Categories)
	{
		// Make an FFilterCategory using the current PlacementCategory if it doesn't already exist (built in)
		TSharedPtr<FFilterCategory> FilterCategory;
		if (TSharedPtr<FFilterCategory>* FoundCategory = FilterBarCategories.Find(Category.UniqueHandle))
		{
			FilterCategory = *FoundCategory;
		}
		else
		{
			FilterCategory = MakeShared<FFilterCategory>(Category.DisplayName, FText::GetEmpty());
		}

		FName CategoryName = Category.UniqueHandle;

		// If it is a built in placement mode category, convert the name to a built in filter category
		if (FName* MappedCategoryName = PlacementToFilterCategoryMap.Find(CategoryName))
		{
			CategoryName = *MappedCategoryName;
		}

		// Add the category to our list
		FilterBarCategories.Add(Category.UniqueHandle, FilterCategory);

		// Get all the items belonging to the current category
		TArray<TSharedPtr<FPlaceableItem>> Items;
		PlacementModeModule.RegenerateItemsForCategory(Category.UniqueHandle);
		PlacementModeModule.GetItemsForCategory(Category.UniqueHandle, Items);

		// Add each item to as a filter
		for(TSharedPtr<FPlaceableItem>& Item : Items)
		{
			// Get the underlying class from the Actor belonging to this item
			const bool bIsClass = Item->AssetData.GetClass() == UClass::StaticClass();
			const bool bIsActor = bIsClass ? CastChecked<UClass>(Item->AssetData.GetAsset())->IsChildOf(AActor::StaticClass()) : false;
			const AActor* DefaultActor = nullptr;
			if (Item->Factory != nullptr)
			{
				DefaultActor = Item->Factory->GetDefaultActor(Item->AssetData);
			}
			else if (bIsActor)
			{
				DefaultActor = CastChecked<AActor>(CastChecked<UClass>(Item->AssetData.GetAsset())->GetDefaultObject(false));
			
			}
			if (!DefaultActor)
			{
				continue;
			}
			UClass* Class = DefaultActor->GetClass();

			FindOrAddClassFilter(Class, FilterCategory);
		}
	}

	// Now we add some custom filterable types that are not a part of the place actors panel

	TSharedPtr<FFilterCategory>* CommonFilterCategory = FilterBarCategories.Find(FLevelEditorOutlinerBuiltInCategories::Common());

	if (CommonFilterCategory)
	{
		FindOrAddClassFilter(AStaticMeshActor::StaticClass(), *CommonFilterCategory);
		FindOrAddClassFilter(ALevelInstance::StaticClass(), *CommonFilterCategory);
		FindOrAddClassFilter(ASkeletalMeshActor::StaticClass(), *CommonFilterCategory);
		FindOrAddClassFilter(UBlueprint::StaticClass(), *CommonFilterCategory);
		FindOrAddClassFilter(APostProcessVolume::StaticClass(), *CommonFilterCategory);
	}

	TSharedPtr<FFilterCategory>* GeometryFilterCategory = FilterBarCategories.Find(FLevelEditorOutlinerBuiltInCategories::Geometry());

	if (GeometryFilterCategory)
	{
		FindOrAddClassFilter(AStaticMeshActor::StaticClass(), *GeometryFilterCategory);
		FindOrAddClassFilter(ABrush::StaticClass(), *GeometryFilterCategory);
	}

	TSharedPtr<FFilterCategory>* AnimationFilterCategory = FilterBarCategories.Find(FLevelEditorOutlinerBuiltInCategories::Animation());

	if (AnimationFilterCategory)
	{
		FindOrAddClassFilter(ASkeletalMeshActor::StaticClass(), *AnimationFilterCategory);
	}

	TSharedPtr<FFilterCategory>* EnvironmentFilterCategory = FilterBarCategories.Find(FLevelEditorOutlinerBuiltInCategories::Environment());

	if (EnvironmentFilterCategory)
	{
		FindOrAddClassFilter(AExponentialHeightFog::StaticClass(), *EnvironmentFilterCategory);
		FindOrAddClassFilter(AInstancedFoliageActor::StaticClass(), *EnvironmentFilterCategory);
		FindOrAddClassFilter(ASkyAtmosphere::StaticClass(), *EnvironmentFilterCategory);
		FindOrAddClassFilter(ALocalFogVolume::StaticClass(), *EnvironmentFilterCategory);
		FindOrAddClassFilter(AVolumetricCloud::StaticClass(), *EnvironmentFilterCategory);
	}
	
	TSharedPtr<FFilterCategory>* AudioFilterCategory = FilterBarCategories.Find(FLevelEditorOutlinerBuiltInCategories::Audio());

	if (AudioFilterCategory)
	{
		FindOrAddClassFilter(AAmbientSound::StaticClass(), *AudioFilterCategory);
	}


	// Destroy and recreate our built in custom filters
	BuiltInCustomFilters.Empty();
	
	CreateSCCFilters();
}

bool FLevelEditorOutlinerSettings::DoesActorPassUnsavedFilter(const ISceneOutlinerTreeItem& InItem)
{
	return UnsavedPackages.Contains(*InItem.GetPackageName());
}

bool FLevelEditorOutlinerSettings::DoesActorPassUncontrolledFilter(const ISceneOutlinerTreeItem& InItem)
{
	return UncontrolledPackages.Contains(*InItem.GetPackageName());
}

void FLevelEditorOutlinerSettings::OnUnsavedAssetAdded(const FString& InAsset)
{
	FNameBuilder PackageName;
	if (FPackageName::TryConvertFilenameToLongPackageName(InAsset, PackageName))
	{
		bool bAlreadyInSet = false;
		UnsavedPackages.Add(FName(PackageName), &bAlreadyInSet);

		if (!bAlreadyInSet)
		{
			// Refresh any outliners that have the unsaved assets filter active to refilter on adding an unsaved asset
			RefreshOutlinersWithActiveFilter(/*bFullRefresh*/ true, UnsavedAssetsFilterName);
		}
	}
}

void FLevelEditorOutlinerSettings::OnUnsavedAssetRemoved(const FString& InAsset)
{
	FNameBuilder PackageName;
	if (FPackageName::TryConvertFilenameToLongPackageName(InAsset, PackageName))
	{
		if (UnsavedPackages.Remove(FName(PackageName)) > 0)
		{
			// Refresh any outliners that have the unsaved assets filter active to refilter on adding an unsaved asset
			RefreshOutlinersWithActiveFilter(/*bFullRefresh*/ true, UnsavedAssetsFilterName);
		}
	}
}

void FLevelEditorOutlinerSettings::CreateSCCFilters()
{
	// Source Control Category
	TSharedPtr<FFilterCategory> SCCFiltersCategory = GetFilterCategory(FLevelEditorOutlinerBuiltInCategories::SourceControl());
	
	// Uncontrolled Actors Filter
	FUncontrolledChangelistsModule::Get().OnUncontrolledChangelistModuleChanged.AddSP(this, &FLevelEditorOutlinerSettings::OnUncontrolledChangelistModuleChanged);
	OnUncontrolledChangelistModuleChanged();
	
	FGenericFilter<const ISceneOutlinerTreeItem&>::FOnItemFiltered UncontrolledFilterDelegate = FGenericFilter<const ISceneOutlinerTreeItem&>::FOnItemFiltered::CreateSP(this, &FLevelEditorOutlinerSettings::DoesActorPassUncontrolledFilter);
	TSharedPtr<FGenericFilter<const ISceneOutlinerTreeItem&>> UncontrolledFilter = MakeShared<FGenericFilter<const ISceneOutlinerTreeItem&>>(SCCFiltersCategory, UncontrolledAssetsFilterName, LOCTEXT("UncontrolledFilterName", "Uncontrolled"), UncontrolledFilterDelegate);
	UncontrolledFilter->SetToolTipText(LOCTEXT("UncontrolledFilterTooltip", "Only show items that are uncontrolled (locally modified outside of revision control)"));
	BuiltInCustomFilters.Add(UncontrolledFilter.ToSharedRef());
	
	// File Management Category
	TSharedPtr<FFilterCategory> FileManagementFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("FileManagementFiltersCategory", "File Management"), FText::GetEmpty());

	// Unsaved Actors
	FUnsavedAssetsTrackerModule& UnsavedAssetsTrackerModule = FUnsavedAssetsTrackerModule::Get();
	
	// Update our cached unsaved packages
	{
		UnsavedPackages.Reset();

		FNameBuilder PackageName;
		for (const FString& Filename : UnsavedAssetsTrackerModule.GetUnsavedAssets())
		{
			if (FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName))
			{
				UnsavedPackages.Add(FName(PackageName));
			}
		}
	}

	UnsavedAssetsTrackerModule.OnUnsavedAssetAdded.AddSP(this, &FLevelEditorOutlinerSettings::OnUnsavedAssetAdded);
	UnsavedAssetsTrackerModule.OnUnsavedAssetRemoved.AddSP(this, &FLevelEditorOutlinerSettings::OnUnsavedAssetRemoved);
	
	FGenericFilter<const ISceneOutlinerTreeItem&>::FOnItemFiltered UnsavedActorsFilterDelegate = FGenericFilter<const ISceneOutlinerTreeItem&>::FOnItemFiltered::CreateSP(this, & FLevelEditorOutlinerSettings::DoesActorPassUnsavedFilter);
	TSharedPtr<FGenericFilter<const ISceneOutlinerTreeItem&>> UnsavedAssetsFilter = MakeShared<FGenericFilter<const ISceneOutlinerTreeItem&>>(FileManagementFiltersCategory, UnsavedAssetsFilterName, LOCTEXT("UnsavedFilterName", "Unsaved"), UnsavedActorsFilterDelegate);
	UnsavedAssetsFilter->SetToolTipText(LOCTEXT("UnsavedAssetsFilterTooltip", "Only show items that are unsaved"));
	BuiltInCustomFilters.Add(UnsavedAssetsFilter.ToSharedRef());
}

void FLevelEditorOutlinerSettings::OnUncontrolledChangelistModuleChanged()
{
	// Update our cached uncontrolled packages
	{
		UncontrolledPackages.Reset();

		FNameBuilder PackageName;
		for (const TSharedRef<FUncontrolledChangelistState>& UncontrolledChangelistState : FUncontrolledChangelistsModule::Get().GetChangelistStates())
		{
			for (const FString& Filename : UncontrolledChangelistState->GetFilenames())
			{
				if (FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName))
				{
					UncontrolledPackages.Add(FName(PackageName));
				}
			}
		}
	}

	// Refresh any Outliners that have the Uncontrolled Filter active
	RefreshOutlinersWithActiveFilter(/*bFullRefresh*/ true, UncontrolledAssetsFilterName);
}

void FLevelEditorOutlinerSettings::GetOutlinerFilters(FSceneOutlinerFilterBarOptions& OutFilterBarOptions)
{
	// Sort the type filters by name
	CustomClassFilters.Sort([](const TSharedRef<FCustomClassFilterData>& ClassA, const TSharedRef<FCustomClassFilterData>& ClassB)
		{
			return ClassA->GetName().CompareTo(ClassB->GetName()) < 0;
		});
	
	OutFilterBarOptions.CustomClassFilters.Append(CustomClassFilters);
	OutFilterBarOptions.CustomFilters.Append(CustomFilters);
	OutFilterBarOptions.CustomFilters.Append(BuiltInCustomFilters);

	for(FOutlinerFilterFactory& CreateFilterDelegate : CustomFilterDelegates)
	{
		TSharedRef<FFilterBase<const ISceneOutlinerTreeItem&>> NewFilter = CreateFilterDelegate.Execute().ToSharedRef();
		OutFilterBarOptions.CustomFilters.Add(NewFilter);
	}
}

#undef LOCTEXT_NAMESPACE