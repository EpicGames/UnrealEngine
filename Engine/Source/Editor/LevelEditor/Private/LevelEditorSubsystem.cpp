// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorSubsystem.h"
#include "EditorActorFolders.h"

#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorScriptingHelpers.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Engine/MapBuildDataRegistry.h"
#include "FileHelpers.h"
#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "LevelEditorMenuContext.h"
#include "Modules/ModuleManager.h"
#include "SLevelViewport.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "UnrealEdGlobals.h"
#include "EditorModeManager.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "ClassIconFinder.h"
#include "LightingBuildOptions.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "LevelEditorInternalTools.h"
#include "EditorState/EditorStateSubsystem.h"
#include "LevelEditorCameraEditorState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelEditorSubsystem)

DEFINE_LOG_CATEGORY_STATIC(LevelEditorSubsystem, Log, All);

#define LOCTEXT_NAMESPACE "LevelEditorSubsystem"

void ULevelEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Collection.InitializeDependency<UActorEditorContextSubsystem>();

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &ULevelEditorSubsystem::ExtendQuickActionMenu));
	UActorEditorContextSubsystem::Get()->RegisterClient(this);
	FWorldDelegates::LevelAddedToWorld.AddUObject(this, &ULevelEditorSubsystem::OnLevelAddedOrRemoved);
	FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &ULevelEditorSubsystem::OnLevelAddedOrRemoved);
	FWorldDelegates::OnCurrentLevelChanged.AddUObject(this, &ULevelEditorSubsystem::OnCurrentLevelChanged);

	FEditorDelegates::PreSaveWorldWithContext.AddUObject(this, &ULevelEditorSubsystem::HandleOnPreSaveWorldWithContext);
	FEditorDelegates::PostSaveWorldWithContext.AddUObject(this, &ULevelEditorSubsystem::HandleOnPostSaveWorldWithContext);

	FEditorDelegates::OnEditorCameraMoved.AddUObject(this, &ULevelEditorSubsystem::HandleOnEditorCameraMoved);

	FEditorDelegates::MapChange.AddUObject(this, &ULevelEditorSubsystem::HandleOnMapChanged);
	FEditorDelegates::OnMapOpened.AddUObject(this, &ULevelEditorSubsystem::HandleOnMapOpened);

	Collection.InitializeDependency<UEditorStateSubsystem>();
	UEditorStateSubsystem::RegisterEditorStateType<ULevelEditorCameraEditorState>();
}

void ULevelEditorSubsystem::Deinitialize()
{
	UActorEditorContextSubsystem::Get()->UnregisterClient(this);
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	FWorldDelegates::OnCurrentLevelChanged.RemoveAll(this);
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	FEditorDelegates::PreSaveWorldWithContext.RemoveAll(this);
	FEditorDelegates::PostSaveWorldWithContext.RemoveAll(this);

	FEditorDelegates::OnEditorCameraMoved.RemoveAll(this);

	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::OnMapOpened.RemoveAll(this);

	UEditorStateSubsystem::UnregisterEditorStateType<ULevelEditorCameraEditorState>();
}

void ULevelEditorSubsystem::ExtendQuickActionMenu()
{
	FToolMenuOwnerScoped MenuOwner(this);
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.InViewportPanel");
	{
		FToolMenuSection& Section = Menu->FindOrAddSection("QuickActions");
		FToolMenuEntry& Entry = Section.AddDynamicEntry("LevelActors", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
			{
				UQuickActionMenuContext* Context = InSection.FindContext<UQuickActionMenuContext>();
				if (Context && Context->CurrentSelection && Context->CurrentSelection->GetElementList()->Num() == 1)
				{
				
					FToolUIAction PilotActorAction;
					PilotActorAction.ExecuteAction = FToolMenuExecuteAction::CreateUObject(this, &ULevelEditorSubsystem::PilotLevelActor);
					FToolMenuEntry& PilotActorEntry = InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
						"PilotActor",
						PilotActorAction,
						LOCTEXT("PilotSelectedActor", "Pilot Selected Actor"),
						LOCTEXT("PilotSelectedActorToolTip", "Move the selected actor around using the viewport controls, and bind the viewport to the actor's location and orientation."),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelViewport.PilotSelectedActor")
					));
				//	LevelEditorSubsystem.AddKeybindFromCommand(FLightEditingCommands::Get().SwapLightType);
				}
			}));
	}

}

void ULevelEditorSubsystem::PilotLevelActor(const FToolMenuContext& InContext)
{
	UQuickActionMenuContext* QuickMenuContext = InContext.FindContext<UQuickActionMenuContext>();
	AActor* SelectedActor = QuickMenuContext->CurrentSelection->GetTopSelectedObject<AActor>();
	PilotLevelActor(SelectedActor);
}

void ULevelEditorSubsystem::PilotLevelActor(AActor* ActorToPilot, FName ViewportConfigKey) 
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();

		LevelViewportClient.SetActorLock(ActorToPilot);
		if (LevelViewportClient.IsPerspective() && LevelViewportClient.GetActiveActorLock().IsValid())
		{
			LevelViewportClient.MoveCameraToLockedActor();
		}
	}
}

void ULevelEditorSubsystem::EjectPilotLevelActor(FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();

		if (AActor* LockedActor = LevelViewportClient.GetActiveActorLock().Get())
		{
			//// Check to see if the locked actor was previously overriding the camera settings
			//if (CanGetCameraInformationFromActor(LockedActor))
			//{
			//	// Reset the settings
			//	LevelViewportClient.ViewFOV = LevelViewportClient.FOVAngle;
			//}

			LevelViewportClient.SetActorLock(nullptr);

			// remove roll and pitch from camera when unbinding from actors
			GEditor->RemovePerspectiveViewRotation(true, true, false);
		}
	}
}

AActor* ULevelEditorSubsystem::GetPilotLevelActor(FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (!LevelViewport.IsValid())
	{
		return nullptr;
	}
	
	FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();
	if (AActor* CinematicActorLock = LevelViewportClient.GetCinematicActorLock().GetLockedActor())
	{
		return CinematicActorLock;
	}
	return LevelViewportClient.GetActiveActorLock().Get();
}

void ULevelEditorSubsystem::SetExactCameraView(bool bExactCameraView, FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (!LevelViewport.IsValid())
	{
		return;
	}

	FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();
	LevelViewportClient.bLockedCameraView = bExactCameraView;
}

bool ULevelEditorSubsystem::GetExactCameraView(FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (!LevelViewport.IsValid())
	{
		return false;
	}

	FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();
	return LevelViewportClient.bLockedCameraView;
}

void ULevelEditorSubsystem::EditorPlaySimulate()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();
	if (ActiveLevelViewport.IsValid())
	{
		FRequestPlaySessionParams SessionParams;
		SessionParams.WorldType = EPlaySessionWorldType::SimulateInEditor;
		SessionParams.DestinationSlateViewport = ActiveLevelViewport;

		GUnrealEd->RequestPlaySession(SessionParams);
	}
}

void ULevelEditorSubsystem::EditorInvalidateViewports()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveLevelViewport();
	if (ActiveLevelViewport.IsValid())
	{
		FLevelEditorViewportClient& LevelViewportClient = ActiveLevelViewport->GetLevelViewportClient();
		LevelViewportClient.Invalidate();
	}
}

void ULevelEditorSubsystem::EditorSetViewportRealtime(bool bInRealtime, FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();

		FText RealTimeOverrideSystemName = LOCTEXT("LevelEditorSubsystemRealtimeOverride", "Level Editor Subsystem Realtime Override");

		if (bInRealtime)
		{
			LevelViewportClient.RemoveRealtimeOverride(RealTimeOverrideSystemName);
		}
		else
		{
			LevelViewportClient.AddRealtimeOverride(false, RealTimeOverrideSystemName);
		}
	}
}

void ULevelEditorSubsystem::EditorSetGameView(bool bGameView, FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		if (LevelViewport->IsInGameView() != bGameView)
		{
			LevelViewport->ToggleGameView();
		}
	}
}

bool ULevelEditorSubsystem::EditorGetGameView(FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		return LevelViewport->IsInGameView();
	}
	return false;
}

void ULevelEditorSubsystem::EditorRequestBeginPlay()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();
	if (ActiveLevelViewport.IsValid())
	{
		FRequestPlaySessionParams SessionParams;
		SessionParams.WorldType = EPlaySessionWorldType::PlayInEditor;
		SessionParams.DestinationSlateViewport = ActiveLevelViewport;

		GUnrealEd->RequestPlaySession(SessionParams);
	}
}

void ULevelEditorSubsystem::EditorRequestEndPlay()
{
	GUnrealEd->RequestEndPlayMap();
}

bool ULevelEditorSubsystem::IsInPlayInEditor() const
{
	return GUnrealEd->IsPlayingSessionInEditor();
}

TArray<FName> ULevelEditorSubsystem::GetViewportConfigKeys()
{
	TArray<FName> ViewportConfigKeys;
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
		{
			if (LevelViewport.IsValid())
			{
				ViewportConfigKeys.Add(LevelViewport->GetConfigKey());
			}
		}
	}

	return ViewportConfigKeys;
}

FName ULevelEditorSubsystem::GetActiveViewportConfigKey()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditor->GetActiveViewportInterface();
		if (ActiveLevelViewport.IsValid())
		{
			return ActiveLevelViewport->GetConfigKey();
		}
	}
	return NAME_None;
}

void ULevelEditorSubsystem::SetAllowsCinematicControl(bool bAllow, FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		return LevelViewport->SetAllowsCinematicControl(bAllow);
	}
}

bool ULevelEditorSubsystem::GetAllowsCinematicControl(FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		return LevelViewport->GetAllowsCinematicControl();
	}
	return false;
}

bool ULevelEditorSubsystem::GetLevelViewportCameraInfo(FVector& CameraLocation, FRotator& CameraRotation, FName ViewportConfigKey)
{
	CameraLocation = FVector::ZeroVector;
	CameraRotation = FRotator::ZeroRotator;
	
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		FLevelEditorViewportClient* LevelVC = &LevelViewport->GetLevelViewportClient();
		if (LevelVC)
		{
			CameraLocation = LevelVC->GetViewLocation();
			CameraRotation = LevelVC->GetViewRotation();
			return true;
		}
	}
	return false;
}

void ULevelEditorSubsystem::SetLevelViewportCameraInfo(FVector CameraLocation, FRotator CameraRotation, FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		FLevelEditorViewportClient* LevelVC = &LevelViewport->GetLevelViewportClient();
		if (LevelVC)
		{
			LevelVC->SetViewLocationForOrbiting(CameraLocation);
			LevelVC->SetViewLocation(CameraLocation);
			LevelVC->SetViewRotation(CameraRotation);
		}
	}
}

bool ULevelEditorSubsystem::GetLevelViewportFOV(float& FOV, FName ViewportConfigKey)
{
	FOV = 0.f;
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		FLevelEditorViewportClient* LevelVC = &LevelViewport->GetLevelViewportClient();
		if (LevelVC)
		{
			FOV = LevelVC->ViewFOV;
			return true;
		}
	}
	
	return false;
}

void ULevelEditorSubsystem::SetLevelViewportFOV(float FOV, FName ViewportConfigKey)
{
	TSharedPtr<SLevelViewport> LevelViewport = InternalEditorLevelLibrary::GetLevelViewport(ViewportConfigKey);
	if (LevelViewport.IsValid())
	{
		FLevelEditorViewportClient* LevelVC = &LevelViewport->GetLevelViewportClient();
		if (LevelVC)
		{
			LevelVC->ViewFOV = FMath::Clamp(FOV, 5.f, 170.f);
		}
	}
}

/**
 *
 * Editor Scripting | Level
 *
 **/

bool ULevelEditorSubsystem::NewLevel(const FString& AssetPath, bool bIsPartitionedWorld)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	FString FailureReason;
	FString ObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(AssetPath, FailureReason);
	if (ObjectPath.IsEmpty())
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevel. Failed to create the level. %ls", *FailureReason);
		return false;
	}

	if (!EditorScriptingHelpers::IsAValidPathForCreateNewAsset(ObjectPath, FailureReason))
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevel. Failed to validate the destination. %ls", *FailureReason);
		return false;
	}

	if (FPackageName::DoesPackageExist(ObjectPath))
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevel. Failed to validate the destination '%ls'. There's alreay an asset at the destination.", *ObjectPath);
		return false;
	}

	UWorld* World = GEditor->NewMap(bIsPartitionedWorld);
	if (!World)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevel. Failed to create the new level.");
		return false;
	}

	FString DestinationLongPackagePath = FPackageName::ObjectPathToPackageName(ObjectPath);
	if (!UEditorLoadingAndSavingUtils::SaveMap(World, DestinationLongPackagePath))
	{
		UE_LOGF(LevelEditorSubsystem, Warning, "NewLevel. Failed to save the new level.");
		return false;
	}

	return true;
}

bool ULevelEditorSubsystem::NewLevelFromTemplate(const FString& AssetPath, const FString& TemplateAssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	FString FailureReason;
	FString ObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(AssetPath, FailureReason);
	if (ObjectPath.IsEmpty())
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevelFromTemplate. Failed to create the level. %ls", *FailureReason);
		return false;
	}

	if (!EditorScriptingHelpers::IsAValidPathForCreateNewAsset(ObjectPath, FailureReason))
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevelFromTemplate. Failed to validate the destination. %ls", *FailureReason);
		return false;
	}

	// DuplicateAsset does it, but failed with a Modal
	if (FPackageName::DoesPackageExist(ObjectPath))
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevelFromTemplate. Failed to validate the destination '%ls'. There's alreay an asset at the destination.", *ObjectPath);
		return false;
	}

	FString TemplateObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(TemplateAssetPath, FailureReason);
	if (TemplateObjectPath.IsEmpty())
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevelFromTemplate. Failed to create the level. %ls", *FailureReason);
		return false;
	}

	const bool bLoadAsTemplate = true;
	// Load the template map file - passes LoadAsTemplate==true making the
	// level load into an untitled package that won't save over the template
	if (!FEditorFileUtils::LoadMap(*TemplateObjectPath, bLoadAsTemplate))
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevelFromTemplate. Failed to create the new level from template.");
		return false;
	}

	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevelFromTemplate. Failed to find the new created world.");
		return false;
	}

	FString DestinationLongPackagePath = FPackageName::ObjectPathToPackageName(ObjectPath);
	if (!UEditorLoadingAndSavingUtils::SaveMap(World, DestinationLongPackagePath))
	{
		UE_LOGF(LevelEditorSubsystem, Error, "NewLevelFromTemplate. Failed to save the new level.");
		return false;
	}

	return true;
}

bool ULevelEditorSubsystem::LoadLevel(const FString& AssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	FString FailureReason;
	FString ObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(AssetPath, FailureReason);
	if (ObjectPath.IsEmpty())
	{
		UE_LOGF(LevelEditorSubsystem, Error, "LoadLevel. Failed to load level: %ls", *FailureReason);
		return false;
	}

	return UEditorLoadingAndSavingUtils::LoadMap(ObjectPath) != nullptr;
}

bool ULevelEditorSubsystem::SaveCurrentLevel()
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	UWorld* World = UnrealEditorSubsystem->GetEditorWorld();
	if (!World)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "SaveCurrentLevel. Can't save the current level because there is no world.");
		return false;
	}

	ULevel* Level = World->GetCurrentLevel();
	if (!Level)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "SaveCurrentLevel. Can't save the level because there is no current level.");
		return false;
	}

	FString Filename = FEditorFileUtils::GetFilename(Level->OwningWorld);
	if (Filename.Len() == 0)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "SaveCurrentLevel. Can't save the level because it doesn't have a filename. Use EditorLoadingAndSavingUtils.");
		return false;
	}

	TArray<UPackage*> MapPackages;
	MapPackages.Add(Level->GetOutermost());

	if (Level->MapBuildData)
	{
		MapPackages.AddUnique(Level->MapBuildData->GetOutermost());
	}

	// Checkout without a prompt
	TArray<UPackage*>* PackagesCheckedOut = nullptr;
	const bool bErrorIfAlreadyCheckedOut = false;
	FEditorFileUtils::CheckoutPackages(MapPackages, PackagesCheckedOut, bErrorIfAlreadyCheckedOut);

	return FEditorFileUtils::SaveLevel(Level);
}

bool ULevelEditorSubsystem::SaveAllDirtyLevels()
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	UWorld* World = UnrealEditorSubsystem->GetEditorWorld();
	if (!World)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "SaveAllDirtyLevels. Can't save the current level because there is no world.");
		return false;
	}

	TArray<UPackage*> DirtyMapPackages;
	TArray<ULevel*> DirtyLevels;
	for (ULevel* Level : World->GetLevels())
	{
		if (Level)
		{
			UPackage* OutermostPackage = Level->GetOutermost();
			if (OutermostPackage->IsDirty())
			{
				FString Filename = FEditorFileUtils::GetFilename(Level->OwningWorld);
				if (Filename.Len() == 0)
				{
					UE_LOGF(LevelEditorSubsystem, Warning, "SaveAllDirtyLevels. Can't save the level '%ls' because it doesn't have a filename. Use EditorLoadingAndSavingUtils.", *OutermostPackage->GetName());
					continue;
				}

				DirtyLevels.Add(Level);
				DirtyMapPackages.Add(OutermostPackage);

				if (Level->MapBuildData)
				{
					UPackage* BuiltDataPackage = Level->MapBuildData->GetOutermost();
					if (BuiltDataPackage->IsDirty() && BuiltDataPackage != OutermostPackage)
					{
						DirtyMapPackages.Add(BuiltDataPackage);
					}
				}
			}
		}
	}

	bool bAllSaved = true;
	if (DirtyMapPackages.Num() > 0)
	{
		// Checkout without a prompt
		TArray<UPackage*>* PackagesCheckedOut = nullptr;
		const bool bErrorIfAlreadyCheckedOut = false;
		FEditorFileUtils::CheckoutPackages(DirtyMapPackages, PackagesCheckedOut, bErrorIfAlreadyCheckedOut);

		for (ULevel* Level : DirtyLevels)
		{
			bool bSaved = FEditorFileUtils::SaveLevel(Level);
			if (!bSaved)
			{
				UE_LOGF(LevelEditorSubsystem, Warning, "SaveAllDirtyLevels. Can't save the level '%ls'.", *World->GetOutermost()->GetName());
				bAllSaved = false;
			}
		}
	}
	else
	{
		UE_LOGF(LevelEditorSubsystem, Log, "SaveAllDirtyLevels. There is no dirty level.");
	}

	return bAllSaved;
}

bool ULevelEditorSubsystem::SetCurrentLevelByName(FName LevelName)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (LevelName == NAME_None)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "SetCurrentLevel. LevelName is invalid.");
		return false;
	}

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	UWorld* World = UnrealEditorSubsystem->GetEditorWorld();
	if (!World)
	{
		UE_LOGF(LevelEditorSubsystem, Warning, "SetCurrentLevel. Can't set the current level because there is no world.");
		return false;
	}

	bool bLevelFound = false;
	const TArray<ULevel*>& AllLevels = World->GetLevels();
	if (AllLevels.Num() > 0)
	{
		FString LevelNameStr = LevelName.ToString();
		for (ULevel* Level : AllLevels)
		{
			if (FPackageName::GetShortName(Level->GetOutermost()) == LevelNameStr)
			{
				// SetCurrentLevel return true only if the level is changed and it's not the same as the current.
				//For UEditorLevelLibrary, always return true.
				World->SetCurrentLevel(Level);
				bLevelFound = true;
				break;
			}
		}
	}

	return bLevelFound;
}

bool ULevelEditorSubsystem::DeleteActorFolder(FName FolderPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	UWorld* World = UnrealEditorSubsystem->GetEditorWorld();
	if (!World)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "DeleteActorFolder. Can't delete folder because there is no world.");
		return false;
	}

	FActorFolders& ActorFolders = FActorFolders::Get();
	FFolder Folder(FFolder::GetWorldRootFolder(World).GetRootObject(), FolderPath);
	if (!ActorFolders.ContainsFolder(*World, Folder))
	{
		return false;
	}

	ActorFolders.DeleteFolder(*World, Folder);
	return true;
}

ULevel* ULevelEditorSubsystem::GetCurrentLevel()
{
	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	if (!UnrealEditorSubsystem)
	{
		return nullptr;
	}

	UWorld* World = UnrealEditorSubsystem->GetEditorWorld();

	if (!World)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "GetCurrentLevel. Can't Get the current level because there is no world.");
		return nullptr;
	}

	return World->GetCurrentLevel();
}

UTypedElementSelectionSet* ULevelEditorSubsystem::GetSelectionSet()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		return LevelEditor->GetMutableElementSelectionSet();
	}

	return nullptr;
}

bool ULevelEditorSubsystem::BuildLightMaps(ELightingBuildQuality Quality, bool bWithReflectionCaptures)
{
	FLightingBuildOptions LightingOptions;
	LightingOptions.QualityLevel = Quality;

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	UWorld* World = UnrealEditorSubsystem->GetEditorWorld();
	if (!World)
	{
		UE_LOGF(LevelEditorSubsystem, Error, "BuildLightMaps. Can't build the light maps of the current level because there is no world.");
		return false;
	}

	bool Success = false;

	auto BuildFailedDelegate = [&World, &Success]() {
		UE_LOGF(LevelEditorSubsystem, Error, "BuildLightMaps. Failed building lighting for %ls", *World->GetOutermost()->GetName());
		Success = false;
	};
	FDelegateHandle BuildFailedDelegateHandle = FEditorDelegates::OnLightingBuildFailed.AddLambda(BuildFailedDelegate);

	auto BuildSucceededDelegate = [&World, &Success]() {
		UE_LOGF(LevelEditorSubsystem, Log, "BuildLightMaps. Successfully built lighting for %ls", *World->GetOutermost()->GetName());
		Success = true;
	};
	FDelegateHandle BuildSucceededDelegateHandle = FEditorDelegates::OnLightingBuildSucceeded.AddLambda(BuildSucceededDelegate);

	UE_LOGF(LevelEditorSubsystem, Log, "BuildLightMaps. Start building lighting for %ls", *World->GetOutermost()->GetName());
	GEditor->BuildLighting(LightingOptions);
	while (GEditor->IsLightingBuildCurrentlyRunning())
	{
		GEditor->UpdateBuildLighting();
	}

	if (bWithReflectionCaptures)
	{
		GEditor->BuildReflectionCaptures();
	}

	FEditorDelegates::OnLightingBuildFailed.Remove(BuildFailedDelegateHandle);
	FEditorDelegates::OnLightingBuildSucceeded.Remove(BuildSucceededDelegateHandle);

	return Success;
}

FEditorModeTools* ULevelEditorSubsystem::GetLevelEditorModeManager()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid() && !IsRunningCommandlet())
	{
		return &LevelEditor->GetEditorModeManager();
	}

	return nullptr;
}

// Widget used to show current level in viewport
class SCurrentLevelWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCurrentLevelWidget) {}
	SLATE_ARGUMENT(UWorld*, World)
	SLATE_END_ARGS()

	~SCurrentLevelWidget()
	{
		GEditor->GetEditorWorldContext().RemoveRef(World);
	}
	void Construct(const FArguments& InArgs)
	{
		World = (InArgs._World);
		GEditor->GetEditorWorldContext().AddRef(World);
		CommandList = MakeShareable(new FUICommandList);
				
		ChildSlot
		[
			SNew(SComboButton)
			.Cursor(EMouseCursor::Default)
			.VAlign(VAlign_Center)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.Visibility(this, &SCurrentLevelWidget::GetCurrentLevelButtonVisibility)
			.OnGetMenuContent(this, &SCurrentLevelWidget::GenerateLevelMenu)
			.ButtonContent()
			[
				// Current Level
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(0.0f, 0.0f, 2.0f, 0.0f)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Visibility(this, &SCurrentLevelWidget::GetCurrentLevelTextVisibility)
					.Text(this, &SCurrentLevelWidget::GetCurrentLevelText)
				]
				// Referencing Level Instance (if any)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					.Visibility_Lambda([this]() { return IsEditingLevelInstanceCurrentLevel() ? EVisibility::Visible : EVisibility::Collapsed; })
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FromBegin", "("))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(1.f, 1.f, 1.f, 1.f)
					[
						SNew(SBox)
						.WidthOverride(16.f)
						.HeightOverride(16.f)
						[
							SNew(SImage)
							.Image_Lambda([this]() { return FClassIconFinder::FindIconForActor(GetEditingLevelInstance()); })
							.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.AccentGreen"))
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							AActor* LevelInstance = GetEditingLevelInstance();
							return LevelInstance ? FText::FromString(LevelInstance->GetActorLabel()) : FText::GetEmpty();
						})
						.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.AccentGreen"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.f, 0.f, 4.f, 0.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FromEnd", ")"))
					]
				]
			]
		];
	}

private:
	AActor* GetEditingLevelInstance() const
	{
		return InternalEditorLevelLibrary::GetEditingLevelInstance(GetWorld());
	}

	bool IsEditingLevelInstanceCurrentLevel() const
	{
		return InternalEditorLevelLibrary::IsEditingLevelInstanceCurrentLevel(GetWorld());
	}

	FText GetCurrentLevelText() const
	{
		if (GetWorld() && GetWorld()->GetCurrentLevel())
		{
			// Get the level name 
			const FText ActualLevelName = FText::FromName(FPackageName::GetShortFName(GetWorld()->GetCurrentLevel()->GetOutermost()->GetFName()));
			if (GetWorld()->GetCurrentLevel() == GetWorld()->PersistentLevel)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ActualLevelName"), ActualLevelName);
				return FText::Format(LOCTEXT("LevelName", "{0} (Persistent)"), ActualLevelName);
			}
			return ActualLevelName;
		}
		return FText::GetEmpty();
	}
	
	bool IsVisible() const
	{
		return InternalEditorLevelLibrary::IsActorEditorContextVisible(GetWorld());
	}

	EVisibility GetCurrentLevelTextVisibility() const
	{
		return IsVisible() ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
	}

	EVisibility GetCurrentLevelButtonVisibility() const
	{
		return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	TSharedRef<SWidget> GenerateLevelMenu() const
	{
		// Get all menu extenders for this context menu from the level editor module
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedRef<FUICommandList> InCommandList = CommandList.ToSharedRef();
		TSharedPtr<FExtender> MenuExtender = LevelEditorModule.AssembleExtenders(InCommandList, LevelEditorModule.GetAllLevelEditorLevelMenuExtenders());

		// Create the menu
		FMenuBuilder LevelMenuBuilder(/*bShouldCloseWindowAfterMenuSelection*/true, InCommandList, MenuExtender);
		LevelMenuBuilder.BeginSection("LevelListing", LOCTEXT("Levels", "Levels"));
		LevelMenuBuilder.EndSection();
		return LevelMenuBuilder.MakeWidget();
	}

	UWorld* GetWorld() const
	{
		return World;
	}

	TSharedPtr<FUICommandList> CommandList;
	UWorld* World;
};

bool ULevelEditorSubsystem::GetActorEditorContextDisplayInfo(UWorld* InWorld, FActorEditorContextClientDisplayInfo& OutDiplayInfo) const
{
	if (InternalEditorLevelLibrary::IsActorEditorContextVisible(InWorld))
	{
		OutDiplayInfo.Title = TEXT("Level");
		OutDiplayInfo.Brush = FSlateIconFinder::FindIconBrushForClass(UWorld::StaticClass());
		return true;
	}
	return false;
}

void ULevelEditorSubsystem::OnLevelAddedOrRemoved(ULevel* InLevel, UWorld* InWorld)
{
	if (!InWorld->IsGameWorld())
	{
		ActorEditorContextClientChanged.Broadcast(this);
	}
}

void ULevelEditorSubsystem::OnCurrentLevelChanged(ULevel* InNewLevel, ULevel* InOldLevel, UWorld* InWorld)
{
	if (!InWorld->IsGameWorld())
	{
		ActorEditorContextClientChanged.Broadcast(this);
	}
}

void ULevelEditorSubsystem::HandleOnPreSaveWorldWithContext(class UWorld* World, FObjectPreSaveContext ObjectSaveContext)
{
	OnPreSaveWorld.Broadcast(ObjectSaveContext.GetSaveFlags(), World);
}

void ULevelEditorSubsystem::HandleOnPostSaveWorldWithContext(class UWorld* World, FObjectPostSaveContext ObjectSaveContext)
{
	OnPostSaveWorld.Broadcast(ObjectSaveContext.GetSaveFlags(), World, ObjectSaveContext.SaveSucceeded());
}

void ULevelEditorSubsystem::HandleOnEditorCameraMoved(const FVector& Location, const FRotator& Rotation, ELevelViewportType ViewportType, int32 ViewIndex)
{
	OnEditorCameraMoved.Broadcast(Location, Rotation, ViewportType, ViewIndex);
}

void ULevelEditorSubsystem::HandleOnMapChanged(uint32 MapChangeFlags)
{
	OnMapChanged.Broadcast(MapChangeFlags);
}

void ULevelEditorSubsystem::HandleOnMapOpened(const FString& Filename, bool bAsTemplate)
{
	OnMapOpened.Broadcast(Filename, bAsTemplate);
}

TSharedRef<SWidget> ULevelEditorSubsystem::GetActorEditorContextWidget(UWorld* InWorld) const
{
	return SNew(SCurrentLevelWidget).World(InWorld);
}

#undef LOCTEXT_NAMESPACE
