// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorToolBar.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SSpacer.h"
#include "Styling/AppStyle.h"
#include "Settings/EditorExperimentalSettings.h"
#include "GameMapsSettings.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/GameModeBase.h"
#include "GameFramework/HUD.h"
#include "GameFramework/GameStateBase.h"
#include "Engine/TextureStreamingTypes.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "SourceCodeNavigation.h"
#include "Kismet2/DebuggerCommands.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"
#include "ActorTreeItem.h"
#include "SScalabilitySettings.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "LevelSequenceActor.h"
#include "LevelSequence.h"
#include "Engine/LevelScriptBlueprint.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Input/SVolumeControl.h"
#include "Features/IModularFeatures.h"
#include "EngineUtils.h"
#include "ScopedTransaction.h"
#include "Features/EditorFeatures.h"
#include "Misc/ConfigCacheIni.h"
#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"
#include "Misc/ScopedSlowTask.h"
#include "MaterialShaderQualitySettings.h"
#include "LevelEditorMenuContext.h"
#include "ToolMenus.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "LevelEditorModesActions.h"
#include "ISourceControlModule.h"
#include "Styling/ToolBarStyle.h"
#include "PlatformInfo.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ViewportToolbar/LevelEditorSubmenus.h"

namespace UE::Editor::Private
{

int32 CVarLevelEditorToolbarSettingsValue = 0;

}

static FAutoConsoleVariableRef CVarToolMenusViewportToolbars(
	TEXT("LevelEditorToolbarSettings"),
	UE::Editor::Private::CVarLevelEditorToolbarSettingsValue,
	TEXT("Control whether the level editor Settings menu is visible in the toolbar. Set to 1 to show it and "
		 "0 (default) to hide it."),
	ECVF_Default
);

FName FLevelEditorToolBar::SecondaryModeToolbarName("LevelEditor.SecondaryToolbar");

namespace PreviewModeFunctionality
{
	FText GetPreviewModeText()
	{
		const FPreviewPlatformMenuItem* Item = FDataDrivenPlatformInfoRegistry::GetAllPreviewPlatformMenuItems().FindByPredicate([](const FPreviewPlatformMenuItem& TestItem)
			{
				return GEditor->PreviewPlatform.PreviewPlatformName == TestItem.PlatformName && GEditor->PreviewPlatform.PreviewShaderFormatName == TestItem.ShaderFormat && GEditor->PreviewPlatform.PreviewShaderPlatformName == TestItem.PreviewShaderPlatformName;
			});
		return Item ? Item->IconText : FText();
	}

	FText GetPreviewModeTooltip()
	{
#define LOCTEXT_NAMESPACE "LevelEditorToolBar"
		EShaderPlatform PreviewShaderPlatform = GEditor->PreviewPlatform.GetShaderPlatform();
		FText FriendlyNamePreviewOff = FDataDrivenShaderPlatformInfo::GetFriendlyName(GMaxRHIShaderPlatform);
		FText FriendlyNamePreviewOn = GEditor->PreviewPlatform.GetFriendlyName();
		{
			const FText& RenderingAsPlatformName = GEditor->PreviewPlatform.bPreviewShaderPlatformActive ? FriendlyNamePreviewOn : FriendlyNamePreviewOff;
			const FText& SwitchToPlatformName = GEditor->PreviewPlatform.bPreviewShaderPlatformActive ? FriendlyNamePreviewOff : FriendlyNamePreviewOn;
			if (PreviewShaderPlatform == GMaxRHIShaderPlatform)
			{
				return FText::Format(LOCTEXT("PreviewModeViewingAs", "Viewing {0}."), RenderingAsPlatformName);
			}
			else if (GEditor->IsPreviewPlatformActive())
			{
				return FText::Format(LOCTEXT("PreviewModeViewingAsSwitchTo", "Viewing {0}. Click to preview {1}."), RenderingAsPlatformName, SwitchToPlatformName);
			}
			else
			{
				return FText::Format(LOCTEXT("PreviewModePreviewingAsSwitchTo", "Previewing {0}. Click to view {1}."), RenderingAsPlatformName, SwitchToPlatformName);
			}
		}
#undef LOCTEXT_NAMESPACE
	}

	FSlateIcon  GetPreviewModeIcon()
	{
		const FPreviewPlatformMenuItem* Item = FDataDrivenPlatformInfoRegistry::GetAllPreviewPlatformMenuItems().FindByPredicate([](const FPreviewPlatformMenuItem& TestItem)
			{
				return GEditor->PreviewPlatform.PreviewPlatformName == TestItem.PlatformName;
			});
		if (Item)
		{
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), GEditor->IsPreviewPlatformActive() ? Item->ActiveIconName : Item->InactiveIconName);
		}

		EShaderPlatform ShaderPlatform = FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(GEditor->PreviewPlatform.PreviewShaderPlatformName);

		if (ShaderPlatform == SP_NumPlatforms)
		{
			ShaderPlatform = GetFeatureLevelShaderPlatform(GEditor->PreviewPlatform.PreviewFeatureLevel);
		}
		switch (GEditor->PreviewPlatform.PreviewFeatureLevel)
		{
		case ERHIFeatureLevel::ES3_1:
		{
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), GEditor->IsPreviewPlatformActive() ? "LevelEditor.PreviewMode.Enabled" : "LevelEditor.PreviewMode.Disabled");
		}
		default:
		{
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), GEditor->IsPreviewPlatformActive() ? "LevelEditor.PreviewMode.Enabled" : "LevelEditor.PreviewMode.Disabled");
		}
		}
	}

	LEVELEDITOR_API void AddPreviewToggleButton(FToolMenuSection& Section)
	{
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(
			FLevelEditorCommands::Get().TogglePreviewPlatform,
			TAttribute<FText>::Create(&GetPreviewModeText),
			TAttribute<FText>::Create(&GetPreviewModeTooltip),
			TAttribute<FSlateIcon>::Create(&GetPreviewModeIcon)
		));
	}
}

namespace LevelEditorActionHelpers
{
	/** Filters out any classes for the Class Picker when creating or selecting classes in the Blueprints dropdown */
	class FBlueprintParentFilter_MapModeSettings : public IClassViewerFilter
	{
	public:
		/** Classes to not allow any children of into the Class Viewer/Picker. */
		TSet< const UClass* > AllowedChildrenOfClasses;

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
		{
			return InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) == EFilterReturn::Passed;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) == EFilterReturn::Passed;
		}
	};

	/**
	 * Retrieves the GameMode class
	 *
	 * @param InLevelEditor				The editor to extract the world from
	 * @param bInIsProjectSettings		TRUE if retrieving the game mode from the project settings
	 * @return							The GameMode class in the Project Settings or World Settings
	 */
	static UClass* GetGameModeClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback for the label to display for the GameMode menu selection */
	static FText GetOpenGameModeBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback when selecting a GameMode class, assigns it to the world */
	static void OnSelectGameModeClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback when creating a new GameMode class, creates the Blueprint and assigns it to the world */
	static void OnCreateGameModeClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/**
	 * Retrieves the active GameState class from
	 *
	 * @param InLevelEditor		The editor to extract the world from
	 * @param bInIsProjectSettings		TRUE if retrieving the game mode from the project settings
	 * @return					The active GameState class in the World Settings
	 */
	static UClass* GetGameStateClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback for the label to display for the GameState menu selection */
	static FText GetOpenGameStateBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);
	
	/** Callback when selecting a GameState class, assigns it to the world */
	static void OnSelectGameStateClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback when creating a new GameState class, creates the Blueprint and assigns it to the world */
	static void OnCreateGameStateClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/**
	 * Retrieves the active Pawn class from
	 *
	 * @param InLevelEditor		The editor to extract the world from
	 * @return					The active Pawn class in the World Settings
	 */
	static UClass* GetPawnClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback for the label to display for the Pawn menu selection */
	static FText GetOpenPawnBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback for the tooltip to display for the Pawn menu selection */
	static FText GetOpenPawnBlueprintTooltip(TWeakPtr< SLevelEditor > InLevelEditor);

	/** Callback when selecting a Pawn class, assigns it to the world */
	static void OnSelectPawnClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback when creating a new Pawn class, creates the Blueprint and assigns it to the world */
	static void OnCreatePawnClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/**
	 * Retrieves the active HUD class from
	 *
	 * @param InLevelEditor		The editor to extract the world from
	 * @param bInIsProjectSettings		TRUE if retrieving the game mode from the project settings
	 * @return					The active HUD class in the World Settings
	 */
	static UClass* GetHUDClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback for the label to display for the HUD menu selection */
	static FText GetOpenHUDBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);
	
	/** Callback when selecting a HUD class, assigns it to the world */
	static void OnSelectHUDClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback when creating a new HUD class, creates the Blueprint and assigns it to the world */
	static void OnCreateHUDClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/**
	 * Retrieves the active PlayerController class from
	 *
	 * @param InLevelEditor		The editor to extract the world from
	 * @param bInIsProjectSettings		TRUE if retrieving the game mode from the project settings
	 * @return					The active PlayerController class in the World Settings
	 */
	static UClass* GetPlayerControllerClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback for the label to display for the PlayerController menu selection */
	static FText GetOpenPlayerControllerBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback when selecting a PlayerController class, assigns it to the world */
	static void OnSelectPlayerControllerClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Callback when creating a new PlayerController class, creates the Blueprint and assigns it to the world */
	static void OnCreatePlayerControllerClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings);

	/** Opens a native class's header file if the compiler is available. */
	static void OpenNativeClass(UClass* InClass)
	{
		if(InClass->HasAllClassFlags(CLASS_Native) && FSourceCodeNavigation::IsCompilerAvailable())
		{
			FString NativeParentClassHeaderPath;
			const bool bFileFound = FSourceCodeNavigation::FindClassHeaderPath(InClass, NativeParentClassHeaderPath) 
				&& (IFileManager::Get().FileSize(*NativeParentClassHeaderPath) != INDEX_NONE);
			if (bFileFound)
			{
				const FString AbsoluteHeaderPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*NativeParentClassHeaderPath);
				FSourceCodeNavigation::OpenSourceFileAsync( AbsoluteHeaderPath );
			}
		}
	}

	/** Open the game mode blueprint, in the project settings or world settings */
	static void OpenGameModeBlueprint( TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings )
	{
		if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
		{
			if(UBlueprint* BlueprintClass = Cast<UBlueprint>(GameModeClass->ClassGeneratedBy))
			{
				// @todo Re-enable once world centric works
				const bool bOpenWorldCentric = false;
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
					BlueprintClass,
					bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
					InLevelEditor.Pin() );
			}
			else
			{
				OpenNativeClass(GameModeClass);
			}
		}
	}

	/** Open the game state blueprint, in the project settings or world settings */
	static void OpenGameStateBlueprint( TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings )
	{
		if(UClass* GameStateClass = GetGameStateClass(InLevelEditor, bInIsProjectSettings))
		{
			if(UBlueprint* BlueprintClass = Cast<UBlueprint>(GameStateClass->ClassGeneratedBy))
			{
				// @todo Re-enable once world centric works
				const bool bOpenWorldCentric = false;
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
					BlueprintClass,
					bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
					InLevelEditor.Pin() );
			}
			else
			{
				OpenNativeClass(GameStateClass);
			}
		}
	}

	/** Open the default pawn blueprint, in the project settings or world settings */
	static void OpenDefaultPawnBlueprint( TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings )
	{
		if(UClass* DefaultPawnClass = GetPawnClass(InLevelEditor, bInIsProjectSettings))
		{
			if(UBlueprint* BlueprintClass = Cast<UBlueprint>(DefaultPawnClass->ClassGeneratedBy))
			{
				// @todo Re-enable once world centric works
				const bool bOpenWorldCentric = false;
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
					BlueprintClass,
					bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
					InLevelEditor.Pin() );
			}
			else
			{
				OpenNativeClass(DefaultPawnClass);
			}
		}
	}

	/** Open the HUD blueprint, in the project settings or world settings */
	static void OpenHUDBlueprint( TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings )
	{
		if(UClass* DefaultHUDClass = GetHUDClass(InLevelEditor, bInIsProjectSettings))
		{
			if(UBlueprint* BlueprintClass = Cast<UBlueprint>(DefaultHUDClass->ClassGeneratedBy))
			{
				// @todo Re-enable once world centric works
				const bool bOpenWorldCentric = false;
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
					BlueprintClass,
					bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
					InLevelEditor.Pin() );
			}
			else
			{
				OpenNativeClass(DefaultHUDClass);
			}
		}
	}

	/** Open the player controller blueprint, in the project settings or world settings */
	static void OpenPlayerControllerBlueprint( TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings )
	{
		if(UClass* PlayerControllerClass = GetPlayerControllerClass(InLevelEditor, bInIsProjectSettings))
		{
			if(UBlueprint* BlueprintClass = Cast<UBlueprint>(PlayerControllerClass->ClassGeneratedBy))
			{
				// @todo Re-enable once world centric works
				const bool bOpenWorldCentric = false;
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
					BlueprintClass,
					bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
					InLevelEditor.Pin() );
			}
			else
			{
				OpenNativeClass(PlayerControllerClass);
			}
		}
	}

	/**
	 * Builds a sub-menu for selecting a class
	 *
	 * @param InMenu		Object to append menu items/widgets to
	 * @param InRootClass		The root class to filter the Class Viewer by to only show children of
	 * @param InOnClassPicked	Callback delegate to fire when a class is picked
	 */
	void GetSelectSettingsClassSubMenu(UToolMenu* InMenu, UClass* InRootClass, FOnClassPicked InOnClassPicked)
	{
		// The contents of this menu are added as a custom widget with its own search field so we
		// disable searching in this parent menu to avoid displaying two search fields to the user.
		InMenu->bSearchable = false;

		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.DisplayMode = EClassViewerDisplayMode::ListView;
		Options.bShowObjectRootClass = true;
		Options.bShowNoneOption = true;

		// Only want blueprint actor base classes.
		Options.bIsBlueprintBaseOnly = true;

		// This will allow unloaded blueprints to be shown.
		Options.bShowUnloadedBlueprints = true;

		TSharedPtr< FBlueprintParentFilter_MapModeSettings > Filter = MakeShareable(new FBlueprintParentFilter_MapModeSettings);
		Filter->AllowedChildrenOfClasses.Add(InRootClass);
		Options.ClassFilters.Add(Filter.ToSharedRef());

		FText RootClassName = FText::FromString(InRootClass->GetName());
		TSharedRef<SWidget> ClassViewer = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, InOnClassPicked);
		FFormatNamedArguments FormatArgs;
		FormatArgs.Add(TEXT("RootClass"), RootClassName);
		FToolMenuSection& Section = InMenu->AddSection("SelectSettingsClass", FText::Format(NSLOCTEXT("LevelToolBarViewMenu", "SelectGameModeLabel", "Select {RootClass} class"), FormatArgs));
		Section.AddEntry(FToolMenuEntry::InitWidget("ClassViewer", ClassViewer, FText::GetEmpty(), true));
	}

	/**
	 * Builds a sub-menu for creating a class
	 *
	 * @param InMenu		Object to append menu items/widgets to
	 * @param InRootClass		The root class to filter the Class Viewer by to only show children of
	 * @param InOnClassPicked	Callback delegate to fire when a class is picked
	 */
	void GetCreateSettingsClassSubMenu(UToolMenu* InMenu, UClass* InRootClass, FOnClassPicked InOnClassPicked)
	{
		// The contents of this menu are added as a custom widget with its own search field so we
		// disable searching in this parent menu to avoid displaying two search fields to the user.
		InMenu->bSearchable = false;

		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.DisplayMode = EClassViewerDisplayMode::ListView;
		Options.bShowObjectRootClass = true;

		// Only want blueprint actor base classes.
		Options.bIsBlueprintBaseOnly = true;

		// This will allow unloaded blueprints to be shown.
		Options.bShowUnloadedBlueprints = true;

		TSharedPtr< FBlueprintParentFilter_MapModeSettings > Filter = MakeShareable(new FBlueprintParentFilter_MapModeSettings);
		Filter->AllowedChildrenOfClasses.Add(InRootClass);
		Options.ClassFilters.Add(Filter.ToSharedRef());

		FText RootClassName = FText::FromString(InRootClass->GetName());
		TSharedRef<SWidget> ClassViewer = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, InOnClassPicked);
		FFormatNamedArguments FormatArgs;
		FormatArgs.Add(TEXT("RootClass"), RootClassName);
		FToolMenuSection& Section = InMenu->AddSection("CreateSettingsClass", FText::Format(NSLOCTEXT("LevelToolBarViewMenu", "CreateGameModeLabel", "Select {RootClass} parent class"), FormatArgs));
		Section.AddEntry(FToolMenuEntry::InitWidget("ClassViewer", ClassViewer, FText::GetEmpty(), true));
	}

	/** Helper struct for passing all required data to the GetBlueprintSettingsSubMenu function */
	struct FBlueprintMenuSettings
	{
		/** The UI command for editing the Blueprint class associated with the menu */
		FUIAction EditCommand;

		/** Current class associated with the menu */
		UClass* CurrentClass;

		/** Root class that defines what class children can be set through the menu */
		UClass* RootClass;

		/** Callback when a class is picked, to assign the new class */
		FOnClassPicked OnSelectClassPicked;

		/** Callback when a class is picked, to create a new child class of and assign */
		FOnClassPicked OnCreateClassPicked;

		/** Level Editor these menu settings are for */
		TWeakPtr< SLevelEditor > LevelEditor;

		/** TRUE if these represent Project Settings, FALSE if they represent World Settings */
		bool bIsProjectSettings;
	};

	/** Returns the label of the "Check Out" option based on if source control is present or not */
	FText GetCheckOutLabel()
	{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
		if(ISourceControlModule::Get().IsEnabled())
		{
			return LOCTEXT("CheckoutMenuLabel", "Check Out");
		}
		else
		{
			return LOCTEXT("MakeWritableLabel", "Make Writable");
		}
#undef LOCTEXT_NAMESPACE
	}

	/** Returns the tooltip of the "Check Out" option based on if source control is present or not */
	FText GetCheckOutTooltip()
	{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
		if(ISourceControlModule::Get().IsEnabled())
		{
			return LOCTEXT("CheckoutMenuTooltip", "Checks out the project settings config file so the game mode can be set.");
		}
		else
		{
			return LOCTEXT("MakeWritableTooltip", "Forces the project settings config file to be writable so the game mode can be set.");
		}
#undef LOCTEXT_NAMESPACE
	}

	/**
	 * A sub-menu for the Blueprints dropdown, facilitates all the sub-menu actions such as creating, editing, and selecting classes for the world settings game mode.
	 *
	 * @param InMenu		Object to append menu items/widgets to
	 * @param InCommandList		Commandlist for menu items
	 * @param InSettingsData	All the data needed to create the menu actions
	 */
	void GetBlueprintSettingsSubMenu(UToolMenu* InMenu, FBlueprintMenuSettings InSettingsData);

	/** Returns TRUE if the class can be edited, always TRUE for Blueprints and for native classes a compiler must be present */
	bool CanEditClass(UClass* InClass)
	{
		// For native classes, we can only edit them if a compiler is available
		if(InClass && InClass->HasAllClassFlags(CLASS_Native))
		{
			return FSourceCodeNavigation::IsCompilerAvailable();
		}
		return true;
	}

	/** Returns TRUE if the GameMode's sub-class can be created or selected */
	bool CanCreateSelectSubClass(UClass* InGameModeClass, bool bInIsProjectSettings)
	{
		// Can never create or select project settings sub-classes if the config file is not checked out
		if(bInIsProjectSettings && !FLevelEditorActionCallbacks::CanSelectGameModeBlueprint())
		{
			return false;
		}

		// If the game mode class is native, we cannot set the sub class
		if(!InGameModeClass || InGameModeClass->HasAllClassFlags(CLASS_Native))
		{
			return false;
		}
		return true;
	}

	/** Creates a tooltip for a submenu */
	FText GetSubMenuTooltip(UClass* InClass, UClass* InRootClass, bool bInIsProjectSettings)
	{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
		FFormatNamedArguments Args;
		Args.Add(TEXT("Class"), FText::FromString(InRootClass->GetName()));
		Args.Add(TEXT("TargetLocation"), bInIsProjectSettings? LOCTEXT("Project", "project") : LOCTEXT("World", "world"));
		return FText::Format(LOCTEXT("ClassSubmenu_Tooltip", "Select, edit, or create a new {Class} blueprint for the {TargetLocation}"), Args);
#undef LOCTEXT_NAMESPACE
	}

	/** Creates a tooltip for the create class submenu */
	FText GetCreateMenuTooltip(UClass* InGameModeClass, UClass* InRootClass, bool bInIsProjectSettings)
	{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
		FText ResultText;

		// Game modes can always be created and selected (providing the config is checked out, handled separately)
		if(InRootClass != AGameModeBase::StaticClass() && InGameModeClass->HasAllClassFlags(CLASS_Native))
		{
			ResultText = LOCTEXT("CannotCreateClasses", "Cannot create classes when the game mode is a native class!");
		}
		else if(bInIsProjectSettings && !FLevelEditorActionCallbacks::CanSelectGameModeBlueprint())
		{
			ResultText = LOCTEXT("CannotCreateClasses_NeedsCheckOut", "Cannot create classes when the config file is not writable!");
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("RootClass"), FText::FromString(InRootClass->GetName()));
			Args.Add(TEXT("TargetLocation"), bInIsProjectSettings? LOCTEXT("Project", "project") : LOCTEXT("World", "world"));
			ResultText = FText::Format( LOCTEXT("CreateClass_Tooltip", "Create a new {RootClass} based on a selected class and auto-assign it to the {TargetLocation}"), Args );
		}

		return ResultText;
#undef LOCTEXT_NAMESPACE
	}

	/** Creates a tooltip for the select class submenu */
	FText GetSelectMenuTooltip(UClass* InGameModeClass, UClass* InRootClass, bool bInIsProjectSettings)
	{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
		FText ResultText;

		// Game modes can always be created and selected (providing the config is checked out, handled separately)
		if(InRootClass != AGameModeBase::StaticClass() && InGameModeClass->HasAllClassFlags(CLASS_Native))
		{
			ResultText = LOCTEXT("CannotSelectClasses", "Cannot select classes when the game mode is a native class!");
		}
		else if(bInIsProjectSettings && !FLevelEditorActionCallbacks::CanSelectGameModeBlueprint())
		{
			ResultText = LOCTEXT("CannotSelectClasses_NeedsCheckOut", "Cannot select classes when the config file is not writable!");
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("RootClass"), FText::FromString(InRootClass->GetName()));
			Args.Add(TEXT("TargetLocation"), bInIsProjectSettings? LOCTEXT("Project", "project") : LOCTEXT("World", "world"));
			ResultText = FText::Format( LOCTEXT("SelectClass_Tooltip", "Select a new {RootClass} based on a selected class and auto-assign it to the {TargetLocation}"), Args );
		}
		return ResultText;
#undef LOCTEXT_NAMESPACE
	}

	void CreateGameModeSubMenu(FToolMenuSection& Section, const FName InName, bool bInProjectSettings)
	{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
		Section.AddDynamicEntry(InName, FNewToolMenuSectionDelegate::CreateLambda([=](FToolMenuSection& InSection)
		{
			if (ULevelEditorMenuContext* const Context = InSection.FindContext<ULevelEditorMenuContext>();
			Context && Context->LevelEditor.IsValid())
			{
				LevelEditorActionHelpers::FBlueprintMenuSettings GameModeMenuSettings;
				GameModeMenuSettings.EditCommand =
					FUIAction(
						FExecuteAction::CreateStatic(&OpenGameModeBlueprint, Context->LevelEditor, bInProjectSettings)
					);
				GameModeMenuSettings.OnCreateClassPicked = FOnClassPicked::CreateStatic(&LevelEditorActionHelpers::OnCreateGameModeClassPicked, Context->LevelEditor, bInProjectSettings);
				GameModeMenuSettings.OnSelectClassPicked = FOnClassPicked::CreateStatic(&LevelEditorActionHelpers::OnSelectGameModeClassPicked, Context->LevelEditor, bInProjectSettings);
				GameModeMenuSettings.CurrentClass = LevelEditorActionHelpers::GetGameModeClass(Context->LevelEditor, bInProjectSettings);
				GameModeMenuSettings.RootClass = AGameModeBase::StaticClass();
				GameModeMenuSettings.LevelEditor = Context->LevelEditor;
				GameModeMenuSettings.bIsProjectSettings = bInProjectSettings;

				auto IsGameModeActive = [](TWeakPtr< SLevelEditor > InLevelEditorPtr, bool bInProjSettings)->bool
				{
					UClass* WorldSettingsGameMode = LevelEditorActionHelpers::GetGameModeClass(InLevelEditorPtr, false);
					if ((WorldSettingsGameMode == nullptr) ^ bInProjSettings) //(WorldSettingsGameMode && !bInProjectSettings) || (!WorldSettingsGameMode && bInProjectSettings) )
					{
						return false;
					}
					return true;
				};

				InSection.AddSubMenu(InName, LevelEditorActionHelpers::GetOpenGameModeBlueprintLabel(Context->LevelEditor, bInProjectSettings),
					GetSubMenuTooltip(GameModeMenuSettings.CurrentClass, GameModeMenuSettings.RootClass, bInProjectSettings),
					FNewToolMenuDelegate::CreateStatic(&LevelEditorActionHelpers::GetBlueprintSettingsSubMenu, GameModeMenuSettings),
					FUIAction(FExecuteAction(), FCanExecuteAction(), FIsActionChecked::CreateStatic(IsGameModeActive, Context->LevelEditor, bInProjectSettings)),
					EUserInterfaceActionType::RadioButton);
			}
		}));
#undef LOCTEXT_NAMESPACE
	}

	/**
	 * Builds the game mode's sub menu objects
	 *
	 * @param InSection			Object to append menu items/widgets to
	 * @param InCommandList		Commandlist for menu items
	 * @param InSettingsData	All the data needed to create the menu actions
	 */
	void GetGameModeSubMenu(FToolMenuSection& InSection, const FBlueprintMenuSettings& InSettingsData)
	{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
		// Game State
		LevelEditorActionHelpers::FBlueprintMenuSettings GameStateMenuSettings;
		GameStateMenuSettings.EditCommand = 
			FUIAction(
				FExecuteAction::CreateStatic( &OpenGameStateBlueprint, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings )
			);
		GameStateMenuSettings.OnCreateClassPicked = FOnClassPicked::CreateStatic( &LevelEditorActionHelpers::OnCreateGameStateClassPicked, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings );
		GameStateMenuSettings.OnSelectClassPicked = FOnClassPicked::CreateStatic( &LevelEditorActionHelpers::OnSelectGameStateClassPicked, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings );
		GameStateMenuSettings.CurrentClass = LevelEditorActionHelpers::GetGameStateClass(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings);
		GameStateMenuSettings.RootClass = AGameStateBase::StaticClass();
		GameStateMenuSettings.LevelEditor = InSettingsData.LevelEditor;
		GameStateMenuSettings.bIsProjectSettings = InSettingsData.bIsProjectSettings;

		InSection.AddSubMenu("OpenGameStateBlueprint", LevelEditorActionHelpers::GetOpenGameStateBlueprintLabel(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings),
			GetSubMenuTooltip(GameStateMenuSettings.CurrentClass, GameStateMenuSettings.RootClass, InSettingsData.bIsProjectSettings),
			FNewToolMenuDelegate::CreateStatic( &LevelEditorActionHelpers::GetBlueprintSettingsSubMenu, GameStateMenuSettings )
		);

		// Pawn
		LevelEditorActionHelpers::FBlueprintMenuSettings PawnMenuSettings;
		PawnMenuSettings.EditCommand = 
			FUIAction(
				FExecuteAction::CreateStatic( &OpenDefaultPawnBlueprint, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings )
			);
		PawnMenuSettings.OnCreateClassPicked = FOnClassPicked::CreateStatic( &LevelEditorActionHelpers::OnCreatePawnClassPicked, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings );
		PawnMenuSettings.OnSelectClassPicked = FOnClassPicked::CreateStatic( &LevelEditorActionHelpers::OnSelectPawnClassPicked, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings );
		PawnMenuSettings.CurrentClass = LevelEditorActionHelpers::GetPawnClass(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings);
		PawnMenuSettings.RootClass = APawn::StaticClass();
		PawnMenuSettings.LevelEditor = InSettingsData.LevelEditor;
		PawnMenuSettings.bIsProjectSettings = InSettingsData.bIsProjectSettings;

		InSection.AddSubMenu("OpenPawnBlueprint", LevelEditorActionHelpers::GetOpenPawnBlueprintLabel(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings),
			GetSubMenuTooltip(PawnMenuSettings.CurrentClass, PawnMenuSettings.RootClass, InSettingsData.bIsProjectSettings),
			FNewToolMenuDelegate::CreateStatic( &LevelEditorActionHelpers::GetBlueprintSettingsSubMenu, PawnMenuSettings )
		);

		// HUD
		LevelEditorActionHelpers::FBlueprintMenuSettings HUDMenuSettings;
		HUDMenuSettings.EditCommand = 
			FUIAction(
				FExecuteAction::CreateStatic( &OpenHUDBlueprint, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings )
			);
		HUDMenuSettings.OnCreateClassPicked = FOnClassPicked::CreateStatic( &LevelEditorActionHelpers::OnCreateHUDClassPicked, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings );
		HUDMenuSettings.OnSelectClassPicked = FOnClassPicked::CreateStatic( &LevelEditorActionHelpers::OnSelectHUDClassPicked, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings );
		HUDMenuSettings.CurrentClass = LevelEditorActionHelpers::GetHUDClass(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings);
		HUDMenuSettings.RootClass = AHUD::StaticClass();
		HUDMenuSettings.LevelEditor = InSettingsData.LevelEditor;
		HUDMenuSettings.bIsProjectSettings = InSettingsData.bIsProjectSettings;

		InSection.AddSubMenu("OpenHUDBlueprint", LevelEditorActionHelpers::GetOpenHUDBlueprintLabel(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings),
			GetSubMenuTooltip(HUDMenuSettings.CurrentClass, HUDMenuSettings.RootClass, InSettingsData.bIsProjectSettings),
			FNewToolMenuDelegate::CreateStatic( &LevelEditorActionHelpers::GetBlueprintSettingsSubMenu, HUDMenuSettings )
		);

		// Player Controller
		LevelEditorActionHelpers::FBlueprintMenuSettings PlayerControllerMenuSettings;
		PlayerControllerMenuSettings.EditCommand = 
			FUIAction(
				FExecuteAction::CreateStatic( &OpenPlayerControllerBlueprint, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings )
			);
		PlayerControllerMenuSettings.OnCreateClassPicked = FOnClassPicked::CreateStatic( &LevelEditorActionHelpers::OnCreatePlayerControllerClassPicked, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings );
		PlayerControllerMenuSettings.OnSelectClassPicked = FOnClassPicked::CreateStatic( &LevelEditorActionHelpers::OnSelectPlayerControllerClassPicked, InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings );
		PlayerControllerMenuSettings.CurrentClass = LevelEditorActionHelpers::GetPlayerControllerClass(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings);
		PlayerControllerMenuSettings.RootClass = APlayerController::StaticClass();
		PlayerControllerMenuSettings.LevelEditor = InSettingsData.LevelEditor;
		PlayerControllerMenuSettings.bIsProjectSettings = InSettingsData.bIsProjectSettings;

		InSection.AddSubMenu("OpenPlayerControllerBlueprint", LevelEditorActionHelpers::GetOpenPlayerControllerBlueprintLabel(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings),
			GetSubMenuTooltip(PlayerControllerMenuSettings.CurrentClass, PlayerControllerMenuSettings.RootClass, InSettingsData.bIsProjectSettings),
			FNewToolMenuDelegate::CreateStatic( &LevelEditorActionHelpers::GetBlueprintSettingsSubMenu, PlayerControllerMenuSettings )
		);
#undef LOCTEXT_NAMESPACE
	}

	struct FLevelSortByName
	{
		bool operator ()(const ULevel* LHS, const ULevel* RHS) const
		{
			if (LHS != NULL && LHS->GetOutermost() != NULL && RHS != NULL && RHS->GetOutermost() != NULL)
			{
				return FPaths::GetCleanFilename(LHS->GetOutermost()->GetName()) < FPaths::GetCleanFilename(RHS->GetOutermost()->GetName());
			}
			else
			{
				return false;
			}
		}
	};
}

void LevelEditorActionHelpers::GetBlueprintSettingsSubMenu(UToolMenu* Menu, FBlueprintMenuSettings InSettingsData)
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"

	FSlateIcon EditBPIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Edit"));
	FSlateIcon NewBPIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.PlusCircle"));
	FText RootClassName = FText::FromString(InSettingsData.RootClass->GetName());

	// If there is currently a valid GameMode Blueprint, offer to edit the Blueprint
	if(InSettingsData.CurrentClass)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("RootClass"), RootClassName);
		Args.Add(TEXT("TargetLocation"), InSettingsData.bIsProjectSettings? LOCTEXT("Project", "project") : LOCTEXT("World", "world"));

		FToolMenuSection& Section = Menu->AddSection("EditBlueprintOrClass");
		if(InSettingsData.CurrentClass->ClassGeneratedBy)
		{
			FText BlueprintName = FText::FromString(InSettingsData.CurrentClass->ClassGeneratedBy->GetName());
			Args.Add(TEXT("Blueprint"), BlueprintName);
			Section.AddMenuEntry("EditBlueprint", FText::Format( LOCTEXT("EditBlueprint", "Edit {Blueprint}"), Args), FText::Format( LOCTEXT("EditBlueprint_Tooltip", "Open the {TargetLocation}'s assigned {RootClass} blueprint"), Args), EditBPIcon, InSettingsData.EditCommand );
		}
		else
		{
			FText ClassName = FText::FromString(InSettingsData.CurrentClass->GetName());
			Args.Add(TEXT("Class"), ClassName);

			FText MenuDescription = FText::Format( LOCTEXT("EditNativeClass", "Edit {Class}.h"), Args);
			if(FSourceCodeNavigation::IsCompilerAvailable())
			{
				Section.AddMenuEntry("EditNativeClass", MenuDescription, FText::Format( LOCTEXT("EditNativeClass_Tooltip", "Open the {TargetLocation}'s assigned {RootClass} header"), Args), EditBPIcon, InSettingsData.EditCommand );
			}
			else
			{
				auto CannotEditClass = []() -> bool
				{
					return false;
				};

				// There is no compiler present, this is always disabled with a tooltip to explain why
				Section.AddMenuEntry("EditNativeClass", MenuDescription, FText::Format( LOCTEXT("CannotEditNativeClass_Tooltip", "Cannot edit the {TargetLocation}'s assigned {RootClass} header because no compiler is present!"), Args), EditBPIcon, FUIAction(FExecuteAction(), FCanExecuteAction::CreateStatic(CannotEditClass)) );
			}
		}
	}

	if(InSettingsData.bIsProjectSettings && InSettingsData.CurrentClass && InSettingsData.CurrentClass->IsChildOf(AGameModeBase::StaticClass()) && !FLevelEditorActionCallbacks::CanSelectGameModeBlueprint())
	{
		FToolMenuSection& Section = Menu->AddSection("CheckoutSection", LOCTEXT("CheckoutSection", "Check Out Project Settings") );
		TAttribute<FText> CheckOutLabel;
		CheckOutLabel.BindStatic(&GetCheckOutLabel);

		TAttribute<FText> CheckOutTooltip;
		CheckOutTooltip.BindStatic(&GetCheckOutTooltip);
		Section.AddMenuEntry(FLevelEditorCommands::Get().CheckOutProjectSettingsConfig, CheckOutLabel, CheckOutTooltip, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Error")));
	}

	auto CannotCreateSelectNativeProjectGameMode = [](bool bInIsProjectSettings) -> bool
	{
		// For the project settings, we can only create/select the game mode class if the config is writable
		if(bInIsProjectSettings)
		{
			return FLevelEditorActionCallbacks::CanSelectGameModeBlueprint();
		}
		return true;
	};

	FToolMenuSection& Section = Menu->AddSection("CreateBlueprint");

	// Create a new GameMode, this is always available so the user can easily create a new one
	Section.AddSubMenu("CreateBlueprint", LOCTEXT("CreateBlueprint", "Create..."),
		GetCreateMenuTooltip(GetGameModeClass(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings), InSettingsData.RootClass, InSettingsData.bIsProjectSettings),
		FNewToolMenuDelegate::CreateStatic( &LevelEditorActionHelpers::GetCreateSettingsClassSubMenu, InSettingsData.RootClass, InSettingsData.OnCreateClassPicked ),
		FUIAction(
			FExecuteAction(), 
			InSettingsData.RootClass == AGameModeBase::StaticClass()? 
				FCanExecuteAction::CreateStatic(CannotCreateSelectNativeProjectGameMode, InSettingsData.bIsProjectSettings) 
				: FCanExecuteAction::CreateStatic( &CanCreateSelectSubClass, GetGameModeClass(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings), InSettingsData.bIsProjectSettings )
		),
		EUserInterfaceActionType::Button, false, NewBPIcon
	);

	// Select a game mode, this is always available so the user can switch their selection
	FFormatNamedArguments Args;
	Args.Add(TEXT("RootClass"), RootClassName);
	Section.AddSubMenu("SelectGameModeClass", FText::Format(LOCTEXT("SelectGameModeClass", "Select {RootClass} Class"), Args),
		GetSelectMenuTooltip(GetGameModeClass(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings), InSettingsData.RootClass, InSettingsData.bIsProjectSettings),
		FNewToolMenuDelegate::CreateStatic( &LevelEditorActionHelpers::GetSelectSettingsClassSubMenu, InSettingsData.RootClass, InSettingsData.OnSelectClassPicked ),
		FUIAction(
			FExecuteAction(), 
			InSettingsData.RootClass == AGameModeBase::StaticClass()?
				FCanExecuteAction::CreateStatic(CannotCreateSelectNativeProjectGameMode, InSettingsData.bIsProjectSettings) 
				: FCanExecuteAction::CreateStatic( &CanCreateSelectSubClass, GetGameModeClass(InSettingsData.LevelEditor, InSettingsData.bIsProjectSettings), InSettingsData.bIsProjectSettings )
		),
		EUserInterfaceActionType::Button
	);

	// For GameMode classes only, there are some sub-classes we need to add to the menu
	if(InSettingsData.RootClass == AGameModeBase::StaticClass())
	{
		FToolMenuSection& GameModeClassesSection = Menu->AddSection("GameModeClasses", LOCTEXT("GameModeClasses", "Game Mode Classes"));
		if(InSettingsData.CurrentClass)
		{
			GetGameModeSubMenu(GameModeClassesSection, InSettingsData);
		}
	}

#undef LOCTEXT_NAMESPACE
}

UClass* LevelEditorActionHelpers::GetGameModeClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	UClass* GameModeClass = nullptr;
	if(bInIsProjectSettings)
	{
		UObject* GameModeObject = LoadObject<UObject>(nullptr, *UGameMapsSettings::GetGlobalDefaultGameMode());
		if(UBlueprint* GameModeAsBlueprint = Cast<UBlueprint>(GameModeObject))
		{
			GameModeClass = GameModeAsBlueprint->GeneratedClass;
		}
		else
		{
			GameModeClass = FindObject<UClass>(nullptr, *UGameMapsSettings::GetGlobalDefaultGameMode());
		}
	}
	else
	{
		AWorldSettings* WorldSettings = InLevelEditor.Pin()->GetWorld()->GetWorldSettings();
		if(WorldSettings->DefaultGameMode)
		{
			GameModeClass = WorldSettings->DefaultGameMode;
		}
	}
	return GameModeClass;
}

FText LevelEditorActionHelpers::GetOpenGameModeBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
	if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
	{
		if(GameModeClass->ClassGeneratedBy)
		{
			return FText::Format( LOCTEXT("GameModeEditBlueprint", "GameMode: Edit {0}"), FText::FromString(GameModeClass->ClassGeneratedBy->GetName()));
		}

		return FText::Format( LOCTEXT("GameModeBlueprint", "GameMode: {0}"), FText::FromString(GameModeClass->GetName()));
	}

	if(bInIsProjectSettings)
	{
		return LOCTEXT("GameModeCreateBlueprint", "GameMode: New...");
	}

	// For World Settings, we want to inform the user that they are not overridding the Project Settings
	return LOCTEXT("GameModeNotOverridden", "GameMode: Not overridden!");

#undef LOCTEXT_NAMESPACE
}

void LevelEditorActionHelpers::OnCreateGameModeClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(InChosenClass)
	{
		const FString NewBPName(TEXT("NewGameMode"));
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(NSLOCTEXT("LevelEditorCommands", "CreateGameModeBlueprint_Title", "Create GameMode Blueprint"), InChosenClass, NewBPName);

		if(Blueprint)
		{
			// @todo Re-enable once world centric works
			const bool bOpenWorldCentric = false;
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
				Blueprint,
				bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
				InLevelEditor.Pin()  );
			OnSelectGameModeClassPicked(Blueprint->GeneratedClass, InLevelEditor, bInIsProjectSettings);
		}
	}
	FSlateApplication::Get().DismissAllMenus();
}

void LevelEditorActionHelpers::OnSelectGameModeClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(bInIsProjectSettings)
	{
		UGameMapsSettings::SetGlobalDefaultGameMode(InChosenClass? InChosenClass->GetPathName() : FString());

		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			ISettingsContainerPtr SettingsContainer = SettingsModule->GetContainer("Project");

			if (SettingsContainer.IsValid())
			{
				ISettingsCategoryPtr SettingsCategory = SettingsContainer->GetCategory("Project");

				if(SettingsCategory.IsValid())
				{
					SettingsCategory->GetSection("Maps")->Save();
				}
			}
		}
	}
	else
	{
		const FScopedTransaction Transaction( NSLOCTEXT("LevelEditorCommands", "SelectGameModeClassAction", "Set Override Game Mode Class") );

		AWorldSettings* WorldSettings = InLevelEditor.Pin()->GetWorld()->GetWorldSettings();
		WorldSettings->Modify();
		WorldSettings->DefaultGameMode = InChosenClass;
	}
	FSlateApplication::Get().DismissAllMenus();
}

UClass* LevelEditorActionHelpers::GetGameStateClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
	{
		AGameModeBase* ActiveGameMode = Cast<AGameModeBase>(GameModeClass->GetDefaultObject());
		if(ActiveGameMode)
		{
			return ActiveGameMode->GameStateClass;
		}
	}
	return NULL;
}

FText LevelEditorActionHelpers::GetOpenGameStateBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
	if(UClass* GameStateClass = GetGameStateClass(InLevelEditor, bInIsProjectSettings))
	{
		FFormatNamedArguments FormatArgs;
		if(GameStateClass->ClassGeneratedBy)
		{
			FormatArgs.Add(TEXT("GameStateName"), FText::FromString(GameStateClass->ClassGeneratedBy->GetName()));
			return FText::Format(LOCTEXT("GameStateEditBlueprint", "GameState: Edit {GameStateName}"), FormatArgs);
		}

		FormatArgs.Add(TEXT("GameStateName"), FText::FromString(GameStateClass->GetName()));
		return FText::Format(LOCTEXT("GameStateBlueprint", "GameState: {GameStateName}"), FormatArgs);
	}

	return LOCTEXT("GameStateCreateBlueprint", "GameState: New...");
#undef LOCTEXT_NAMESPACE
}

void LevelEditorActionHelpers::OnCreateGameStateClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(InChosenClass)
	{
		const FString NewBPName(TEXT("NewGameState"));
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(NSLOCTEXT("LevelEditorCommands", "CreateGameStateBlueprint_Title", "Create GameState Blueprint"), InChosenClass, NewBPName);

		if(Blueprint)
		{
			// @todo Re-enable once world centric works
			const bool bOpenWorldCentric = false;
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
				Blueprint,
				bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
				InLevelEditor.Pin()  );

			OnSelectGameStateClassPicked(Blueprint->GeneratedClass, InLevelEditor, bInIsProjectSettings);
		}
	}
	FSlateApplication::Get().DismissAllMenus();
}

void LevelEditorActionHelpers::OnSelectGameStateClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("LevelEditorCommands", "SelectGameStateClassAction", "Set Game State Class") );
		AGameModeBase* ActiveGameMode = Cast<AGameModeBase>(GameModeClass->GetDefaultObject());
		ActiveGameMode->GameStateClass = InChosenClass;

		UBlueprint* Blueprint = Cast<UBlueprint>(GameModeClass->ClassGeneratedBy);
		if (ensure(Blueprint))
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}

	FSlateApplication::Get().DismissAllMenus();
}

UClass* LevelEditorActionHelpers::GetPawnClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
	{
		AGameModeBase* ActiveGameMode = Cast<AGameModeBase>(GameModeClass->GetDefaultObject());

		if(ActiveGameMode)
		{
			return ActiveGameMode->DefaultPawnClass;
		}
	}
	return NULL;
}

FText LevelEditorActionHelpers::GetOpenPawnBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
	if(UClass* PawnClass = GetPawnClass(InLevelEditor, bInIsProjectSettings))
	{
		FFormatNamedArguments FormatArgs;
		if(PawnClass->ClassGeneratedBy)
		{
			FormatArgs.Add(TEXT("PawnName"), FText::FromString(PawnClass->ClassGeneratedBy->GetName()));
			return FText::Format(LOCTEXT("PawnEditBlueprint", "Pawn: Edit {PawnName}"), FormatArgs);
		}

		FormatArgs.Add(TEXT("PawnName"), FText::FromString(PawnClass->GetName()));
		return FText::Format(LOCTEXT("PawnBlueprint", "Pawn: {PawnName}"), FormatArgs);
	}

	return LOCTEXT("PawnCreateBlueprint", "Pawn: New...");
#undef LOCTEXT_NAMESPACE
}

void LevelEditorActionHelpers::OnCreatePawnClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(InChosenClass)
	{
		const FString NewBPName(TEXT("NewPawn"));
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(NSLOCTEXT("LevelEditorCommands", "CreatePawnBlueprint_Title", "Create Pawn Blueprint"), InChosenClass, NewBPName);

		if(Blueprint)
		{
			// @todo Re-enable once world centric works
			const bool bOpenWorldCentric = false;
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
				Blueprint,
				bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
				InLevelEditor.Pin()  );

			OnSelectPawnClassPicked(Blueprint->GeneratedClass, InLevelEditor, bInIsProjectSettings);
		}
	}
	FSlateApplication::Get().DismissAllMenus();
}

void LevelEditorActionHelpers::OnSelectPawnClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("LevelEditorCommands", "SelectPawnClassAction", "Set Pawn Class") );

		AGameModeBase* ActiveGameMode = Cast<AGameModeBase>(GameModeClass->GetDefaultObject());
		ActiveGameMode->DefaultPawnClass = InChosenClass;

		UBlueprint* Blueprint = Cast<UBlueprint>(GameModeClass->ClassGeneratedBy);
		if (ensure(Blueprint))
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}

	FSlateApplication::Get().DismissAllMenus();
}

UClass* LevelEditorActionHelpers::GetHUDClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
	{
		AGameModeBase* ActiveGameMode = Cast<AGameModeBase>(GameModeClass->GetDefaultObject());
		if(ActiveGameMode)
		{
			return ActiveGameMode->HUDClass;
		}
	}
	return NULL;
}

FText LevelEditorActionHelpers::GetOpenHUDBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
	if(UClass* HUDClass = GetHUDClass(InLevelEditor, bInIsProjectSettings))
	{
		FFormatNamedArguments FormatArgs;
		if (HUDClass->ClassGeneratedBy)
		{
			FormatArgs.Add(TEXT("HUDName"), FText::FromString(HUDClass->ClassGeneratedBy->GetName()));
			return FText::Format(LOCTEXT("HUDEditBlueprint", "HUD: Edit {HUDName}"), FormatArgs);
		}

		FormatArgs.Add(TEXT("HUDName"), FText::FromString(HUDClass->GetName()));
		return FText::Format(LOCTEXT("HUDBlueprint", "HUD: {HUDName}"), FormatArgs);
	}

	return LOCTEXT("HUDCreateBlueprint", "HUD: New...");
#undef LOCTEXT_NAMESPACE
}

void LevelEditorActionHelpers::OnCreateHUDClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(InChosenClass)
	{
		const FString NewBPName(TEXT("NewHUD"));
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(NSLOCTEXT("LevelEditorCommands", "CreateHUDBlueprint_Title", "Create HUD Blueprint"), InChosenClass, NewBPName);

		if(Blueprint)
		{
			// @todo Re-enable once world centric works
			const bool bOpenWorldCentric = false;
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
				Blueprint,
				bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
				InLevelEditor.Pin()  );

			OnSelectHUDClassPicked(Blueprint->GeneratedClass, InLevelEditor, bInIsProjectSettings);
		}
	}
	FSlateApplication::Get().DismissAllMenus();
}

void LevelEditorActionHelpers::OnSelectHUDClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("LevelEditorCommands", "SelectHUDClassAction", "Set HUD Class") );

		AGameModeBase* ActiveGameMode = Cast<AGameModeBase>(GameModeClass->GetDefaultObject());
		ActiveGameMode->HUDClass = InChosenClass;

		UBlueprint* Blueprint = Cast<UBlueprint>(GameModeClass->ClassGeneratedBy);
		if (ensure(Blueprint))
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}

	FSlateApplication::Get().DismissAllMenus();
}

UClass* LevelEditorActionHelpers::GetPlayerControllerClass(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
	{
		AGameModeBase* ActiveGameMode = Cast<AGameModeBase>(GameModeClass->GetDefaultObject());
		if(ActiveGameMode)
		{
			return ActiveGameMode->PlayerControllerClass;
		}
	}
	return NULL;
}

FText LevelEditorActionHelpers::GetOpenPlayerControllerBlueprintLabel(TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
	if(UClass* PlayerControllerClass = GetPlayerControllerClass(InLevelEditor, bInIsProjectSettings))
	{
		FFormatNamedArguments FormatArgs;
		if (PlayerControllerClass->ClassGeneratedBy)
		{
			FormatArgs.Add(TEXT("PlayerControllerName"), FText::FromString(PlayerControllerClass->ClassGeneratedBy->GetName()));
			return FText::Format(LOCTEXT("PlayerControllerEditBlueprint", "PlayerController: Edit {PlayerControllerName}"), FormatArgs);
		}

		FormatArgs.Add(TEXT("PlayerControllerName"), FText::FromString(PlayerControllerClass->GetName()));
		return FText::Format(LOCTEXT("PlayerControllerBlueprint", "PlayerController: {PlayerControllerName}"), FormatArgs);
	}

	return LOCTEXT("PlayerControllerCreateBlueprint", "PlayerController: New...");
#undef LOCTEXT_NAMESPACE
}

void LevelEditorActionHelpers::OnCreatePlayerControllerClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(InChosenClass)
	{
		const FString NewBPName(TEXT("NewPlayerController"));
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(NSLOCTEXT("LevelEditorCommands", "CreatePlayerControllerBlueprint_Title", "Create PlayerController Blueprint"), InChosenClass, NewBPName);

		if(Blueprint)
		{
			// @todo Re-enable once world centric works
			const bool bOpenWorldCentric = false;
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(
				Blueprint,
				bOpenWorldCentric ? EToolkitMode::WorldCentric : EToolkitMode::Standalone,
				InLevelEditor.Pin()  );

			OnSelectPlayerControllerClassPicked(Blueprint->GeneratedClass, InLevelEditor, bInIsProjectSettings);
		}
	}
	FSlateApplication::Get().DismissAllMenus();
}

void LevelEditorActionHelpers::OnSelectPlayerControllerClassPicked(UClass* InChosenClass, TWeakPtr< SLevelEditor > InLevelEditor, bool bInIsProjectSettings)
{
	if(UClass* GameModeClass = GetGameModeClass(InLevelEditor, bInIsProjectSettings))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("LevelEditorCommands", "SelectPlayerControllerClassAction", "Set Player Controller Class") );

		AGameModeBase* ActiveGameMode = Cast<AGameModeBase>(GameModeClass->GetDefaultObject());
		ActiveGameMode->PlayerControllerClass = InChosenClass;

		UBlueprint* Blueprint = Cast<UBlueprint>(GameModeClass->ClassGeneratedBy);
		if (ensure(Blueprint))
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
	FSlateApplication::Get().DismissAllMenus();
}

FText FLevelEditorToolBar::GetActiveModeName(TWeakPtr<SLevelEditor> LevelEditorPtr)
{
#define LOCTEXT_NAMESPACE "LevelEditorToolBar"

	for (const FEditorModeInfo& Mode : GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetEditorModeInfoOrderedByPriority())
	{
		TSharedPtr<SLevelEditor> LevelEditorPin = LevelEditorPtr.Pin();
		if (LevelEditorPin.IsValid() && LevelEditorPin->GetEditorModeManager().IsModeActive(Mode.ID) && Mode.IsVisible())
		{
			return FText::Format(LOCTEXT("ActiveMode", "{0} Mode"), Mode.Name);
		}
	}

	return LOCTEXT("NoActiveMode", "No Active Mode");

#undef LOCTEXT_NAMESPACE
}

const FSlateBrush* FLevelEditorToolBar::GetActiveModeIcon(TWeakPtr<SLevelEditor> LevelEditorPtr)
{
	for (const FEditorModeInfo& Mode : GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetEditorModeInfoOrderedByPriority())
	{
		TSharedPtr<SLevelEditor> LevelEditorPin = LevelEditorPtr.Pin();
		if (LevelEditorPin.IsValid() && LevelEditorPin->GetEditorModeManager().IsModeActive(Mode.ID) && Mode.IsVisible())
		{
			return Mode.IconBrush.GetIcon();
		}
	}
	return nullptr;
}

void FLevelEditorToolBar::RegisterLevelEditorToolBar( const TSharedRef<FUICommandList>& InCommandList, const TSharedRef<SLevelEditor> InLevelEditor)
{
	static bool bHasRegistered = false;
	if (!bHasRegistered)
	{
		bHasRegistered = true;

		RegisterSourceControlMenu();
		RegisterCinematicsMenu();

		RegisterQuickSettingsMenu();
		RegisterOpenBlueprintMenu();
		RegisterAddMenu();
	}

#define LOCTEXT_NAMESPACE "LevelEditorToolBar"

	UToolMenu* ModesToolbar = UToolMenus::Get()->RegisterMenu("LevelEditor.LevelEditorToolBar.ModesToolBar", NAME_None, EMultiBoxType::SlimHorizontalToolBar, /*warn*/false);
	ModesToolbar->StyleName = "AssetEditorToolbar";
	{
		{
			FToolMenuSection& Section = ModesToolbar->AddSection("File");

			// Save All Levels
			Section.AddEntry(FToolMenuEntry::InitToolBarButton(
				FLevelEditorCommands::Get().Save,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset"),
				NAME_None,
				FName("SaveAllLevels")
			));

			// Browse Level
			Section.AddEntry(FToolMenuEntry::InitToolBarButton(
				FLevelEditorCommands::Get().BrowseLevel,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser")
			));
		}

		TWeakPtr<SLevelEditor> LevelEditorPtr = InLevelEditor;
		ModesToolbar->AddDynamicSection("EditorModes", FNewToolMenuDelegate::CreateLambda([LevelEditorPtr](UToolMenu* ToolMenu)
			{
				FToolMenuSection& Section = ToolMenu->AddSection("EditorModes");
				
				// Combo Button to swap editor modes
				TSharedRef<SComboButton> EditorModesComboButton = SNew(SComboButton)
					.OnGetMenuContent_Lambda([LevelEditorPtr]()
					{
						const FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
						const FLevelEditorModesCommands& Commands = LevelEditorModule.GetLevelEditorModesCommands();

						TArray<FEditorModeInfo> DefaultModes;
						TArray<FEditorModeInfo> NonDefaultModes;
						TArray<TSharedPtr<FUICommandInfo>> CommandInfos;

						for (const FEditorModeInfo& Mode : GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetEditorModeInfoOrderedByPriority())
						{
							TSharedPtr<SLevelEditor> LevelEditorPin = LevelEditorPtr.Pin();
							if (LevelEditorPin.IsValid() && LevelEditorPin->GetEditorModeManager().IsDefaultMode(Mode.ID))
							{
								DefaultModes.Add(Mode);
							}
							else
							{
								NonDefaultModes.Add(Mode);
							}

						}

						auto GetCommandForModes = [&CommandInfos, &Commands](TArrayView<FEditorModeInfo> Modes)
						{
							for (const FEditorModeInfo& Mode : Modes)
							{
								FName EditorModeCommandName = FName(*(FString("EditorMode.") + Mode.ID.ToString()));

								TSharedPtr<FUICommandInfo> EditorModeCommand =
									FInputBindingManager::Get().FindCommandInContext(Commands.GetContextName(), EditorModeCommandName);

								if (Mode.IsVisible() && EditorModeCommand.IsValid())
								{
									CommandInfos.Add(EditorModeCommand);
								}
									
							}
						};

						// Default Modes first
						GetCommandForModes(DefaultModes);

						GetCommandForModes(NonDefaultModes);

						FMenuBuilder MenuBuilder(true, LevelEditorModule.GetGlobalLevelEditorActions());

						TSharedPtr<SLevelEditor> LevelEditorPin = LevelEditorPtr.Pin();
						if (LevelEditorPin.IsValid())
						{
							MenuBuilder.PushCommandList(LevelEditorPin->GetLevelEditorActions().ToSharedRef());
						}

						MenuBuilder.BeginSection("EditorModes");

						for (TSharedPtr<FUICommandInfo> Command : CommandInfos)
						{
							MenuBuilder.AddMenuEntry(Command);
						}

						MenuBuilder.EndSection();

						return MenuBuilder.MakeWidget();
					})
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(0.f, 0.f, 6.f, 0.f))
						[
							SNew(SBox)
							.WidthOverride(16.f)
							.HeightOverride(16.f)
							[
								SNew(SImage)
								.Image_Static(&FLevelEditorToolBar::GetActiveModeIcon, LevelEditorPtr)
							]
							
						]
						+ SHorizontalBox::Slot()
						[
							SNew(STextBlock)
							.Text_Static(&FLevelEditorToolBar::GetActiveModeName, LevelEditorPtr)
						]
						
					];

				// Horizontal Box to add some spacing beside the modes combo button
				TSharedRef<SHorizontalBox> EditorModesWidget = 
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					[
						SNew(SSpacer)
						.Size(FVector2D(10.f, 1.0f))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						EditorModesComboButton
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					[
						SNew(SSpacer)
						.Size(FVector2D(10.f, 1.0f))
					];
					

				Section.AddEntry(FToolMenuEntry::InitWidget("Editor Modes", EditorModesWidget, LOCTEXT("EditorModesLabel", "Editor Modes")));

				Section.AddSeparator(NAME_None);
			}));
	}

	UToolMenu* AssetsToolBar = UToolMenus::Get()->RegisterMenu("LevelEditor.LevelEditorToolBar.AssetsToolBar", NAME_None, EMultiBoxType::SlimHorizontalToolBar, false);
	AssetsToolBar->StyleName = "AssetEditorToolbar";
	{
		{
			FToolMenuSection& Section = AssetsToolBar->AddSection("Content");

			FToolMenuEntry AddContentEntry = FToolMenuEntry::InitComboButton(
				"AddContent",
				FUIAction(),
				FOnGetContent::CreateStatic(&FLevelEditorToolBar::GenerateAddMenuWidget, InCommandList, TWeakPtr<SLevelEditor>(InLevelEditor)),
				LOCTEXT("AddContent_Label", "Add"),
				LOCTEXT("AddContent_Tooltip", "Quickly add to the project."),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.OpenAddContent.Background", NAME_None, "LevelEditor.OpenAddContent.Overlay")
			);
			AddContentEntry.StyleNameOverride = "AssetEditorToolbar";
			Section.AddEntry(AddContentEntry);

			FToolMenuEntry BlueprintEntry = FToolMenuEntry::InitComboButton(
				"OpenBlueprint",
				FUIAction(),
				FOnGetContent::CreateStatic(&FLevelEditorToolBar::GenerateOpenBlueprintMenuContent, InCommandList, TWeakPtr<SLevelEditor>(InLevelEditor)),
				LOCTEXT("OpenBlueprint_Label", "Blueprints"),
				LOCTEXT("OpenBlueprint_ToolTip", "List of world Blueprints available to the user for editing or creation."),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.CreateBlankBlueprintClass")
			);
			BlueprintEntry.StyleNameOverride = "AssetEditorToolbar";
			Section.AddEntry(BlueprintEntry);

			FToolMenuEntry CinematicsEntry = FToolMenuEntry::InitComboButton(
				"EditCinematics",
				FUIAction(),
				FOnGetContent::CreateStatic(&FLevelEditorToolBar::GenerateCinematicsMenuContent, InCommandList, TWeakPtr<SLevelEditor>(InLevelEditor)),
				LOCTEXT("EditCinematics_Label", "Cinematics"),
				LOCTEXT("EditCinematics_Tooltip", "Displays a list of Level Sequence objects to open in their respective editors"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.OpenCinematic")
			);
			CinematicsEntry.StyleNameOverride = "AssetEditorToolbar";
			Section.AddEntry(CinematicsEntry);
		}

	}

	UToolMenu* PlayToolBar = UToolMenus::Get()->RegisterMenu("LevelEditor.LevelEditorToolBar.PlayToolBar", NAME_None, EMultiBoxType::SlimHorizontalToolBar, false);
	PlayToolBar->StyleName = "AssetEditorToolbar";
	{
		FToolMenuSection& PlaySection = PlayToolBar->AddSection("Play");

		PlaySection.AddSeparator(NAME_None);

		PreviewModeFunctionality::AddPreviewToggleButton(PlaySection);

		// Add the shared play-world commands that will be shown on the Kismet toolbar as well
		FPlayWorldCommands::BuildToolbar(PlaySection);


	}

	UToolMenu* UserToolbar = UToolMenus::Get()->RegisterMenu("LevelEditor.LevelEditorToolBar.User", NAME_None, EMultiBoxType::SlimHorizontalToolBar, false);
	UserToolbar->StyleName = "AssetEditorToolbar";

	UToolMenu* SettingsToolbar = UToolMenus::Get()->RegisterMenu("LevelEditor.LevelEditorToolBar.SettingsToolbar", NAME_None, EMultiBoxType::SlimHorizontalToolBar, false);
	SettingsToolbar->StyleName = "AssetEditorToolbar";
	{
		FToolMenuSection& SettingsSection = SettingsToolbar->AddSection("ProjectSettings");

		FUIAction Action;
		Action.IsActionVisibleDelegate = FIsActionButtonVisible::CreateLambda(
			[]() -> bool
			{
				return UE::Editor::Private::CVarLevelEditorToolbarSettingsValue != 0;
			}
		);

		FToolMenuEntry SettingsEntry = FToolMenuEntry::InitComboButton(
			"LevelToolbarQuickSettings",
			Action,
			FOnGetContent::CreateStatic(
				&FLevelEditorToolBar::GenerateQuickSettingsMenu, InCommandList, TWeakPtr<SLevelEditor>(InLevelEditor)
			),
			LOCTEXT("QuickSettingsCombo", "Settings"),
			LOCTEXT("QuickSettingsCombo_ToolTip", "Project and Editor settings"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings"),
			false,
			"LevelToolbarQuickSettings"
		);
		SettingsEntry.StyleNameOverride = "CalloutToolbar";

		SettingsSection.AddEntry(SettingsEntry);
	}
#undef LOCTEXT_NAMESPACE
}

FName FLevelEditorToolBar::GetSecondaryModeToolbarName()
{
	return SecondaryModeToolbarName;
}

TSharedRef< SWidget > FLevelEditorToolBar::MakeLevelEditorSecondaryModeToolbar( TSharedRef<FUICommandList> InCommandList, TMap<FName, TSharedPtr<FLevelEditorModeUILayer>>& ModeUILayers )
{
	FToolMenuContext MenuContext(InCommandList);

	for(const TPair<FName, TSharedPtr<FLevelEditorModeUILayer>>& ModeUILayer : ModeUILayers)
	{
		MenuContext.AppendCommandList(ModeUILayer.Value->GetModeCommands());
	}

	return SNew(SBorder)
	.Padding(0.f)
	.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
	.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
	[
		UToolMenus::Get()->GenerateWidget(SecondaryModeToolbarName, MenuContext)
	];

}

void FLevelEditorToolBar::RegisterLevelEditorSecondaryModeToolbar()
{
	UToolMenu* ModesToolbar = UToolMenus::Get()->RegisterMenu(SecondaryModeToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar, false);
	ModesToolbar->StyleName = "SecondaryToolbar";
}

/**
 * Static: Creates a widget for the level editor tool bar
 *
 * @return	New widget
 */
TSharedRef< SWidget > FLevelEditorToolBar::MakeLevelEditorToolBar( const TSharedRef<FUICommandList>& InCommandList, const TSharedRef<SLevelEditor> InLevelEditor )
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FToolMenuContext MenuContext(InCommandList, LevelEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	ULevelEditorMenuContext* LevelEditorMenuContext = NewObject<ULevelEditorMenuContext>();
	LevelEditorMenuContext->LevelEditor = InLevelEditor;
	MenuContext.AddObject(LevelEditorMenuContext);

	TWeakPtr<SLevelEditor> LevelEditorWeakPtr(InLevelEditor);

	// Create the tool bar!
	return
		SNew(SOverlay)
		+SOverlay::Slot()
		[
			SNew(SImage)
			.Image(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar").BackgroundBrush)
		]
		+ SOverlay::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				[
					UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.ModesToolBar", MenuContext)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				[
					UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.AssetsToolBar", MenuContext)
				]
				
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
				[
					UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.PlayToolBar", MenuContext) // Always enabled
				]
				
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.FillWidth(1.0f)
			[
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				[
					UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.User", MenuContext)
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(0.0f, 0.0f, 7.0f, 0.0f)
			[
				SNew(SBorder)
				.Visibility_Lambda([]
				{
					return UE::Editor::Private::CVarLevelEditorToolbarSettingsValue != 0 ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.Padding(0.f)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				[
					UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.SettingsToolBar", MenuContext)
				]
			]
		];
}

static void MakeScalabilityMenu( UToolMenu* InMenu )
{
	{
		FToolMenuSection& Section = InMenu->AddSection("Section");
		Section.AddEntry(FToolMenuEntry::InitWidget("ScalabilitySettings", SNew(SScalabilitySettings), FText(), true));
	}
}

static void MakePreviewSettingsMenu( UToolMenu* InMenu )
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
	{
		FToolMenuSection& Section = InMenu->AddSection("LevelEditorPreview", LOCTEXT("PreviewHeading", "Previewing"));
		Section.AddMenuEntry(FLevelEditorCommands::Get().DrawBrushMarkerPolys);
		Section.AddMenuEntry(FLevelEditorCommands::Get().OnlyLoadVisibleInPIE);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleParticleSystemLOD);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleParticleSystemHelpers);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleFreezeParticleSimulation);
		Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleLODViewLocking);
		Section.AddMenuEntry(FLevelEditorCommands::Get().LevelStreamingVolumePrevis);
	}
#undef LOCTEXT_NAMESPACE
}



TSharedRef< SWidget > FLevelEditorToolBar::GenerateQuickSettingsMenu(TSharedRef<FUICommandList> InCommandList, TWeakPtr<SLevelEditor> InLevelEditor)
{
	// Get all menu extenders for this context menu from the level editor module
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	TSharedPtr<FExtender> MenuExtender = LevelEditorModule.AssembleExtenders(InCommandList, LevelEditorModule.GetAllLevelEditorToolbarViewMenuExtenders());

	FToolMenuContext MenuContext(InCommandList, MenuExtender);
	ULevelEditorMenuContext* LevelEditorMenuContext = NewObject<ULevelEditorMenuContext>();
	LevelEditorMenuContext->LevelEditor = InLevelEditor;
	MenuContext.AddObject(LevelEditorMenuContext);

	return UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.LevelToolbarQuickSettings", MenuContext);
}

void FLevelEditorToolBar::RegisterQuickSettingsMenu()
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
	if (UToolMenus::Get()->IsMenuRegistered("LevelEditor.LevelEditorToolBar.LevelToolbarQuickSettings"))
	{
		return;
	}

	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("LevelEditor.LevelEditorToolBar.LevelToolbarQuickSettings");

	struct Local
	{
		static void OpenSettings(FName ContainerName, FName CategoryName, FName SectionName)
		{
			FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(ContainerName, CategoryName, SectionName);
		}
	};

	{
		FToolMenuSection& Section = Menu->AddSection("ProjectSettingsSection", LOCTEXT("ProjectSettings", "Game Specific Settings"));

		Section.AddMenuEntry(FLevelEditorCommands::Get().WorldProperties);

		Section.AddMenuEntry(
			"ProjectSettings",
			LOCTEXT("ProjectSettingsMenuLabel", "Project Settings..."),
			LOCTEXT("ProjectSettingsMenuToolTip", "Change the settings of the currently loaded project"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon"),
			FUIAction(FExecuteAction::CreateStatic(&Local::OpenSettings, FName("Project"), FName("Project"), FName("General")))
			);

		Section.AddDynamicEntry("PluginsEditor", FNewToolMenuDelegateLegacy::CreateLambda([](FMenuBuilder& InMenuBuilder, UToolMenu* InMenu)
		{
			if (IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				FGlobalTabmanager::Get()->PopulateTabSpawnerMenu(InMenuBuilder, "PluginsEditor");
			}
		}));
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorSelection", LOCTEXT("SelectionHeading","Selection") );
		Section.AddMenuEntry( FLevelEditorCommands::Get().AllowTranslucentSelection );
		Section.AddMenuEntry( FLevelEditorCommands::Get().AllowGroupSelection );
		Section.AddMenuEntry( FLevelEditorCommands::Get().StrictBoxSelect );
		Section.AddMenuEntry( FLevelEditorCommands::Get().TransparentBoxSelect );
		Section.AddMenuEntry( FLevelEditorCommands::Get().ShowTransformWidget );
		Section.AddMenuEntry( FLevelEditorCommands::Get().ShowSelectionSubcomponents );
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorScalability", LOCTEXT("ScalabilityHeading", "Scalability") );
		Section.AddSubMenu(
			"Scalability",
			LOCTEXT( "ScalabilitySubMenu", "Engine Scalability Settings" ),
			LOCTEXT( "ScalabilitySubMenu_ToolTip", "Open the engine scalability settings" ),
			FNewToolMenuDelegate::CreateStatic( &MakeScalabilityMenu ) );

		Section.AddEntry(UE::LevelEditor::CreateMaterialQualityLevelSubmenu());
		Section.AddEntry(UE::LevelEditor::CreateFeatureLevelPreviewSubmenu());
	}

	{
		FToolMenuSection& Section = Menu->AddSection( "Snapping", LOCTEXT("SnappingHeading","Snapping") );
		Section.AddMenuEntry( FLevelEditorCommands::Get().EnableActorSnap );
		TSharedRef<SWidget> SnapItem = 
		SNew(SHorizontalBox)
	          +SHorizontalBox::Slot()
	          .FillWidth(0.9f)
	          [
		          SNew(SSlider)
		          .ToolTipText_Static(&FLevelEditorActionCallbacks::GetActorSnapTooltip)
		          .Value_Static(&FLevelEditorActionCallbacks::GetActorSnapSetting)
		          .OnValueChanged_Static(&FLevelEditorActionCallbacks::SetActorSnapSetting)
	          ]
	          +SHorizontalBox::Slot()
	          .FillWidth(0.1f);
		Section.AddEntry(FToolMenuEntry::InitWidget("Snap", SnapItem, LOCTEXT("ActorSnapLabel", "Distance")));

		Section.AddMenuEntry( FLevelEditorCommands::Get().ToggleSocketSnapping );
		Section.AddMenuEntry( FLevelEditorCommands::Get().EnableVertexSnap );
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelEditorViewport", LOCTEXT("ViewportHeading", "Viewport") );
		Section.AddMenuEntry( FLevelEditorCommands::Get().ToggleHideViewportUI );

		Section.AddSubMenu( "Preview", LOCTEXT("PreviewMenu", "Previewing"), LOCTEXT("PreviewMenuTooltip","Game Preview Settings"), FNewToolMenuDelegate::CreateStatic( &MakePreviewSettingsMenu ) );
	}

#undef LOCTEXT_NAMESPACE
}


TSharedRef< SWidget > FLevelEditorToolBar::GenerateSourceControlMenu(TSharedRef<FUICommandList> InCommandList, TWeakPtr< SLevelEditor > InLevelEditor)
{
	// Get all menu extenders for this context menu from the level editor module
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	TSharedPtr<FExtender> MenuExtender = LevelEditorModule.AssembleExtenders(InCommandList, LevelEditorModule.GetAllLevelEditorToolbarSourceControlMenuExtenders());

	FToolMenuContext MenuContext(InCommandList, MenuExtender);
	ULevelEditorMenuContext* LevelEditorMenuContext = NewObject<ULevelEditorMenuContext>();
	LevelEditorMenuContext->LevelEditor = InLevelEditor;
	MenuContext.AddObject(LevelEditorMenuContext);

	return UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.SourceControl", MenuContext);
}

void FLevelEditorToolBar::RegisterSourceControlMenu()
{
#define LOCTEXT_NAMESPACE "LevelToolBarSourceControlMenu"



	
#undef LOCTEXT_NAMESPACE
}

TSharedRef< SWidget > FLevelEditorToolBar::GenerateOpenBlueprintMenuContent( TSharedRef<FUICommandList> InCommandList, TWeakPtr< SLevelEditor > InLevelEditor )
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(LevelEditorModule.GetAllLevelEditorToolbarBlueprintsMenuExtenders());

	FToolMenuContext MenuContext(InCommandList, MenuExtender);
	ULevelEditorMenuContext* LevelEditorMenuContext = NewObject<ULevelEditorMenuContext>();
	LevelEditorMenuContext->LevelEditor = InLevelEditor;
	MenuContext.AddObject(LevelEditorMenuContext);

	return UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.OpenBlueprint", MenuContext);
#undef LOCTEXT_NAMESPACE
}

void FLevelEditorToolBar::RegisterOpenBlueprintMenu()
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
	if (UToolMenus::Get()->IsMenuRegistered("LevelEditor.LevelEditorToolBar.OpenBlueprint"))
	{
		return;
	}

	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("LevelEditor.LevelEditorToolBar.OpenBlueprint");

	struct FBlueprintMenus
	{
		/** Generates a sub-level Blueprints sub-menu */
		static void MakeSubLevelsMenu(UToolMenu* InMenu)
		{
			ULevelEditorMenuContext* Context = InMenu->FindContext<ULevelEditorMenuContext>();
			if (Context && Context->LevelEditor.IsValid())
			{
				FSlateIcon EditBP(FAppStyle::Get().GetStyleSetName(), TEXT("LevelEditor.OpenLevelBlueprint"));

				{
					FToolMenuSection& Section = InMenu->AddSection("SubLevels", LOCTEXT("SubLevelsHeading", "Sub-Level Blueprints"));
					UWorld* World = Context->LevelEditor.Pin()->GetWorld();
					// Sort the levels alphabetically 
					TArray<ULevel*> SortedLevels = World->GetLevels();
					Algo::Sort(SortedLevels, LevelEditorActionHelpers::FLevelSortByName());

					for (ULevel* const Level : SortedLevels)
					{
						if (Level != NULL && Level->GetOutermost() != NULL && !Level->IsPersistentLevel() && !Level->IsInstancedLevel())
						{
							FUIAction UIAction
							(
								FExecuteAction::CreateStatic(&FLevelEditorToolBar::OnOpenSubLevelBlueprint, Level)
							);

							FText DisplayName = FText::Format(LOCTEXT("SubLevelBlueprintItem", "Edit {0}"), FText::FromString(FPaths::GetCleanFilename(Level->GetOutermost()->GetName())));
							Section.AddMenuEntry(NAME_None, DisplayName, FText::GetEmpty(), EditBP, UIAction);
						}
					}
				}
			}
		}

		/** Handle BP being selected from popup picker */
		static void OnBPSelected(const struct FAssetData& AssetData)
		{
			UBlueprint* SelectedBP = Cast<UBlueprint>(AssetData.GetAsset());
			if(SelectedBP)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SelectedBP);
			}
		}


		/** Generates 'open blueprint' sub-menu */
		static void MakeOpenBPClassMenu(UToolMenu* InMenu)
		{
			// The contents of this menu are added as a custom widget with its own search field so we
			// disable searching in this parent menu to avoid displaying two search fields to the user.
			InMenu->bSearchable = false;

			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

			// Configure filter for asset picker
			FAssetPickerConfig Config;
			Config.Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
			Config.InitialAssetViewType = EAssetViewType::List;
			Config.OnAssetSelected = FOnAssetSelected::CreateStatic(&FBlueprintMenus::OnBPSelected);
			Config.bAllowDragging = false;
			// Allow saving user defined filters via View Options
			Config.SaveSettingsName = FString(TEXT("ToolbarOpenBPClass"));

			TSharedRef<SWidget> Widget = 
				SNew(SBox)
				.WidthOverride(300.f)
				.HeightOverride(300.f)
				[
					ContentBrowserModule.Get().CreateAssetPicker(Config)
				];
		

			{
				FToolMenuSection& Section = InMenu->AddSection("Browse", LOCTEXT("BrowseHeader", "Browse"));
				Section.AddEntry(FToolMenuEntry::InitWidget("PickClassWidget", Widget, FText::GetEmpty()));
			}
		}
	};

	{
		FToolMenuSection& Section = Menu->AddSection("BlueprintClass", LOCTEXT("BlueprintClass", "Blueprint Class"));

		// Create a blank BP
		Section.AddMenuEntry(FLevelEditorCommands::Get().CreateBlankBlueprintClass);

		// Convert selection to BP
		Section.AddMenuEntry(FLevelEditorCommands::Get().ConvertSelectionToBlueprint);

		// Open an existing Blueprint Class...
		FSlateIcon OpenBPIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.OpenClassBlueprint");
		Section.AddSubMenu(
			"OpenBlueprintClass",
			LOCTEXT("OpenBlueprintClassSubMenu", "Open Blueprint Class..."),
			LOCTEXT("OpenBlueprintClassSubMenu_ToolTip", "Open an existing Blueprint Class in this project"),
			FNewToolMenuDelegate::CreateStatic(&FBlueprintMenus::MakeOpenBPClassMenu),
			false,
			OpenBPIcon);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("LevelScriptBlueprints", LOCTEXT("LevelScriptBlueprints", "Level Blueprints"));
		Section.AddMenuEntry( FLevelEditorCommands::Get().OpenLevelBlueprint );

		Section.AddDynamicEntry("SubLevels", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			ULevelEditorMenuContext* Context = InSection.FindContext<ULevelEditorMenuContext>();
			if (Context && Context->LevelEditor.IsValid())
			{
				// If there are any sub-levels, display the sub-menu. A single level means there is only the persistent level
				UWorld* World = Context->LevelEditor.Pin()->GetWorld();
				if (World->GetNumLevels() > 1)
				{
					InSection.AddSubMenu(
						"SubLevels",
						LOCTEXT("SubLevelsSubMenu", "Sub-Levels"),
						LOCTEXT("SubLevelsSubMenu_ToolTip", "Shows available sub-level Blueprints that can be edited."),
						FNewToolMenuDelegate::CreateStatic(&FBlueprintMenus::MakeSubLevelsMenu),
						FUIAction(), EUserInterfaceActionType::Button, false, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("LevelEditor.OpenLevelBlueprint")));
				}
			}
		}));
	}

	{
		FToolMenuSection& Section = Menu->AddSection("ProjectSettingsClasses", LOCTEXT("ProjectSettingsClasses", "Project Settings"));
		LevelEditorActionHelpers::CreateGameModeSubMenu(Section, "ProjectSettingsClasses", true);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("WorldSettingsClasses", LOCTEXT("WorldSettingsClasses", "World Override"));
		LevelEditorActionHelpers::CreateGameModeSubMenu(Section, "WorldSettingsClasses", false);
	}

	// If source control is enabled, queue up a query to the status of the config file so it is (hopefully) ready before we get to the sub-menu
	if(ISourceControlModule::Get().IsEnabled())
	{
		FString ConfigFilePath = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%sDefaultEngine.ini"), *FPaths::SourceConfigDir()));

		// note: calling QueueStatusUpdate often does not spam status updates as an internal timer prevents this
		ISourceControlModule::Get().QueueStatusUpdate(ConfigFilePath);
	}
#undef LOCTEXT_NAMESPACE
}

void FLevelEditorToolBar::RegisterAddMenu()
{
#define LOCTEXT_NAMESPACE "LevelToolBarViewMenu"
	if (UToolMenus::Get()->IsMenuRegistered("LevelEditor.LevelEditorToolBar.AddQuickMenu"))
	{
		return;
	}

	UToolMenu* AddMenu = UToolMenus::Get()->RegisterMenu("LevelEditor.LevelEditorToolBar.AddQuickMenu");
	{
		FToolMenuSection& Section = AddMenu->FindOrAddSection("Content");

		Section.InitSection("Content", LOCTEXT("Content_Label", "Get Content"), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

		Section.AddMenuEntry(FLevelEditorCommands::Get().ImportContent).InsertPosition.Position = EToolMenuInsertType::First;
	}
#undef LOCTEXT_NAMESPACE
}

void FLevelEditorToolBar::OnOpenSubLevelBlueprint( ULevel* InLevel )
{
	ULevelScriptBlueprint* LevelScriptBlueprint = InLevel->GetLevelScriptBlueprint();

	if( LevelScriptBlueprint )
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LevelScriptBlueprint);
	}
	else
	{
		FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "UnableToCreateLevelScript", "Unable to find or create a level blueprint for this level.") );
	}
}

TSharedRef< SWidget > FLevelEditorToolBar::GenerateCinematicsMenuContent(TSharedRef<FUICommandList> InCommandList, TWeakPtr<SLevelEditor> InLevelEditor)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

	FToolMenuContext MenuContext(InCommandList, FExtender::Combine(LevelEditorModule.GetAllLevelEditorToolbarCinematicsMenuExtenders()));
	ULevelEditorMenuContext* LevelEditorMenuContext = NewObject<ULevelEditorMenuContext>();
	LevelEditorMenuContext->LevelEditor = InLevelEditor;
	MenuContext.AddObject(LevelEditorMenuContext);

	return UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.Cinematics", MenuContext);
}
TSharedRef< SWidget > FLevelEditorToolBar::GenerateAddMenuWidget(TSharedRef<FUICommandList> InCommandList, TWeakPtr<SLevelEditor> InLevelEditor)
{
	FToolMenuContext MenuContext(InCommandList);
	ULevelEditorMenuContext* LevelEditorMenuContext = NewObject<ULevelEditorMenuContext>();
	LevelEditorMenuContext->LevelEditor = InLevelEditor;
	MenuContext.AddObject(LevelEditorMenuContext);

	return UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.AddQuickMenu", MenuContext);
}

void FLevelEditorToolBar::RegisterCinematicsMenu()
{
#define LOCTEXT_NAMESPACE "LevelToolBarCinematicsMenu"
	if (UToolMenus::Get()->IsMenuRegistered("LevelEditor.LevelEditorToolBar.Cinematics"))
	{
		return;
	}

	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("LevelEditor.LevelEditorToolBar.Cinematics");
	Menu->bShouldCloseWindowAfterMenuSelection = true;

	Menu->AddSection("LevelEditorNewCinematics", LOCTEXT("CinematicsMenuCombo_NewHeading", "New"));

	//Add a heading to separate the existing cinematics from the 'Add New Cinematic Actor' button
	FToolMenuSection& ExistingCinematicSection = Menu->AddSection("LevelEditorExistingCinematic", LOCTEXT("CinematicMenuCombo_ExistingHeading", "Edit Existing Cinematic"));
	ExistingCinematicSection.AddDynamicEntry("LevelEditorExistingCinematic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		ULevelEditorMenuContext* FoundContext = InSection.Context.FindContext<ULevelEditorMenuContext>();
		if (!FoundContext)
		{
			return;
		}

		UWorld* World = FoundContext->LevelEditor.IsValid() ? FoundContext->LevelEditor.Pin()->GetWorld() : nullptr;
		const bool bHasAnyCinematicsActors = !!TActorIterator<ALevelSequenceActor>(World);
		if (!bHasAnyCinematicsActors)
		{
			return;
		}

		// We can't build a list of LevelSequenceActors while the current World is a PIE world.
		FSceneOutlinerInitializationOptions InitOptions;
		{
			// We hide the header row to keep the UI compact.
			// @todo: Might be useful to have this sometimes, actually.  Ideally the user could summon it.
			InitOptions.bShowHeaderRow = false;
			InitOptions.bShowSearchBox = true;
			InitOptions.bShowCreateNewFolder = false;

			InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn(), false, 0.6f));
			InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::ActorInfo(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10, FCreateSceneOutlinerColumn(), false, 0.4f));

			// Only display MovieScene actors
			auto ActorFilter = [&](const AActor* Actor) {
				return Actor && Actor->IsA(ALevelSequenceActor::StaticClass());
			};
			InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(ActorFilter));
		}

		// actor selector to allow the user to choose an actor
		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
		TSharedRef< SWidget > MiniSceneOutliner =
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(400.0f)
			[
				SNew(SBox)
				.WidthOverride(360.f)
				[
					SceneOutlinerModule.CreateActorPicker(
						InitOptions,
						FOnActorPicked::CreateStatic(&FLevelEditorToolBar::OnCinematicsActorPicked))
				]
			];

		InSection.AddEntry(FToolMenuEntry::InitWidget("LevelEditorExistingCinematic", MiniSceneOutliner, FText::GetEmpty(), true));
	}));

#undef LOCTEXT_NAMESPACE
}


void FLevelEditorToolBar::OnCinematicsActorPicked( AActor* Actor )
{
	//The sequencer editor will not tick unless the editor viewport is in realtime mode.
	//the scene outliner eats input, so we must close any popups manually.
	FSlateApplication::Get().DismissAllMenus();

	// Make sure we dismiss the menus before we open this
	if (ALevelSequenceActor* LevelSequenceActor = Cast<ALevelSequenceActor>(Actor))
	{
		if (LevelSequenceActor->GetSequence())
		{
			FScopedSlowTask SlowTask(1.f, NSLOCTEXT("LevelToolBarCinematicsMenu", "LoadSequenceSlowTask", "Loading Level Sequence..."));
			SlowTask.MakeDialog();
			SlowTask.EnterProgressFrame();
			UObject* Asset = LevelSequenceActor->GetSequence();

			if (Asset != nullptr)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Asset);
			}
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Type::Ok, NSLOCTEXT("LevelToolBarCinematicsMenu", "LoadSequenceNotValid", "This Level Sequence actor does not have a Sequence asset assigned in the Details panel and cannot be opened."));
		}
	}
}
