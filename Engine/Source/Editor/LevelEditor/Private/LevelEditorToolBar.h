// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/UICommandList.h"
#include "SLevelEditor.h"

class UToolMenu;

/**
 * Unreal level editor main toolbar
 */
class FLevelEditorToolBar
{

public:

	/**
	 * Static: Creates a widget for the main tool bar
	 *
	 * @return	New widget
	 */
	static TSharedRef< SWidget > MakeLevelEditorToolBar( const TSharedRef<FUICommandList>& InCommandList, const TSharedRef<SLevelEditor> InLevelEditor );
	static void RegisterLevelEditorToolBar( const TSharedRef<FUICommandList>& InCommandList, const TSharedRef<SLevelEditor> InLevelEditor );

	/**
	 * Static: Creates a widget for the secondary tool bar which is displayed below the main toolbar
	 *
	 * @return	New widget
	 */
	static TSharedRef< SWidget > MakeLevelEditorSecondaryModeToolbar(TSharedRef<FUICommandList> InCommandList, TMap<FName, TSharedPtr<FLevelEditorModeUILayer>>& ModeUILayers );
	static void RegisterLevelEditorSecondaryModeToolbar();
	static FName GetSecondaryModeToolbarName();

protected:

	/**
	 * Generates menu content for the build combo button drop down menu
	 *
	 * @return	Menu content widget
	 */
	static TSharedRef< SWidget > GenerateBuildMenuContent(TSharedRef<FUICommandList> InCommandList, TWeakPtr<SLevelEditor> InLevelEditor);

	/**
	 * Generates menu content for the quick settings combo button drop down menu
	 *
	 * @return	Menu content widget
	 */
	static TSharedRef< SWidget > GenerateQuickSettingsMenu(TSharedRef<FUICommandList> InCommandList, TWeakPtr<SLevelEditor> InLevelEditor);

	/**
	 * Generates menu content for the source control combo button drop down menu
	 *
	 * @return	Menu content widget
	 */
	static TSharedRef< SWidget > GenerateSourceControlMenu(TSharedRef<FUICommandList> InCommandList, TWeakPtr<SLevelEditor> InLevelEditor);

	/**
	 * Generates menu content for the compile combo button drop down menu
	 *
	 * @return	Menu content widget
	 */
	static TSharedRef< SWidget > GenerateOpenBlueprintMenuContent(TSharedRef<FUICommandList> InCommandList, TWeakPtr<SLevelEditor> InLevelEditor);

	/**
	 * Generates menu content for the Cinematics combo button drop down menu
	 *
	 * @return	Menu content widget
	 */
	static TSharedRef< SWidget > GenerateCinematicsMenuContent(TSharedRef<FUICommandList> InCommandList, TWeakPtr<SLevelEditor> InLevelEditor);

	static TSharedRef< SWidget > GenerateAddMenuWidget(TSharedRef<FUICommandList> InCommandList, TWeakPtr<SLevelEditor> InLevelEditor);

	/**
	 * Delegate for actor selection within the Cinematics popup menu's SceneOutliner.
	 * Opens the editor for the selected actor and dismisses all popup menus.
	 */
	static void OnCinematicsActorPicked( AActor* Actor );

	/**
	 * Callback to open a sub-level script Blueprint
	 *
	 * @param InLevel	The level to open the Blueprint of (creates if needed)
	 */
	static void OnOpenSubLevelBlueprint( ULevel* InLevel );

private:
	static void RegisterSourceControlMenu();
	static void RegisterCinematicsMenu();

	static void RegisterQuickSettingsMenu();
	static void RegisterOpenBlueprintMenu();
	static void RegisterAddMenu();

	static FText GetActiveModeName(TWeakPtr<SLevelEditor> LevelEditorPtr);
	static const FSlateBrush* GetActiveModeIcon(TWeakPtr<SLevelEditor> LevelEditorPtr);

private:
	static FName SecondaryModeToolbarName;

};
