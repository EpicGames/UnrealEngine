// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelViewportActions.h"
#include "ShowFlags.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Engine.h"
#include "Editor/UnrealEdEngine.h"
#include "GameFramework/WorldSettings.h"
#include "EditorShowFlags.h"
#include "Stats/StatsData.h"
#include "BufferVisualizationData.h"
#include "NaniteVisualizationData.h"
#include "LumenVisualizationData.h"
#include "SubstrateVisualizationData.h"
#include "GroomVisualizationData.h"
#include "VirtualShadowMapVisualizationData.h"
#include "Bookmarks/BookmarkUI.h"
#include "UnrealEdGlobals.h"
#include "Camera/CameraActor.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "LevelViewportActions"

UE_DEFINE_TCOMMANDS(FLevelViewportCommands)

FLevelViewportCommands::FOnNewStatCommandAdded FLevelViewportCommands::NewStatCommandDelegate;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FLevelViewportCommands::~FLevelViewportCommands()
{
	UEngine::NewStatDelegate.RemoveAll(this);
#if STATS
	FStatGroupGameThreadNotifier::Get().NewStatGroupDelegate.Unbind();
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** UI_COMMAND takes long for the compile to optimize */
UE_DISABLE_OPTIMIZATION_SHIP
void FLevelViewportCommands::RegisterCommands()
{
	UI_COMMAND( ToggleMaximize, "Maximize Viewport", "Toggles the Maximize state of the current viewport", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleGameView, "Game View", "Toggles game view.  Game view shows the scene as it appears in game", EUserInterfaceActionType::ToggleButton, FInputChord( EKeys::G ) );
	UI_COMMAND(
		ToggleImmersive,
		"Immersive View",
		"Switches this viewport between immersive view and regular view",
		EUserInterfaceActionType::ToggleButton,
		PLATFORM_MAC ? FInputChord(EModifierKey::Control, EKeys::F11) : FInputChord(EKeys::F11)
	);
	UI_COMMAND(ToggleSidebarAllTabs, "Sidebar All Tabs", "Moves all tabs except the level editor to a sidebar or restores any previous state before if all tabs are already sidebared", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::F10));

	UI_COMMAND( HighResScreenshot, "High Resolution Screenshot...", "Opens the control panel for high resolution screenshots", EUserInterfaceActionType::Button, FInputChord() );
	
	UI_COMMAND( UseDefaultShowFlags, "Use Defaults", "Resets all show flags to default", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( SelectPilotedActor, "Select Piloted Actor", "Select the currently piloted actor.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( PilotSelectedActor, "Pilot Selected Actor", "Move the selected actor around using the viewport controls, and bind the viewport to the actor's location and orientation.", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Control | EModifierKey::Shift, EKeys::P ) );
	UI_COMMAND( EjectActorPilot, "Eject from Actor Pilot", "Stop piloting an actor with the current viewport. Unlocks the viewport's position and orientation from the actor the viewport is currently piloting.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ToggleActorPilotCameraView, "Actor Pilot Camera View", "Toggles showing the exact camera view when using the viewport to pilot a camera", EUserInterfaceActionType::ToggleButton, FInputChord( EModifierKey::Control | EModifierKey::Shift, EKeys::C ) );

	UI_COMMAND( ViewportConfig_OnePane, "Layout One Pane", "Changes the viewport arrangement to one pane", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_TwoPanesH, "Layout Two Panes (horizontal)", "Changes the viewport arrangement to two panes, side-by-side", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_TwoPanesV, "Layout Two Panes (vertical)", "Changes the viewport arrangement to two panes, one above the other", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_ThreePanesLeft, "Layout Three Panes (one left, two right)", "Changes the viewport arrangement to three panes, one on the left, two on the right", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_ThreePanesRight, "Layout Three Panes (one right, two left)", "Changes the viewport arrangement to three panes, one on the right, two on the left", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_ThreePanesTop, "Layout Three Panes (one top, two bottom)", "Changes the viewport arrangement to three panes, one on the top, two on the bottom", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_ThreePanesBottom, "Layout Three Panes (one bottom, two top)", "Changes the viewport arrangement to three panes, one on the bottom, two on the top", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_FourPanesLeft, "Layout Four Panes (one left, three right)", "Changes the viewport arrangement to four panes, one on the left, three on the right", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_FourPanesRight, "Layout Four Panes (one right, three left)", "Changes the viewport arrangement to four panes, one on the right, three on the left", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_FourPanesTop, "Layout Four Panes (one top, three bottom)", "Changes the viewport arrangement to four panes, one on the top, three on the bottom", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_FourPanesBottom, "Layout Four Panes (one bottom, three top)", "Changes the viewport arrangement to four panes, one on the bottom, three on the top", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ViewportConfig_FourPanes2x2, "Layout Four Panes (2x2)", "Changes the viewport arrangement to four panes, in a 2x2 grid", EUserInterfaceActionType::ToggleButton, FInputChord() );

	FUICommandInfo::MakeCommandInfo(
		AsShared(),
		SetDefaultViewportType,
		TEXT("Default Viewport"),
		LOCTEXT("DefaultViewportLabel", "Default Viewport"),
		LOCTEXT("DefaultViewport_Tooltip", "Reconfigures this viewport to the default arrangement"),
		FSlateIconFinder::FindIconForClass(ACameraActor::StaticClass()),
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND( ToggleViewportToolbar, "Show Viewport Toolbar", "Sets the visibility of the viewport toolbar", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::T) );

	UI_COMMAND( ApplyMaterialToActor, "Apply Material", "Attempts to apply a dropped material to this object", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( ToggleCinematicPreview, "Allow Cinematic Control", "If enabled, allows cinematic (Sequencer) previews to play in this viewport", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleAllowConstrainedAspectRatioInPreview, "Constrained Aspect Ratio Preview", "Allows this viewport to have a Constrained Aspect Ratio in Preview", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleFlipConstrainedAspectRatioInPreview, "Flip Constrained Aspect Ratio Preview", "Flip the Constrained Aspect Ratio in Preview", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( FindInLevelScriptBlueprint, "Find In Level Script", "Finds references of a selected actor in the level script blueprint", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::K) );
	UI_COMMAND(
		AdvancedSettings,
		"Open Viewport Preferences...",
		"Opens the advanced viewport settings",
		EUserInterfaceActionType::Button,
		FInputChord()
	);
	UI_COMMAND(
		PlaySettings,
		"Open Play Preferences...",
		"Opens the play settings",
		EUserInterfaceActionType::Button,
		FInputChord()
	);

	UI_COMMAND( EnableEditorShortcutsInPIE, "Enable editor keyboard shortcuts", "Enable editor keyboard shortcuts when PIE is active", EUserInterfaceActionType::ToggleButton, FInputChord() );

	// Generate a command for each volume class
	{
		UI_COMMAND( ShowAllVolumes, "Show All Volumes", "Shows all volumes", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( HideAllVolumes, "Hide All Volumes", "Hides all volumes", EUserInterfaceActionType::Button, FInputChord() );
	}

	// Generate a command for show/hide all layers
	{
		UI_COMMAND( ShowAllLayers, "Show All Layers", "Shows all layers", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( HideAllLayers, "Hide All Layers", "Hides all layers", EUserInterfaceActionType::Button, FInputChord() );
	}

	// Generate a command for each sprite category
	{
		UI_COMMAND( ShowAllSprites, "Show All Sprites", "Shows all sprites", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( HideAllSprites, "Hide All Sprites", "Hides all sprites", EUserInterfaceActionType::Button, FInputChord() );
	}

#if STATS
	// Generate a command for each Stat category
	{
		UI_COMMAND(HideAllStats, "Hide All Stats", "Hides all Stats", EUserInterfaceActionType::Button, FInputChord());

		// Bind a listener here for any additional stat commands that get registered later.
		UEngine::NewStatDelegate.AddRaw(this, &FLevelViewportCommands::HandleNewStat);
		FStatGroupGameThreadNotifier::Get().NewStatGroupDelegate.BindRaw(this, &FLevelViewportCommands::HandleNewStatGroup);
	}
#endif

	// Map the bookmark index to default key.
	TArray< FKey > NumberKeyNames;
	NumberKeyNames.Add( EKeys::Zero );
	NumberKeyNames.Add( EKeys::One );
	NumberKeyNames.Add( EKeys::Two );
	NumberKeyNames.Add( EKeys::Three );
	NumberKeyNames.Add( EKeys::Four );
	NumberKeyNames.Add( EKeys::Five );
	NumberKeyNames.Add( EKeys::Six );
	NumberKeyNames.Add( EKeys::Seven );
	NumberKeyNames.Add( EKeys::Eight );
	NumberKeyNames.Add( EKeys::Nine );

	for( int32 BookmarkIndex = 0; BookmarkIndex < AWorldSettings::NumMappedBookmarks; ++BookmarkIndex )
	{
		TSharedRef< FUICommandInfo > JumpToBookmark =
			FUICommandInfoDecl(
			this->AsShared(), //Command class
			FBookmarkUI::GetJumpToCommandName(BookmarkIndex), //CommandName
			FBookmarkUI::GetJumpToLabel(BookmarkIndex), //Localized label
			FBookmarkUI::GetJumpToTooltip(BookmarkIndex) )//Localized tooltip
			.UserInterfaceType( EUserInterfaceActionType::Button ) //interface type
			.DefaultChord( FInputChord( NumberKeyNames.IsValidIndex( BookmarkIndex ) ? NumberKeyNames[BookmarkIndex] : EKeys::Invalid ) ); //default chord

		JumpToBookmarkCommands.Add( JumpToBookmark );

		TSharedRef< FUICommandInfo > SetBookmark =
			FUICommandInfoDecl(
			this->AsShared(), //Command class
			FBookmarkUI::GetSetCommandName(BookmarkIndex), //CommandName
			FBookmarkUI::GetSetLabel(BookmarkIndex), //Localized label
			FBookmarkUI::GetSetTooltip(BookmarkIndex) )//Localized tooltip
			.UserInterfaceType( EUserInterfaceActionType::Button ) //interface type
			.DefaultChord( FInputChord( EModifierKey::Control, NumberKeyNames.IsValidIndex( BookmarkIndex ) ? NumberKeyNames[BookmarkIndex] : EKeys::Invalid ) ); //default chord

		SetBookmarkCommands.Add( SetBookmark );

		TSharedRef< FUICommandInfo > ClearBookMark =
			FUICommandInfoDecl(
			this->AsShared(), //Command class
			FBookmarkUI::GetClearCommandName(BookmarkIndex), //CommandName
			FBookmarkUI::GetClearLabel(BookmarkIndex), //Localized label
			FBookmarkUI::GetClearTooltip(BookmarkIndex) )//Localized tooltip
			.UserInterfaceType( EUserInterfaceActionType::Button ) //interface type
			.DefaultChord( FInputChord() ); //default chord 

		ClearBookmarkCommands.Add( ClearBookMark );
	}

	UI_COMMAND( CompactBookmarks, "Compact Bookmarks", "Attempts to move bookmark indices so they are continuous (does not mapped bookmarks).", EUserInterfaceActionType::Button, FInputChord() );

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UI_COMMAND( ClearAllBookmarks, "Clear All Bookmarks", "Clears all the bookmarks", EUserInterfaceActionType::Button, FInputChord() );
	ClearAllBookMarks = ClearAllBookmarks;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UI_COMMAND( EnablePreviewMesh, "Hold To Enable Preview Mesh", "When held down a preview mesh appears under the cursor", EUserInterfaceActionType::Button, FInputChord(EKeys::Backslash) );
	UI_COMMAND( CyclePreviewMesh, "Cycles Preview Mesh", "Cycles available preview meshes", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift, EKeys::Backslash ) );

}

#if STATS
void FLevelViewportCommands::HandleNewStatGroup(const TArray<FStatNameAndInfo>& NameAndInfos)
{
	// #Stats: FStatNameAndInfo should be private and visible only to stats2 system
	for (int32 InfoIdx = 0; InfoIdx < NameAndInfos.Num(); InfoIdx++)
	{
		const FStatNameAndInfo& NameAndInfo = NameAndInfos[InfoIdx];
		const FName GroupName = NameAndInfo.GetGroupName();
		const FName GroupCategory = NameAndInfo.GetGroupCategory();
		const FText GroupDescription = FText::FromString(NameAndInfo.GetDescription());	// @todo localize description?
		HandleNewStat(GroupName, GroupCategory, GroupDescription);
	}
}

void FLevelViewportCommands::HandleNewStat(const FName& InStatName, const FName& InStatCategory, const FText& InStatDescription)
{
	FString CommandName = InStatName.ToString();
	if (CommandName.RemoveFromStart(TEXT("STATGROUP_")) || CommandName.RemoveFromStart(TEXT("STAT_")))
	{
		// Trim the front to get our category name
		FString GroupCategory = InStatCategory.ToString();
		if (!GroupCategory.RemoveFromStart(TEXT("STATCAT_")))
		{
			GroupCategory.Empty();
		}

		// If we already have an entry (which can happen if a category has changed [when loading older saved stat data]) or we don't have a valid category then skip adding
		if (!FInputBindingManager::Get().FindCommandInContext(this->GetContextName(), InStatName).IsValid() && !GroupCategory.IsEmpty())
		{
			// Find or Add the category
			TArray<FShowMenuCommand>* ShowStatCommands = ShowStatCatCommands.Find(GroupCategory);
			if (!ShowStatCommands)
			{
				// New category means we'll need to resort
				ShowStatCatCommands.Add(GroupCategory);
				ShowStatCatCommands.KeySort(TLess<FString>());
				ShowStatCommands = ShowStatCatCommands.Find(GroupCategory);
			}

			const int32 NewIndex = FindStatIndex(ShowStatCommands, CommandName);
			if (NewIndex != INDEX_NONE)
			{
				const FText DisplayName = FText::FromString(CommandName);

				FText DescriptionName = InStatDescription;
				FFormatNamedArguments Args;
				Args.Add(TEXT("StatName"), DisplayName);
				if (DescriptionName.IsEmpty())
				{
					DescriptionName = FText::Format(NSLOCTEXT("UICommands", "StatShowCommandName", "Show {StatName} Stat"), Args);
				}

				TSharedPtr<FUICommandInfo> StatCommand
					= FUICommandInfoDecl(this->AsShared(), InStatName, DisplayName, DescriptionName)
					.UserInterfaceType(EUserInterfaceActionType::ToggleButton);

				FLevelViewportCommands::FShowMenuCommand ShowStatCommand(StatCommand, DisplayName);
				ShowStatCommands->Insert(ShowStatCommand, NewIndex);
				NewStatCommandDelegate.Broadcast(ShowStatCommand.ShowMenuItem, ShowStatCommand.LabelOverride.ToString());
			}
		}
	}
}

int32 FLevelViewportCommands::FindStatIndex(const TArray< FShowMenuCommand >* ShowStatCommands, const FString& InCommandName) const
{
	check(ShowStatCommands);
	for (int32 StatIndex = 0; StatIndex < ShowStatCommands->Num(); ++StatIndex)
	{
		const FString CommandName = (*ShowStatCommands)[StatIndex].LabelOverride.ToString();
		const int32 Compare = InCommandName.Compare(CommandName);
		if (Compare == 0)
		{
			return INDEX_NONE;
		}
		else if (Compare < 0)
		{
			return StatIndex;
		}
	}
	return ShowStatCommands->Num();
}
#endif


void FLevelViewportCommands::RegisterShowVolumeCommands()
{
	/** This allows us to register commands to show volumes after all the modules were loaded. */
	TArray< UClass* > VolumeClasses;
	UUnrealEdEngine::GetSortedVolumeClasses(&VolumeClasses);

	for (int32 VolumeClassIndex = 0; VolumeClassIndex < VolumeClasses.Num(); ++VolumeClassIndex)
	{
		//@todo Slate: The show flags system does not support descriptions currently
		const FText VolumeDesc;
		const FName VolumeName = VolumeClasses[VolumeClassIndex]->GetFName();

		// Only add a command if there is none already for this volume class.
		TSharedPtr<FUICommandInfo> FoundVolumeCommand = FInputBindingManager::Get().FindCommandInContext(this->AsShared()->GetContextName(), VolumeName);
		if (!FoundVolumeCommand.IsValid())
		{
			FText DisplayName;
			if (!FEngineShowFlags::FindShowFlagDisplayName(VolumeName.ToString(), DisplayName))
			{
				DisplayName = VolumeClasses[VolumeClassIndex]->GetDisplayNameText();
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("ShowFlagName"), DisplayName);
			const FText LocalizedName = FText::Format(LOCTEXT("ShowFlagLabel_Visualize", "Visualize {ShowFlagName}"), Args);

			TSharedPtr<FUICommandInfo> ShowVolumeCommand
				= FUICommandInfoDecl(this->AsShared(), VolumeName, LocalizedName, VolumeDesc)
				.UserInterfaceType(EUserInterfaceActionType::ToggleButton);

			ShowVolumeCommands.Add(FLevelViewportCommands::FShowMenuCommand(ShowVolumeCommand, DisplayName));
		}
	}
}

void FLevelViewportCommands::RegisterShowSpriteCommands()
{
	// get all the known layers
	// Get a fresh list as GUnrealEd->SortedSpriteInfo may not yet be built.
	TArray<FSpriteCategoryInfo> SortedSpriteInfo;
	UUnrealEdEngine::MakeSortedSpriteInfo(SortedSpriteInfo);

	FString SpritePrefix = TEXT("ShowSprite_");
	for (int32 InfoIndex = 0; InfoIndex < SortedSpriteInfo.Num(); ++InfoIndex)
	{
		const FSpriteCategoryInfo& SpriteInfo = SortedSpriteInfo[InfoIndex];
		const FName CommandName = FName(*(SpritePrefix + SpriteInfo.Category.ToString()));

		// Only add a command if there is none already for this sprite class.
		TSharedPtr<FUICommandInfo> FoundSpriteCommand = FInputBindingManager::Get().FindCommandInContext(this->AsShared()->GetContextName(), CommandName);
		if (!FoundSpriteCommand.IsValid())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("SpriteName"), SpriteInfo.DisplayName);
			const FText LocalizedName = FText::Format(NSLOCTEXT("UICommands", "SpriteShowFlagName", "Show {SpriteName} Sprites"), Args);

			TSharedPtr<FUICommandInfo> ShowSpriteCommand
				= FUICommandInfoDecl(this->AsShared(), CommandName, LocalizedName, SpriteInfo.Description)
				.UserInterfaceType(EUserInterfaceActionType::ToggleButton);

			const int32 ShowSpriteCommandIndex = ShowSpriteCommands.Add(FLevelViewportCommands::FShowMenuCommand(ShowSpriteCommand, SpriteInfo.DisplayName));

			// Update the global table that maps each sprite category to its corresponding visibility command.
			if (GUnrealEd)
			{
				GUnrealEd->SpriteIDToIndexMap.Add(SpriteInfo.Category, ShowSpriteCommandIndex);
			}
		}
	}
}

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
