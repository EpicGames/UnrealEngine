// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelViewportLayout.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Framework/Docking/LayoutService.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SCanvas.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "LevelEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "EditorViewportTabContent.h"
#include "LevelViewportLayoutEntity.h"
#include "SAssetEditorViewport.h"

static const FName LevelEditorModName("LevelEditor");

namespace ViewportLayoutDefs
{
	/** How many seconds to interpolate from restored to maximized state */
	static const float MaximizeTransitionTime = 0.15f;

	/** How many seconds to interpolate from maximized to restored state */
	static const float RestoreTransitionTime = 0.2f;

	/** Default maximized state for new layouts - will only be applied when no config data is restoring state */
	static const bool bDefaultShouldBeMaximized = true;

	/** Default immersive state for new layouts - will only be applied when no config data is restoring state */
	static const bool bDefaultShouldBeImmersive = false;
}

// FLevelViewportLayout /////////////////////////////

FLevelViewportLayout::FLevelViewportLayout()
	: bIsTransitioning( false ),
	  bIsReplacement( false ),
	  bIsQueryingLayoutMetrics( false ),
	  bIsMaximizeSupported( true ),
	  bIsMaximized( false ),
	  bIsImmersive( false ),
	  bWasMaximized( false ),
	  bWasImmersive( false ),
	  MaximizedViewportStartPosition( FVector2D::ZeroVector ),
	  MaximizedViewportStartSize( FVector2D::ZeroVector )
{
	ViewportReplacementWidget = SNew( SSpacer );
}


FLevelViewportLayout::~FLevelViewportLayout()
{
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModName);
	LevelEditor.OnTakeHighResScreenShots().RemoveAll(this);

	// Make sure that we're not locking the immersive window after we go away
	if( bIsImmersive || ( bWasImmersive && bIsTransitioning ) )
	{
		TSharedPtr< SWindow > OwnerWindow( CachedOwnerWindow.Pin() );
		if( OwnerWindow.IsValid() )
		{
			OwnerWindow->SetFullWindowOverlayContent( NULL );
		}
	}
}


TSharedRef<SWidget> FLevelViewportLayout::BuildViewportLayout(TSharedPtr<SDockTab> InParentDockTab, TSharedPtr<FEditorViewportTabContent> InParentTab, const FString& LayoutString)
{
	TSharedRef<SWidget> ViewportLayoutWidget = FAssetEditorViewportLayout::BuildViewportLayout(InParentDockTab, InParentTab, LayoutString);

	// Important: We use raw bindings here because we are releasing our binding in our destructor (where a weak pointer would be invalid)
	// It's imperative that our delegate is removed in the destructor for the level editor module to play nicely with reloading.
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModName);
	LevelEditor.OnTakeHighResScreenShots().AddRaw(this, &FLevelViewportLayout::TakeHighResScreenShot);

	// Prevent maximize if we only have a single viewport
	bIsMaximizeSupported = (Viewports.Num() > 1);

	return ViewportLayoutWidget;
}

TSharedRef<SWidget> FLevelViewportLayout::FactoryViewport(FName InTypeName, const FAssetEditorViewportConstructionArgs& ConstructionArgs)
{
	TSharedPtr<IEditorViewportLayoutEntity> ViewportLayoutEntity;
	TSharedPtr<FEditorViewportTabContent> PinnedTabContent = ParentTabContent.Pin();
	if (PinnedTabContent.IsValid())
	{
		// Manually use the factory function here based on type, because legacy viewport types don't register with our factory functions
		// The level editor module will return an appropriate default if the legacy lookup fails too.
		if (const AssetEditorViewportFactoryFunction* FactoryFunc = PinnedTabContent->FindViewportCreationFactory(InTypeName))
		{
			TSharedPtr<SAssetEditorViewport> EditorViewport = (*FactoryFunc)(ConstructionArgs);
			ViewportLayoutEntity = MakeShareable(new FLevelViewportLayoutEntity(EditorViewport));
		}
	}
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!ViewportLayoutEntity.IsValid())
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModName);
		ViewportLayoutEntity = LevelEditor.FactoryViewport(InTypeName, ConstructionArgs);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	// Both the legacy level editor factory viewport, and the viewport creation functions should fall back to a valid default
	check(ViewportLayoutEntity.IsValid());
	
	Viewports.Add(ConstructionArgs.ConfigKey, ViewportLayoutEntity);
	return ViewportLayoutEntity->AsWidget();
}

void FLevelViewportLayout::BeginThrottleForAnimatedResize()
{
	// Only enter this mode if there is not already a request
	if( !ViewportResizeThrottleRequest.IsValid() )
	{
		if( !FSlateApplication::Get().IsRunningAtTargetFrameRate() )
		{
			ViewportResizeThrottleRequest = FSlateThrottleManager::Get().EnterResponsiveMode();
		}
	}
}


void FLevelViewportLayout::EndThrottleForAnimatedResize()
{
	// Only leave this mode if there is a request
	if( ViewportResizeThrottleRequest.IsValid() )
	{
		FSlateThrottleManager::Get().LeaveResponsiveMode( ViewportResizeThrottleRequest );
	}
}

void FLevelViewportLayout::InitCommonLayoutFromString( const FString& SpecificLayoutString, const FName PerspectiveViewportKey )
{
	FName DefaultMaximizedViewport = PerspectiveViewportKey;
	MaximizedViewport = NAME_None;

	bool bShouldBeMaximized = bIsMaximizeSupported && ViewportLayoutDefs::bDefaultShouldBeMaximized;
	bool bShouldBeImmersive = ViewportLayoutDefs::bDefaultShouldBeImmersive;

	if (!SpecificLayoutString.IsEmpty())
	{
		const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

		// NOTE: We don't support starting back up in immersive mode, even if the user shut down with a window that way.  See the
		// comment below in SaveCommonLayoutString() for more info.
		GConfig->GetBool(*IniSection, *(SpecificLayoutString + TEXT(".bIsMaximized")), bShouldBeMaximized, GEditorPerProjectIni);
		FString MaximizedViewportString;
		if (GConfig->GetString(*IniSection, *(SpecificLayoutString + TEXT(".MaximizedViewport")), MaximizedViewportString, GEditorPerProjectIni))
		{
			DefaultMaximizedViewport = *MaximizedViewportString;
		}
	}
	// Replacement layouts (those selected by the user via a command) don't start maximized so the layout can be seen clearly.
	if (!bIsReplacement && bIsMaximizeSupported && Viewports.Contains(DefaultMaximizedViewport) && (bShouldBeMaximized || bShouldBeImmersive))
	{
		// we are not toggling maximize or immersive state but setting it directly
		const bool bToggle=false;
		// Do not allow animation at startup as it hitches
		const bool bAllowAnimation=false;
		MaximizeViewport(DefaultMaximizedViewport, bShouldBeMaximized, bShouldBeImmersive, bAllowAnimation);
	}
}

void FLevelViewportLayout::SaveCommonLayoutString( const FString& SpecificLayoutString ) const
{
	const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

	// Save all our data using the additional layout config
	for (auto& Pair : Viewports)
	{
		// The Viewports map is keyed on the full config name, so no need to prepend the SpecificLayoutString
		FString ConfigName = Pair.Key.ToString();

		Pair.Value->SaveConfig(ConfigName);

		GConfig->SetString( *IniSection, *( ConfigName + TEXT(".TypeWithinLayout") ), *Pair.Value->GetType().ToString(), GEditorPerProjectIni );
	}

	// We don't bother saving that we were in immersive mode, because we never want to start back up directly in immersive mode
	// unless the user asks for that on the command-line.  The reason is it can be disorientating to not see any editor UI when
	// to restart the editor.  In this case, we'll store the mode they were previously in before they switched to immersive mode.
	if( bIsImmersive )
	{
		GConfig->SetBool( *IniSection, *( SpecificLayoutString + TEXT( ".bIsMaximized" ) ), bIsMaximizeSupported && bWasMaximized, GEditorPerProjectIni );
	}
	else
	{
		GConfig->SetBool(*IniSection, *(SpecificLayoutString + TEXT(".bIsMaximized")), bIsMaximizeSupported && bIsMaximized, GEditorPerProjectIni);
	}
	GConfig->SetString( *IniSection, *( SpecificLayoutString + TEXT( ".MaximizedViewport" ) ), *MaximizedViewport.ToString(), GEditorPerProjectIni );
}

void FLevelViewportLayout::RequestMaximizeViewport( FName ViewportToMaximize, const bool bWantMaximize, const bool bWantImmersive, const bool bAllowAnimation )
{
	if( bAllowAnimation )
	{
		// Ensure the UI is responsive when animating the transition to/from maximize
		BeginThrottleForAnimatedResize();

		// We flush commands here because there could be a pending slow viewport draw already enqueued in the render thread
		// We take the hitch here so that our transition to/from maximize animation is responsive next tick
		FlushRenderingCommands();

		DeferredMaximizeCommands.Add( FMaximizeViewportCommand(ViewportToMaximize, bWantMaximize, bWantImmersive) );
	}
	else
	{
		// Not animating so just maximise now
		MaximizeViewport( ViewportToMaximize, bWantMaximize, bWantImmersive, bAllowAnimation );
	}
}

void FLevelViewportLayout::MaximizeViewport( FName ViewportToMaximize, const bool bWantMaximize, const bool bWantImmersive, const bool bAllowAnimation )
{
	TSharedPtr<ILevelViewportLayoutEntity> Entity = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Viewports.FindRef(ViewportToMaximize));

	// Should never get into a situation where the viewport is being maximized and there is already a maximized viewport. 
	// I.E Maximized viewport is NULL which means this is a new maximize or MaximizeViewport is equal to the passed in one which means this is a restore of the current maximized viewport
	check( Entity.IsValid() );
	check( MaximizedViewport.IsNone() || MaximizedViewport == ViewportToMaximize );
	check(LayoutConfiguration.IsValid());

	// If we're already in immersive mode, toggling maximize just needs to update some state (no visual change)
	if( bIsImmersive )
	{
		bIsMaximized = bWantMaximize;
	}

	// Any changes?
	if( bWantMaximize != bIsMaximized || bWantImmersive != bIsImmersive )
	{
		// Are we already animating a transition?
		if( bIsTransitioning )
		{
			// Instantly finish up the current transition
			FinishMaximizeTransition();

			check( !bIsTransitioning );
		}

		TSharedPtr<SWindow> OwnerWindow;
		bIsQueryingLayoutMetrics = true;
		FWidgetPath ViewportWidgetPath;
		if( bIsMaximized || bIsImmersive )
		{
			// Use the replacement widget for metrics, as our viewport widget has been reparented to the overlay
			FSlateApplication::Get().GeneratePathToWidgetUnchecked( ViewportReplacementWidget.ToSharedRef(), ViewportWidgetPath );
			OwnerWindow = ViewportWidgetPath.TopLevelWindow;
		}
		else
		{
			// Viewport is still within the splitter, so use it for metrics directly
			FSlateApplication::Get().GeneratePathToWidgetUnchecked( Entity->AsWidget(), ViewportWidgetPath );
			OwnerWindow = ViewportWidgetPath.TopLevelWindow;
		}
		bIsQueryingLayoutMetrics = false;

		// If the widget can't be found in the layout pass, attempt to use the cached owner window
		if(!OwnerWindow.IsValid() && CachedOwnerWindow.IsValid())
		{
			OwnerWindow = CachedOwnerWindow.Pin();
		}
		else
		{
			// Keep track of the window we're contained in
			// @todo immersive: Caching this after the transition is risky -- the widget could be moved to a new window!
			//		We really need a safe way to query a widget's window that doesn't require a full layout pass.  Then,
			//	    instead of caching the window we can look it up whenever it's needed
			CachedOwnerWindow = OwnerWindow;
		}

		if( !bIsImmersive && bWantImmersive )
		{
			// If we can't find our owner window, that means we're likely hosted in a background tab, thus
			// can't continue with an immersive transition.  We never want immersive mode to take over the
			// window when the user couldn't even see the viewports before!
			if( !OwnerWindow.IsValid() )
			{
				return;
			}

			// Make sure that our viewport layout has a lock on the window's immersive state.  Only one
			// layout can have a single immersive viewport at a time, so if something else is already immersive,
			// we need to fail the layout change.
			if( OwnerWindow->HasFullWindowOverlayContent() )
			{
				// We can't continue with the layout change, a different window is already immersive
				return;
			}
		}


		// Update state
		bWasMaximized = bIsMaximized;
		bWasImmersive = bIsImmersive;

		bIsMaximized = bWantMaximize;
		bIsImmersive = bWantImmersive;


		// Start transition
		bIsTransitioning = true;

		if( bAllowAnimation )
		{
			// Ensure responsiveness while transitioning
			BeginThrottleForAnimatedResize();
		}

		if( ( bWasMaximized && !bIsMaximized ) ||
			( bWasImmersive && !bIsImmersive ) )
		{
			// Play the transition backwards.  Note that when transitioning from immersive mode, depending on
			// the current state of bIsMaximized, we'll transition to either a maximized state or a "restored" state
			MaximizeAnimation = FCurveSequence();
			MaximizeAnimation.AddCurve( 0.0f, ViewportLayoutDefs::RestoreTransitionTime, ECurveEaseFunction::CubicIn );
			MaximizeAnimation.PlayReverse( ViewportsOverlayWidget->AsShared() );
			
			if( bWasImmersive && !bIsImmersive )
			{
				OwnerWindow->BeginFullWindowOverlayTransition();
				OwnerWindow->SetNativeWindowButtonsVisibility(true);
			}
		}
		else
		{
			if( bIsImmersive && ( bWasMaximized && bIsMaximized ) )
			{
				// Unhook our viewport overlay, as we'll let the window overlay drive this for immersive mode
				ViewportsOverlayPtr.Pin()->RemoveSlot();
			}
			else
			{
				// Store the maximized viewport
				MaximizedViewport = ViewportToMaximize;

				TSharedPtr<ILevelViewportLayoutEntity> MaximizedEntity = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Viewports.FindRef(MaximizedViewport));
				if (MaximizedEntity.IsValid())
				{
					// Replace our viewport with a dummy widget in it's place during the maximize transition.  We can't
					// have a single viewport widget in two places at once!
					LayoutConfiguration->ReplaceWidget( MaximizedEntity->AsWidget(), ViewportReplacementWidget.ToSharedRef() );

					// The attributes need the AllocatedSize of the parent.
					// The size is updated in the Paint function and the attributes in the Prepass (too soon).
					// Update the value manually Tick function (after the parent's Paint).
					class SCanvasInternal : public SCanvas
					{
					public:
						SCanvasInternal()
						{
							SetCanTick(true);
						}

						void Construct(const FArguments& Args, TSharedRef<FLevelViewportLayout> ViewportLayout, TSharedRef<SWidget> MaximizedEntity)
						{
							SCanvas::Construct(Args);
							OwnerViewportLayout = ViewportLayout;
							AddSlot()
								.Expose(ViewportsOverlayWidgetSlot)
								[
									MaximizedEntity
								];
						}

						virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
						{
							SCanvas::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
							if (TSharedPtr<FLevelViewportLayout> Owner = OwnerViewportLayout.Pin())
							{
								ViewportsOverlayWidgetSlot->SetPosition(Owner->GetMaximizedViewportPositionOnCanvas());
								ViewportsOverlayWidgetSlot->SetSize(Owner->GetMaximizedViewportSizeOnCanvas());
							}
						}

					private:
						using SCanvas::AddSlot;
						using SCanvas::RemoveSlot;

					private:
						SCanvas::FSlot* ViewportsOverlayWidgetSlot = nullptr;
						TWeakPtr<FLevelViewportLayout> OwnerViewportLayout;
					};

					ViewportsOverlayWidget = SNew(SCanvasInternal, SharedThis(this), MaximizedEntity->AsWidget());
				}
			}


			// Add the maximized viewport as a top level overlay
			if( bIsImmersive )
			{
				OwnerWindow->SetFullWindowOverlayContent( ViewportsOverlayWidget );
				OwnerWindow->BeginFullWindowOverlayTransition();
			}
			else
			{
				// Create a slot in our overlay to hold the content
				ViewportsOverlayPtr.Pin()->AddSlot()
				[
					ViewportsOverlayWidget.ToSharedRef()
				];
			}

			// Play the "maximize" transition
			MaximizeAnimation = FCurveSequence();
			MaximizeAnimation.AddCurve( 0.0f, ViewportLayoutDefs::MaximizeTransitionTime, ECurveEaseFunction::CubicOut );
			MaximizeAnimation.Play( ViewportsOverlayWidget->AsShared() );
		}


		// We'll only be able to get metrics if we could find an owner window.  Usually that's OK, because the only
		// chance for this code to trigger without an owner window would be at startup, where we might ask to maximize
		// a viewport based on saved layout, while that viewport is hosted in a background tab.  For this case, we'll
		// never animate (checked here), so we don't need to store "before" metrics.
		check( OwnerWindow.IsValid() || !bAllowAnimation );
		if( OwnerWindow.IsValid() && ViewportWidgetPath.IsValid() )
		{
			// Setup transition metrics
			if( bIsImmersive || bWasImmersive )
			{
				const FVector2D WindowScreenPos = OwnerWindow->GetPositionInScreen();
				if( bIsMaximized || bWasMaximized )
				{
					FWidgetPath ViewportsOverlayWidgetPath = ViewportWidgetPath.GetPathDownTo( ViewportsOverlayPtr.Pin().ToSharedRef() );
					const FArrangedWidget& ViewportsOverlayGeometry = ViewportsOverlayWidgetPath.Widgets.Last();
					MaximizedViewportStartPosition = FVector2D(ViewportsOverlayGeometry.Geometry.AbsolutePosition) - WindowScreenPos;
					MaximizedViewportStartSize = ViewportsOverlayPtr.Pin()->GetCachedSize();
				}
				else
				{
					const FArrangedWidget& ViewportGeometry = ViewportWidgetPath.Widgets.Last();	
					MaximizedViewportStartPosition = FVector2D(ViewportGeometry.Geometry.AbsolutePosition) - WindowScreenPos;
					MaximizedViewportStartSize = ViewportGeometry.Geometry.Size;
				}
			}
			else
			{
				const FArrangedWidget& ViewportGeometry = ViewportWidgetPath.Widgets.Last();
				MaximizedViewportStartPosition = FVector2D(ViewportGeometry.Geometry.Position);
				MaximizedViewportStartSize = ViewportGeometry.Geometry.Size;
			}
		}


		if( !bAllowAnimation )
		{
			// Instantly finish up the current transition
			FinishMaximizeTransition();
			check( !bIsTransitioning );
		}

		// Redraw all other viewports, in case there were changes made while in immersive mode that may affect
		// the view in other viewports.
		GUnrealEd->RedrawLevelEditingViewports();
	}
}


FVector2D FLevelViewportLayout::GetMaximizedViewportPositionOnCanvas() const
{
	FVector2D EndPos = FVector2D::ZeroVector;
	if( bIsImmersive )
	{
		TSharedPtr< SWindow > OwnerWindow( CachedOwnerWindow.Pin() );
		if( OwnerWindow.IsValid() && OwnerWindow->IsWindowMaximized() )
		{
			// When maximized we offset by the window border size or else the immersive viewport will be clipped
			FMargin WindowContentMargin = OwnerWindow->GetWindowBorderSize();
			EndPos.Set( WindowContentMargin.Right, WindowContentMargin.Bottom );
		}
	}
	
	return FMath::Lerp( MaximizedViewportStartPosition, EndPos, MaximizeAnimation.GetLerp() );
}



FVector2D FLevelViewportLayout::GetMaximizedViewportSizeOnCanvas() const
{
	// NOTE: Should ALWAYS be valid, however because MaximizedViewport is changed in Tick, it's possible
	//       for widgets we're adding/removing to already have been reported by ArrangeChildren, thus
	//       we need to be able to handle cases where widgets that are not bound can still have delegates fire
	if( !MaximizedViewport.IsNone() || bWasImmersive )
	{
		FVector2D TargetSize = FVector2D::ZeroVector;
		if( bIsImmersive || ( bIsTransitioning && bWasImmersive ) )
		{
			TSharedPtr< SWindow > OwnerWindow( CachedOwnerWindow.Pin() );
			if( OwnerWindow.IsValid() )
			{
				FVector2D ClippedArea = FVector2D::ZeroVector;

				const float ScaleFactor = OwnerWindow->GetNativeWindow()->GetDPIScaleFactor() * FSlateApplication::Get().GetApplicationScale();

				if( OwnerWindow->IsWindowMaximized() )
				{
					// When the window is maximized and we are in immersive we size the canvas to the size of the visible area which does not include the window border
					const FMargin& WindowContentMargin = OwnerWindow->GetWindowBorderSize() * ScaleFactor;
					ClippedArea.Set( WindowContentMargin.GetTotalSpaceAlong<Orient_Horizontal>(), WindowContentMargin.GetTotalSpaceAlong<Orient_Vertical>() );
				}

				TargetSize = (OwnerWindow->GetSizeInScreen() - ClippedArea)/ScaleFactor;
			}
		}
		else
		{
			TargetSize = ViewportsOverlayPtr.Pin()->GetCachedSize();
		}
		return FMath::Lerp( MaximizedViewportStartSize, TargetSize, MaximizeAnimation.GetLerp() );
	}

	// No valid viewport to check size for
	return FVector2D::ZeroVector;
}

/** Method for taking high res screen shots of viewports */

void FLevelViewportLayout::TakeHighResScreenShot()
{
	if (bIsImmersive || bIsMaximized)
	{
		TSharedPtr<ILevelViewportLayoutEntity> MaximizedViewportEntity = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Viewports.FindRef(MaximizedViewport));
		check(MaximizedViewportEntity.IsValid());

		MaximizedViewportEntity->TakeHighResScreenShot();
	}
	else
	{
		for (auto& Elem : Viewports)
		{
			TSharedPtr<ILevelViewportLayoutEntity> ViewportEntity = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Elem.Value);

			if (ViewportEntity.IsValid())
			{
				ViewportEntity->TakeHighResScreenShot();
			}
		}
	}
}

/**
 * @return true if this layout is visible.  It is not visible if its parent tab is not active
 */
bool FLevelViewportLayout::IsVisible()  const
{
	return !ParentTab.IsValid() || ParentTab.Pin()->IsForeground();
}

/**
 * Checks to see the specified level viewport is visible in this layout
 * A viewport is visible in a layout if the layout is visible and the viewport is the maximized viewport or there is no maximized viewport
 *
 * @param InViewport	The viewport within this layout that should be checked
 * @return true if the viewport is visible.  
 */
bool FLevelViewportLayout::IsLevelViewportVisible( FName InViewport ) const
{
	// The passed in viewport is visible if the current layout is visible and their is no maximized viewport or the viewport that is maximized was passed in.
	return IsVisible() && ( MaximizedViewport.IsNone() || MaximizedViewport == InViewport );
}

bool FLevelViewportLayout::IsViewportMaximized( FName InViewport ) const
{
	return bIsMaximized && MaximizedViewport == InViewport;
}

bool FLevelViewportLayout::IsViewportImmersive( FName InViewport ) const
{
	return bIsImmersive && MaximizedViewport == InViewport;
}

EVisibility FLevelViewportLayout::OnGetNonMaximizedVisibility() const
{
	// The non-maximized viewports are not visible if there is a maximized viewport on top of them
	return ( !bIsQueryingLayoutMetrics && !MaximizedViewport.IsNone() && !bIsTransitioning && DeferredMaximizeCommands.Num() == 0 ) ? EVisibility::Collapsed : EVisibility::Visible;
}


void FLevelViewportLayout::FinishMaximizeTransition()
{
	if( bIsTransitioning )
	{
		TSharedPtr<ILevelViewportLayoutEntity> MaximizedViewportEntity = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Viewports.FindRef(MaximizedViewport));
		check(MaximizedViewportEntity.IsValid());
		check(LayoutConfiguration.IsValid());

		// The transition animation is complete, allow the engine to tick normally
		EndThrottleForAnimatedResize();

		// Jump to the end if we're not already there
		MaximizeAnimation.JumpToEnd();

		if( bIsImmersive )
		{
			TSharedPtr< SWindow > OwnerWindow( CachedOwnerWindow.Pin() );
			if( OwnerWindow.IsValid() )
			{
				OwnerWindow->SetNativeWindowButtonsVisibility(false);
				OwnerWindow->EndFullWindowOverlayTransition();
			}

			// Finished transition from restored/maximized to immersive, if this is a PIE window we need to re-register it to capture the mouse.
			MaximizedViewportEntity->RegisterGameViewportIfPIE();
		}
		else if( bIsMaximized && !bWasImmersive )
		{
			// Finished transition from restored to immersive, if this is a PIE window we need to re-register it to capture the mouse.
			MaximizedViewportEntity->RegisterGameViewportIfPIE();
		}
		else if( bWasImmersive )	// Finished transition from immersive to restored/maximized
		{
			TSharedPtr< SWindow > OwnerWindow( CachedOwnerWindow.Pin() );
			if( OwnerWindow.IsValid() )
			{
				OwnerWindow->SetFullWindowOverlayContent( NULL );
				OwnerWindow->EndFullWindowOverlayTransition();
			}
			// Release overlay mouse capture to prevent situations where user is unable to get the mouse cursor back if they were holding one of the buttons down and exited immersive mode.
			FSlateApplication::Get().ReleaseAllPointerCapture();

			if( bIsMaximized )
			{
				// If we're transitioning from immersive to maximized, then we need to add our
				// viewport back to the viewport overlay
				ViewportsOverlayPtr.Pin()->AddSlot()
				[
					ViewportsOverlayWidget.ToSharedRef()
				];

				// Now that the viewport is nested within the overlay again, reset our animation so that
				// our metrics callbacks return the correct value (not the reserved value)
				MaximizeAnimation.Reverse();
				MaximizeAnimation.JumpToEnd();
			}
			else
			{
				// @todo immersive: Viewport flashes yellow for one frame in this transition point (immersive -> restored only!)
			}
		}
		else
		{
			// Finished transition from maximized to restored

			// Kill off our viewport overlay now that the animation has finished
			ViewportsOverlayPtr.Pin()->RemoveSlot();
		}

		// Stop transitioning
		if( !bIsImmersive && !bIsMaximized )
		{
			// We're finished with this temporary overlay widget now
			ViewportsOverlayWidget.Reset();

			// Restore the viewport widget into the viewport layout splitter
			LayoutConfiguration->ReplaceWidget( ViewportReplacementWidget.ToSharedRef(), MaximizedViewportEntity->AsWidget() );

			MaximizedViewport = NAME_None;
		}
		bIsTransitioning = false;


		// Update keyboard focus.  Focus is usually lost when we re-parent the viewport widget.
		{
			// We first need to clear keyboard focus so that Slate doesn't assume that focus won't need to change
			// simply because the viewport widget object is the same -- it has a new widget path!
			FSlateApplication::Get().ClearKeyboardFocus( EFocusCause::SetDirectly );

			// Set keyboard focus directly
			MaximizedViewportEntity->SetKeyboardFocus();
		}

		// If this is a PIE window we need to re-register since the maximized window will have registered itself
		// as the game viewport.
		MaximizedViewportEntity->RegisterGameViewportIfPIE();
	}
}


void FLevelViewportLayout::Tick( float DeltaTime )
{
	// If we have an animation that has finished playing, then complete the transition
	if( bIsTransitioning && !MaximizeAnimation.IsPlaying() )
	{
		FinishMaximizeTransition();
	}

	/** Resolve any maximizes or immersive commands for the viewports */
	if (DeferredMaximizeCommands.Num() > 0)
	{
		// Allow the engine to tick normally.
		EndThrottleForAnimatedResize();

		for (int32 i = 0; i < DeferredMaximizeCommands.Num(); ++i)
		{
			FMaximizeViewportCommand& Command = DeferredMaximizeCommands[i];

			// Only bother with deferred maximize if we don't already have a maximized or immersive viewport unless we are toggling
			if( MaximizedViewport.IsNone() || Command.bToggle )
			{
				MaximizeViewport(Command.Viewport, Command.bMaximize, Command.bImmersive, Command.bAllowAnimation );
			}
		}
		DeferredMaximizeCommands.Empty();
	}
}

bool FLevelViewportLayout::IsTickable() const
{
	return DeferredMaximizeCommands.Num() > 0 || ( bIsTransitioning && !MaximizeAnimation.IsPlaying() );
}

void FLevelViewportLayout::LoadConfig(const FString& LayoutString)
{
	FAssetEditorViewportLayout::LoadConfig(LayoutString);

	if (!LayoutConfiguration.IsValid())
	{
		return;
	}

	LayoutConfiguration->LoadConfig(LayoutString, [this](const FString& SpecificLayoutString, const FName PerspectiveViewportName)
	{
		this->InitCommonLayoutFromString(SpecificLayoutString, PerspectiveViewportName);
	});
}

void FLevelViewportLayout::SaveConfig(const FString& LayoutString) const
{
	FAssetEditorViewportLayout::SaveConfig(LayoutString);

	if (IsTransitioning() || !LayoutConfiguration.IsValid())
	{
		return;
	}

	LayoutConfiguration->SaveConfig(LayoutString, [this](const FString& SpecificLayoutString)
	{
		this->SaveCommonLayoutString(SpecificLayoutString);
	});
}
