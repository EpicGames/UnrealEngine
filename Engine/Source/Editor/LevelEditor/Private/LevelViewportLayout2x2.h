// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportLayout2x2.h"

UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayout2x2 instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayout2x2 FLevelViewportLayout2x2;
