// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelViewportLayoutEntity.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Editor/UnrealEdEngine.h"
#include "LevelEditorViewport.h"
#include "UnrealEdGlobals.h"
#include "SLevelViewport.h"

FLevelViewportLayoutEntity::FLevelViewportLayoutEntity(TSharedPtr<SAssetEditorViewport> InLevelViewport)
	: LevelViewport(StaticCastSharedRef<SLevelViewport>(InLevelViewport.ToSharedRef()))
{
	InLevelViewport->SetTag("LevelViewport");
}

TSharedRef<SWidget> FLevelViewportLayoutEntity::AsWidget() const
{
	return LevelViewport;
}

TSharedPtr<SLevelViewport> FLevelViewportLayoutEntity::AsLevelViewport() const
{
	return LevelViewport;
}

FName FLevelViewportLayoutEntity::GetType() const
{
	static FName DefaultName("Default");
	return DefaultName;
}

FLevelEditorViewportClient& FLevelViewportLayoutEntity::GetLevelViewportClient() const
{
	return LevelViewport->GetLevelViewportClient();
}

bool FLevelViewportLayoutEntity::IsPlayInEditorViewportActive() const
{
	return LevelViewport->IsPlayInEditorViewportActive();
}

void FLevelViewportLayoutEntity::RegisterGameViewportIfPIE()
{
	return LevelViewport->RegisterGameViewportIfPIE();
}

void FLevelViewportLayoutEntity::SetKeyboardFocus()
{
	return LevelViewport->SetKeyboardFocusToThisViewport();
}

void FLevelViewportLayoutEntity::OnLayoutDestroyed()
{
	if (LevelViewport->IsPlayInEditorViewportActive() || LevelViewport->GetLevelViewportClient().IsSimulateInEditorViewport() )
	{
		GUnrealEd->EndPlayMap();
	}
}

void FLevelViewportLayoutEntity::SaveConfig(const FString& ConfigSection)
{
	LevelViewport->SaveConfig(ConfigSection);
}

void FLevelViewportLayoutEntity::TakeHighResScreenShot() const
{
	GetLevelViewportClient().TakeHighResScreenShot();
}