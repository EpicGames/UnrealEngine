// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportLayoutFourPanes.h"

UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutFourPanesLeft instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutFourPanesLeft FLevelViewportLayoutFourPanesLeft;

UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutFourPanesRight instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutFourPanesRight FLevelViewportLayoutFourPanesRight;

UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutFourPanesTop instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutFourPanesTop FLevelViewportLayoutFourPanesTop;

UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutFourPanesBottom instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutFourPanesBottom FLevelViewportLayoutFourPanesBottom;
