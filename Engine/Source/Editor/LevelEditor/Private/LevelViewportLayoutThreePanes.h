// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportLayoutThreePanes.h"

UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutThreePanesLeft instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutThreePanesLeft FLevelViewportLayoutThreePanesLeft;


UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutThreePanesRight instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutThreePanesRight FLevelViewportLayoutThreePanesRight;


UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutThreePanesTop instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutThreePanesTop FLevelViewportLayoutThreePanesTop;


UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutThreePanesBottom instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutThreePanesBottom FLevelViewportLayoutThreePanesBottom;
