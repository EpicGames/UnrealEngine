// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportLayoutTwoPanes.h"

UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutTwoPanesVert instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutTwoPanesVert FLevelViewportLayoutTwoPanesVert;

UE_DEPRECATED(5.0, "Level Pane layouts are now owned by their level layouts. Use FEditorViewportLayoutTwoPanesHoriz instead, and add any custom types to FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName.")
typedef FEditorViewportLayoutTwoPanesHoriz FLevelViewportLayoutTwoPanesHoriz;
