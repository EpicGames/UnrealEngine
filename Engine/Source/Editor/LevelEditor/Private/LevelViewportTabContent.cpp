// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelViewportTabContent.h"
#include "Misc/ConfigCacheIni.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutService.h"
#include "Modules/ModuleManager.h"
#include "LevelEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "LevelViewportLayout.h"
#include "LevelEditorViewport.h"
#include "LevelViewportActions.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "ToolMenus.h"
#include "ViewportTypeDefinition.h"
#include "Templates/SharedPointer.h"
#include "SLevelViewport.h"
#include "LevelViewportLayoutEntity.h"

// FLevelViewportTabContent ///////////////////////////

TSharedPtr< FEditorViewportLayout > FLevelViewportTabContent::FactoryViewportLayout(bool bIsSwitchingLayouts)
{
	TSharedPtr<FLevelViewportLayout> ViewportLayout = MakeShareable(new FLevelViewportLayout);
	ViewportLayout->SetIsReplacement(bIsSwitchingLayouts);
	return ViewportLayout;
}

FName FLevelViewportTabContent::GetLayoutTypeNameFromLayoutString() const
{
	const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

	FString LayoutTypeString;
	if (LayoutString.IsEmpty() ||
		!GConfig->GetString(*IniSection, *(LayoutString + TEXT(".LayoutType")), LayoutTypeString, GEditorPerProjectIni))
	{
		return EditorViewportConfigurationNames::FourPanes2x2;
	}

	return *LayoutTypeString;
}

FLevelViewportTabContent::~FLevelViewportTabContent()
{
	if (GEditor)
	{
		GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
	}
}

void FLevelViewportTabContent::Initialize(AssetEditorViewportFactoryFunction Func, TSharedPtr<SDockTab> InParentTab, const FString& InLayoutString)
{
	check(InParentTab.IsValid());
	InParentTab->SetOnPersistVisualState( SDockTab::FOnPersistVisualState::CreateSP(this, &FLevelViewportTabContent::SaveConfig) );

	OnViewportTabContentLayoutStartChangeEvent.AddSP(this, &FLevelViewportTabContent::OnLayoutStartChange);
	OnViewportTabContentLayoutChangedEvent.AddSP(this, &FLevelViewportTabContent::OnLayoutChanged);

	FEditorViewportTabContent::Initialize(Func, InParentTab, InLayoutString);
}

void FLevelViewportTabContent::BindViewportLayoutCommands(FUICommandList& InOutCommandList, FName ViewportConfigKey)
{
	FEditorViewportTabContent::BindViewportLayoutCommands(InOutCommandList, ViewportConfigKey);

	FLevelViewportCommands& ViewportActions = FLevelViewportCommands::Get();

	auto AddViewportConfigurationAction = [&InOutCommandList, this](const TSharedPtr<FUICommandInfo>& InCommandInfo, FName InLayoutType)
	{
		InOutCommandList.MapAction(
			InCommandInfo,
			FExecuteAction::CreateSP(this, &FLevelViewportTabContent::OnUIActionSetViewportConfiguration, InLayoutType),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FLevelViewportTabContent::IsViewportConfigurationChecked, InLayoutType));
	};

	AddViewportConfigurationAction(ViewportActions.ViewportConfig_OnePane, LevelViewportConfigurationNames::OnePane);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_TwoPanesH, LevelViewportConfigurationNames::TwoPanesHoriz);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_TwoPanesV, LevelViewportConfigurationNames::TwoPanesVert);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_ThreePanesLeft, LevelViewportConfigurationNames::ThreePanesLeft);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_ThreePanesRight, LevelViewportConfigurationNames::ThreePanesRight);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_ThreePanesTop, LevelViewportConfigurationNames::ThreePanesTop);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_ThreePanesBottom, LevelViewportConfigurationNames::ThreePanesBottom);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_FourPanesLeft, LevelViewportConfigurationNames::FourPanesLeft);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_FourPanesRight, LevelViewportConfigurationNames::FourPanesRight);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_FourPanesTop, LevelViewportConfigurationNames::FourPanesTop);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_FourPanesBottom, LevelViewportConfigurationNames::FourPanesBottom);
	AddViewportConfigurationAction(ViewportActions.ViewportConfig_FourPanes2x2, LevelViewportConfigurationNames::FourPanes2x2);

	auto ProcessViewportTypeActions = [&InOutCommandList, ViewportConfigKey, this](FName InViewportTypeName, const FViewportTypeDefinition& InDefinition) {
		if (InDefinition.ActivationCommand.IsValid())
		{
			InOutCommandList.MapAction(InDefinition.ActivationCommand, FUIAction(
				FExecuteAction::CreateSP(this, &FLevelViewportTabContent::OnUIActionSetViewportTypeWithinLayout, ViewportConfigKey, InViewportTypeName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FLevelViewportTabContent::IsViewportTypeWithinLayoutEqual, ViewportConfigKey, InViewportTypeName)
			));
		}
	};
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	LevelEditorModule.IterateViewportTypes(ProcessViewportTypeActions);
}

void FLevelViewportTabContent::OnLayoutStartChange(bool bSwitchingLayouts)
{
	GCurrentLevelEditingViewportClient = nullptr;
	GLastKeyLevelEditingViewportClient = nullptr;
}

void FLevelViewportTabContent::OnLayoutChanged()
{
	// Set the global level editor viewport when the layout changes
	TSharedPtr<FLevelViewportLayout> Layout = StaticCastSharedPtr<FLevelViewportLayout>(ActiveViewportLayout);
	const bool bIsLayoutMaximized = Layout->IsMaximizeSupported() && Layout->IsMaximized();
	FLevelEditorViewportClient* ClientToSetAsCurrent = nullptr;

	for (const TPair<FName, TSharedPtr<IEditorViewportLayoutEntity>>& Viewport : ActiveViewportLayout->GetViewports())
	{
		ILevelViewportLayoutEntity* LevelLayoutEntity = static_cast<ILevelViewportLayoutEntity*>(Viewport.Value.Get());
		// Prefer the maximized viewport in the layout, as it will be the only one visible
		if (bIsLayoutMaximized)
		{
			if (Layout->IsViewportMaximized(Viewport.Key))
			{
				ClientToSetAsCurrent = &LevelLayoutEntity->GetLevelViewportClient();
				break;
			}
		}
		// Or the first perspective viewport we encounter in the layout
		else if (LevelLayoutEntity->GetLevelViewportClient().IsPerspective())
		{
			ClientToSetAsCurrent = &LevelLayoutEntity->GetLevelViewportClient();
			break;
		}
		// Or use the first viewport if we're not maximized and there is no perspective viewport
		else if (!ClientToSetAsCurrent)
		{
			ClientToSetAsCurrent = &LevelLayoutEntity->GetLevelViewportClient();
		}
	}

	if (ensure(ClientToSetAsCurrent))
	{
		ClientToSetAsCurrent->SetCurrentViewport();
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnTabContentChanged().Broadcast();
}

void FLevelViewportTabContent::OnUIActionSetViewportConfiguration(FName InConfigurationName)
{
	SetViewportConfiguration(InConfigurationName);

	FSlateApplication::Get().DismissAllMenus();
	UToolMenus::Get()->CleanupStaleWidgetsNextTick(true);
}

FName FLevelViewportTabContent::GetViewportTypeWithinLayout(FName InConfigKey) const
{
	if (ActiveViewportLayout && !InConfigKey.IsNone())
	{
		if (TSharedPtr<IEditorViewportLayoutEntity> LayoutEntity = ActiveViewportLayout->GetViewports().FindRef(InConfigKey))
		{
			return LayoutEntity->GetType();
		}
	}

	return "Default";
}

void FLevelViewportTabContent::OnUIActionSetViewportTypeWithinLayout(FName InConfigKey, FName InLayoutType)
{
	if (!IsViewportTypeWithinLayoutEqual(InConfigKey, InLayoutType))
	{
		SaveConfig();

		const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();
		GConfig->SetString(*IniSection, *(InConfigKey.ToString() + TEXT(".TypeWithinLayout")), *InLayoutType.ToString(), GEditorPerProjectIni);

		// Force a refresh of the tab content and clear all menus
		RefreshViewportConfiguration();
		FSlateApplication::Get().DismissAllMenus();
	}
}

bool FLevelViewportTabContent::IsViewportTypeWithinLayoutEqual(FName InConfigKey, FName InLayoutType) const
{
	return (GetViewportTypeWithinLayout(InConfigKey) == InLayoutType);
}

bool FLevelViewportTabContent::IsViewportConfigurationChecked(FName InLayoutType) const
{
	// Only visually indicate that a configuration is set if the viewport is not currently maximized 
	return IsViewportConfigurationSet(InLayoutType) && !IsLayoutMaximized();
}

bool FLevelViewportTabContent::IsLayoutMaximized() const
{
	if (TSharedPtr<FLevelViewportLayout> Layout = StaticCastSharedPtr<FLevelViewportLayout>(ActiveViewportLayout))
	{
		return Layout->IsMaximized();
	}
	return false;
}

