// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightEditorSubsystem.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "Selection.h"
#include "Engine/SpotLight.h"
#include "Engine/PointLight.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "LevelEditorMenuContext.h"
#include "ActorFactories/ActorFactory.h"
#include "Editor.h"
#include "Subsystems/EditorActorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightEditorSubsystem)

#define LOCTEXT_NAMESPACE "EditorLightSubsystem"

void FLightEditingCommands::RegisterCommands()
{
	UI_COMMAND(SwapLightType, "Add Key Frame", "Inserts a new key frame at the current time", EUserInterfaceActionType::Button, FInputChord(EKeys::P, EModifierKey::Shift));
}


void ULightEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FLightEditingCommands::Register();
	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &ULightEditorSubsystem::ExtendQuickActionMenu));
}

void ULightEditorSubsystem::Deinitialize()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);
}

void ULightEditorSubsystem::ExtendQuickActionMenu()
{
	FToolMenuOwnerScoped MenuOwner(this);
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.InViewportPanel");
	{
		FToolMenuSection& Section = Menu->FindOrAddSection("QuickActions");
		FToolMenuEntry& Entry = Section.AddDynamicEntry("Lights", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
			{
				UQuickActionMenuContext* Context = InSection.FindContext<UQuickActionMenuContext>();
				if (Context && Context->CurrentSelection && Context->CurrentSelection->GetElementList()->Num() > 0)
				{
					// Only SpotLights Selected
					if (Context->CurrentSelection->CountSelectedObjects<ASpotLight>() == Context->CurrentSelection->GetNumSelectedElements())
					{
						FToolMenuEntry& SwapLightEntry = InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
							"SwapLightTypes",
							FToolUIAction(FToolMenuExecuteAction::CreateUObject(this, &ULightEditorSubsystem::SwapLightType, APointLight::StaticClass())),
							LOCTEXT("SwapToPointLights", "Swap to Point Lights"),
							LOCTEXT("SwapToPointLightsToolTip", "Changes all selected spot lights to point lights."),
							FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PlacementBrowser.Icons.Lights")
						));
						SwapLightEntry.AddKeybindFromCommand(FLightEditingCommands::Get().SwapLightType);
					}
					// Only PointLights selected
					if (Context->CurrentSelection->CountSelectedObjects<APointLight>() == Context->CurrentSelection->GetNumSelectedElements())
					{
						FToolMenuEntry& SwapLightEntry = InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
							"SwapLightTypes",
							FToolUIAction(FToolMenuExecuteAction::CreateUObject(this, &ULightEditorSubsystem::SwapLightType, ASpotLight::StaticClass())),
							LOCTEXT("SwapToSpotLights", "Swap to Spot Lights"),
							LOCTEXT("SwapToSpotLightsToolTip", "Changes all selected point lights to spot lights."),
							FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PlacementBrowser.Icons.Lights")
						));
						SwapLightEntry.AddKeybindFromCommand(FLightEditingCommands::Get().SwapLightType);
					}
				}
			}));
	}

}

void ULightEditorSubsystem::SwapLightType(const FToolMenuContext& InContext, UClass* InClass) const
{
	const FAssetData NoAssetData;
	UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass(InClass);
	if (ActorFactory)
	{
		UEditorActorSubsystem::ReplaceSelectedActors(ActorFactory, NoAssetData);
	}
}





#undef LOCTEXT_NAMESPACE
