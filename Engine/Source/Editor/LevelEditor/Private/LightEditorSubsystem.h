// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "EditorUndoClient.h"
#include "Widgets/Views/SListView.h"
#include "IDetailPropertyRow.h"
#include "EditorSubsystem.h"
#include "UObject/ObjectMacros.h"
#include "Framework/Commands/Commands.h"
#include "LightEditorSubsystem.generated.h"

struct FToolMenuContext;
/**
 * Light manipulation actions
 */
class FLightEditingCommands : public TCommands<FLightEditingCommands>
{

public:
	FLightEditingCommands() : TCommands<FLightEditingCommands>
		(
			"LightActor", // Context name for fast lookup
			NSLOCTEXT("Contexts", "LightActor", "Light Actor"), // Localized context name for displaying
			NAME_None, FAppStyle::GetAppStyleSetName()
			)
	{
	}

	TSharedPtr< FUICommandInfo > SwapLightType;

	virtual void RegisterCommands() override;
};

UCLASS()
class ULightEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()
public:
	ULightEditorSubsystem() {};
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	virtual void Deinitialize() override;
	void ExtendQuickActionMenu();
	void SwapLightType(const FToolMenuContext& InContext, UClass* InClass) const;
	// The list of commands with bound delegates for the level editor.
	TSharedPtr<FUICommandList> LightEditingCommands;
};

