// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	LightmapResRatioAdjust.cpp: Lightmap Resolution Ratio Adjustment helper
================================================================================*/

#include "LightmapResRatioAdjust.h"
#include "UObject/Object.h"
#include "UObject/UObjectIterator.h"
#include "GameFramework/Actor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Brush.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "EditorSupportDelegates.h"
#include "SurfaceIterators.h"
#include "Logging/MessageLog.h"
#include "ActorEditorUtils.h"
#include "Engine/StaticMesh.h"
#include "Engine/Polys.h"
#include "Engine/LevelStreaming.h"

#define LOCTEXT_NAMESPACE "LightmapResRatioAdjustSettings"

/**
 *	LightmapResRatioAdjust settings
 */
/** Static: Global lightmap resolution ratio adjust settings */
FLightmapResRatioAdjustSettings FLightmapResRatioAdjustSettings::LightmapResRatioAdjustSettings;

bool FLightmapResRatioAdjustSettings::ApplyRatioAdjustment()
{
	FLightmapResRatioAdjustSettings& Settings = Get();
	FMessageLog EditorErrors("EditorErrors");

	if ((Settings.bStaticMeshes == false) &&
		(Settings.bBSPSurfaces == false))
	{
		// No primitive type is selected...
		EditorErrors.Warning(LOCTEXT("LMRatioAdjust_NoPrimitivesSelected", "No primitive type selected."));
		EditorErrors.Notify();
		return false;
	}

	bool bRebuildGeometry = false;
	bool bRefreshViewport = false;

	// Collect the levels to process
	//@todo. This code is done in a few places for lighting related changes...
	// It should be moved to a centralized place to remove duplication.
	TArray<UWorld*> Worlds;
	TArray<UStaticMeshComponent*> StaticMeshComponents;
	TArray<ULevel*> MeshLevels;
	TArray<ULevel*> BrushLevels;
	// Gather the relevant levels, static meshes and BSP components.
	for (TObjectIterator<UObject> ObjIt;  ObjIt; ++ObjIt)
	{
		UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(*ObjIt);

		if (StaticMeshComp && (Settings.bStaticMeshes == true))
		{
			bool AddThisItem = false;
			if (Settings.bSelectedObjectsOnly == true)
			{
				AActor* OwnerActor = StaticMeshComp->GetOwner();
				if (OwnerActor && OwnerActor->IsSelected() == true)
				{
					AddThisItem = true;
				}
			}
			else
			{
				AddThisItem = true;				
			}
			if( AddThisItem )
			{
				StaticMeshComponents.AddUnique(StaticMeshComp);
				// Store the required levels relating to this object based on the level option settings
				UWorld* MeshWorld = StaticMeshComp->GetWorld();
				// Exclude editor preview worlds				
				if( MeshWorld && (MeshWorld->WorldType != EWorldType::EditorPreview && MeshWorld->WorldType != EWorldType::Inactive) )
				{
					AddRequiredLevels( Settings.LevelOptions, MeshWorld , MeshLevels );
					Worlds.AddUnique( MeshWorld );
				}
				
			}
		}
		// Check if this object is a brush or has any brushes attached
		AActor* Actor = Cast<AActor>(*ObjIt);
		if ( Actor && Settings.bBSPSurfaces == true)
		{
			ABrush* Brush = Cast< ABrush >( Actor );
			if (Brush && !FActorEditorUtils::IsABuilderBrush(Brush))
			{
				UWorld* BrushWorld = Brush->GetWorld();
				// Exclude editor preview worlds
				if( BrushWorld->WorldType != EWorldType::EditorPreview && BrushWorld->WorldType != EWorldType::Inactive )
				{
					AddRequiredLevels( Settings.LevelOptions, BrushWorld, BrushLevels );				
					Worlds.AddUnique( BrushWorld );
				}
			}
			else
			{
				// Search for any brushes attached to an actor
				TArray<AActor*> AttachedActors;
				Actor->GetAttachedActors( AttachedActors );

				TArray<AActor*> AttachedBrushes;
				// Get any brush actors attached to this one
				if( ContainsObjectOfClass( AttachedActors, ABrush::StaticClass(), EFindObjectFlags::ExactClass, &AttachedBrushes ) )
				{
					for( int32 BrushIndex = 0; BrushIndex < AttachedBrushes.Num(); ++BrushIndex )
					{
						ABrush* AttachedBrush = CastChecked<ABrush>( AttachedBrushes[BrushIndex] );
						UWorld* BrushWorld = AttachedBrush->GetWorld();
						// Exclude editor preview worlds
						if( BrushWorld->WorldType != EWorldType::EditorPreview && BrushWorld->WorldType != EWorldType::Inactive )
						{	
							AddRequiredLevels( Settings.LevelOptions, BrushWorld, BrushLevels );
							Worlds.AddUnique( BrushWorld );
						}
					}
				}
			}
		}
	}

	if ( (MeshLevels.Num() == 0) && ( BrushLevels.Num() == 0 ) )
	{
		// No levels are selected...
		EditorErrors.Warning( LOCTEXT("LMRatioAdjust_NoLevelsToProcess", "No levels to process.") );
		EditorErrors.Notify();
		return false;
	}
	
	// There should only be one world - exit if there isnt !
	if ( Worlds.Num() != 1 )
	{
		// No levels are selected...
		EditorErrors.Warning( LOCTEXT("LMRatioAdjust_TooManyWorldsToProcess", "you can only process one world at a time.") );
		EditorErrors.Notify();
		return false;
	}
	UWorld* World = Worlds[0];
	// Attempt to apply the modification to the static meshes in the relevant levels
	for (int32 LevelIdx = 0; LevelIdx < MeshLevels.Num(); LevelIdx++)
	{		
		ULevel* EachLevel = MeshLevels[ LevelIdx ];
		for (int32 StaticMeshIdx = 0; StaticMeshIdx < StaticMeshComponents.Num(); StaticMeshIdx++)
		{
			UStaticMeshComponent* SMComp = StaticMeshComponents[StaticMeshIdx];
			if (SMComp && SMComp->IsIn(EachLevel) == true)
			{
				// Process it!
				int32 CurrentResolution = SMComp->GetStaticLightMapResolution();
				bool bConvertIt = true;
				if (((SMComp->bOverrideLightMapRes == true) && (CurrentResolution == 0)) ||
					((SMComp->bOverrideLightMapRes == false) && (SMComp->GetStaticMesh() != nullptr) && (SMComp->GetStaticMesh()->GetLightMapResolution() == 0)))
				{
					// Don't convert vertex mapped objects!
					bConvertIt = false;
				}
				else if ((Settings.Ratio >= 1.0f) && (CurrentResolution >= Settings.Max_StaticMeshes))
				{
					bConvertIt = false;
				}
				else if ((Settings.Ratio < 1.0f) && (CurrentResolution <= Settings.Min_StaticMeshes))
				{
					bConvertIt = false;
				}
				else if( SMComp->GetWorld() && (SMComp->GetWorld()->WorldType == EWorldType::EditorPreview || SMComp->GetWorld()->WorldType == EWorldType::Inactive) )
				{
					// Don't do objects with an editor preview or inactive world
					bConvertIt = false;
				}
				if (bConvertIt)
				{
					SMComp->Modify();
					float AdjustedResolution = (float)CurrentResolution * (Settings.Ratio);
					int32 NewResolution = FMath::TruncToInt(AdjustedResolution);
					NewResolution = FMath::Max(NewResolution + 3 & ~3,4);
					SMComp->SetStaticLightingMapping(true, NewResolution);
					SMComp->InvalidateLightingCache();
					SMComp->MarkRenderStateDirty();
					bRefreshViewport = true;
				}
			}
		}

	}
	// Try to update all surfaces in this level...
	if (Settings.bBSPSurfaces == true)
	{
		ULevel* OrigCurrentLevel = World->GetCurrentLevel();
		for (int32 LevelIdx = 0; LevelIdx < BrushLevels.Num(); LevelIdx++)
		{
			ULevel* EachLevel = BrushLevels[ LevelIdx ];
			World->SetCurrentLevel( EachLevel );
			for (TSurfaceIterator<FCurrentLevelSurfaceLevelFilter> It(World); It; ++It)
			{
				UModel* Model = It.GetModel();
				bool bConvertIt = true;
				const int32 SurfaceIndex = It.GetSurfaceIndex();
				FBspSurf& Surf = Model->Surfs[SurfaceIndex];
				if (Settings.bSelectedObjectsOnly == true)
				{
					if ((Surf.PolyFlags & PF_Selected) == 0)
					{
						bConvertIt = false;
					}
				}

				if (bConvertIt == true)
				{
					// Process it!
					int32 CurrentResolution = FMath::TruncToInt32(Surf.LightMapScale);
					float Scalar = 1.0f / Settings.Ratio;
					if ((Scalar < 1.0f) && (CurrentResolution <= Settings.Min_BSPSurfaces))
					{
						bConvertIt = false;
					}
					else if ((Scalar >= 1.0f) && (CurrentResolution >= Settings.Max_BSPSurfaces))
					{
						bConvertIt = false;
					}

					if (bConvertIt == true)
					{
						Model->ModifySurf( SurfaceIndex, true );
						int32 NewResolution = FMath::TruncToInt(Surf.LightMapScale * Scalar);
						NewResolution = FMath::Max(NewResolution + 3 & ~3,4);
						Surf.LightMapScale = (float)NewResolution;
						if (Surf.Actor != NULL)
						{
							Surf.Actor->Brush->Polys->Element[Surf.iBrushPoly].LightMapScale = Surf.LightMapScale;
						}
						bRefreshViewport = true;
						bRebuildGeometry = true;
					}
				}
			}
		}
		World->SetCurrentLevel( OrigCurrentLevel );
	}

	if (bRebuildGeometry == true)
	{
		GUnrealEd->Exec( World, TEXT("MAP REBUILD") );
	}
	if (bRefreshViewport == true)
	{
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
		FEditorSupportDelegates::RefreshPropertyWindows.Broadcast();
	}

	return true;
}

void FLightmapResRatioAdjustSettings::AddRequiredLevels( AdjustLevels InLevelOptions, UWorld* InWorld, TArray<ULevel*>& OutLevels )
{
	switch (InLevelOptions)
	{
	case AdjustLevels::Current:
		{
			check(InWorld);
			OutLevels.AddUnique(InWorld->GetCurrentLevel());
		}
		break;
	case AdjustLevels::Selected:
		{
			TArray<TObjectPtr<class ULevel>>& SelectedLevels = InWorld->GetSelectedLevels();
			for(auto It = SelectedLevels.CreateIterator(); It; ++It)
			{
				ULevel* Level = *It;
				if (Level)
				{
					OutLevels.AddUnique(Level);
				}
			}

			if (OutLevels.Num() == 0)
			{
				// Fall to the current level...
				check(InWorld);
				OutLevels.AddUnique(InWorld->GetCurrentLevel());
			}
		}
		break;
	case AdjustLevels::AllLoaded:
		{
			if (InWorld != NULL)
			{
				// Add main level.
				OutLevels.AddUnique(InWorld->PersistentLevel);

				// Add secondary levels.
				for (ULevelStreaming* StreamingLevel : InWorld->GetStreamingLevels())
				{
					if (StreamingLevel)
					{
						if (ULevel* Level = StreamingLevel->GetLoadedLevel())
						{
							OutLevels.AddUnique(Level);
						}
					}
				}
			}
		}
		break;
	}
}

#undef LOCTEXT_NAMESPACE
