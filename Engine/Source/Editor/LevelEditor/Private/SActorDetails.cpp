// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActorDetails.h"

#include "ActorDetailsSCSEditorUICustomization.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "DetailsViewObjectFilter.h"
#include "Editor.h"
#include "EditorLevelUtils.h"
#include "Editor/UnrealEdEngine.h"
#include "Styling/AppStyle.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Selection.h"
#include "GameFramework/Actor.h"
#include "HAL/FileManager.h"
#include "IDetailRootObjectCustomization.h"
#include "IDetailsView.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "LevelEditor.h"
#include "LevelEditorGenericDetails.h"
#include "LevelEditorSubsystem.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SSubobjectEditor.h"
#include "SSubobjectEditorModule.h"
#include "SSubobjectInstanceEditor.h"
#include "ScopedTransaction.h"
#include "SourceCodeNavigation.h"
#include "SubobjectData.h"
#include "SubobjectDataSubsystem.h"
#include "UnrealEdGlobals.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementHandleColumn.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SActorDetails"

namespace UE::LevelEditor::Private
{
	class SElementSelectionDetailsButtons : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SElementSelectionDetailsButtons)
		{}

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<SWidget> SubobjectEditorButtonBox, TFunction<TArray<TTypedElement<ITypedElementDetailsInterface>>()>&& InGetDetailsHandles)
		{
			GetDetailsHandles = MoveTemp(InGetDetailsHandles);
	
			ChildSlot
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(PromoteElementButton, SButton)
						.ButtonStyle(FAppStyle::Get(), "DetailsView.NameAreaButton")
						.ContentPadding(0.f)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Element.PromoteElement")))
						.OnClicked(this, &SElementSelectionDetailsButtons::OnPromoteElement)
						.ToolTipText(LOCTEXT("PromoteElementTooltip", "Promote the selected elements."))
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.PromoteElements"))
						]
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SubobjectEditorButtonBox
					]
				];
		}

		void UpdateSelection(TArrayView<const TTypedElement<ITypedElementDetailsInterface>>* InDetailsElementsPtr)
		{
			TArrayView<const TTypedElement<ITypedElementDetailsInterface>> DetailsElementsView;
			TArray<TTypedElement<ITypedElementDetailsInterface>> DetailsElementsContainer;

			if (InDetailsElementsPtr)
			{
				DetailsElementsView = *InDetailsElementsPtr;
			}
			else
			{
				DetailsElementsContainer = GetDetailsHandles();
				DetailsElementsView = MakeArrayView(DetailsElementsContainer);
			}

			bool bCanPromoteAnElement = false;
			UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
			for (const TTypedElement<ITypedElementDetailsInterface>& DetailsElement : DetailsElementsView)
			{
				if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(DetailsElement))
				{
					if (WorldElement.CanPromoteElement())
					{
						bCanPromoteAnElement = true;
						break;
					}
				}
			}

			if (bCanPromoteAnElement)
			{
				PromoteElementButton->SetVisibility(EVisibility::Visible);
			}
			else
			{
				PromoteElementButton->SetVisibility(EVisibility::Collapsed);
			}
		}

	private:
		FReply OnPromoteElement()
		{
			FScopedTransaction Transaction(LOCTEXT("PromoteElementsTransaction", "Promote Elements"));

			TArray<TTypedElement<ITypedElementDetailsInterface>> DetailsElements = GetDetailsHandles();

			TArray<AActor*> Actors;
			Actors.Reserve(DetailsElements.Num());

			UTypedElementSelectionSet* SelectionSet = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>()->GetSelectionSet();
			FTypedElementSelectionOptions SelectionOptions;
			UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

			for (const TTypedElement<ITypedElementDetailsInterface>& DetailsElement : DetailsElements)
			{
				if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(DetailsElement))
				{
					if (WorldElement.CanPromoteElement())
					{
						SelectionSet->DeselectElement(WorldElement, SelectionOptions);
					}

					if (FTypedElementHandle PromotedElement = WorldElement.PromoteElement())
					{
						SelectionSet->SelectElement(PromotedElement, SelectionOptions);
					}
				}
			}

			return FReply::Handled();
		}

		TSharedPtr<SButton> PromoteElementButton;
		TFunction<TArray<TTypedElement<ITypedElementDetailsInterface>>()> GetDetailsHandles;
	};
}

class SActorDetailsUneditableComponentWarning : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SActorDetailsUneditableComponentWarning)
		: _WarningText()
		, _OnHyperlinkClicked()
	{}
		
		/** The rich text to show in the warning */
		SLATE_ATTRIBUTE(FText, WarningText)

		/** Called when the hyperlink in the rich text is clicked */
		SLATE_EVENT(FSlateHyperlinkRun::FOnClick, OnHyperlinkClicked)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
				]
				+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(2.f)
					[
						SNew(SRichTextBlock)
						.DecoratorStyleSet(&FAppStyle::Get())
						.Justification(ETextJustify::Left)
						.TextStyle(FAppStyle::Get(), "DetailsView.BPMessageTextStyle")
						.Text(InArgs._WarningText)
						.AutoWrapText(true)
						+ SRichTextBlock::HyperlinkDecorator(TEXT("HyperlinkDecorator"), InArgs._OnHyperlinkClicked)
					]
			]
		];
	}
};

class SActorDetails::FInternalSubobjectCustomization : public ISCSEditorUICustomization
{
public:

	explicit FInternalSubobjectCustomization(TSharedPtr<FActorDetailsSCSEditorUICustomization>& ParentCustomization);
	virtual ~FInternalSubobjectCustomization() override = default;

	void SetHideAddComponentButton(bool bShouldHide);
	void SetHideBlueprintButton(bool bShouldHide);
	
	virtual bool HideAddComponentButton(TArrayView<UObject*> Context) const override;
	virtual bool HideBlueprintButtons(TArrayView<UObject*> Context) const override;
	virtual bool HideComponentsTree(TArrayView<UObject*> Context) const override;
	virtual bool HideComponentsFilterBox(TArrayView<UObject*> Context) const override;
	virtual const FSlateBrush* GetIconBrush(const FSubobjectData&) const override;
	virtual TSharedPtr<SWidget> GetControlsWidget(TSharedRef<SSubobjectEditor>& InSubobjectEditor, const FSubobjectData&) const override;
	virtual bool OverrideDesiredVisibility(const UE::Editor::FSubobjectEditorContext& Context, EVisibility& VisibilityOut) const override;
	virtual EChildActorComponentTreeViewVisualizationMode GetChildActorVisualizationMode() const override;
	virtual TSubclassOf<UActorComponent> GetComponentTypeFilter(TArrayView<UObject*> Context) const override;
	virtual bool FilterSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData) const override;
	virtual bool SortSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData) override;
	virtual bool SupportsTreeViewOverlay(const SSubobjectEditor& Editor) const override;
	virtual TSharedRef<class SWidget> ConstructTreeViewOverlay(TWeakPtr<class SSubobjectEditor> Editor) const override;
	virtual void PrePotentialTargetChange(SSubobjectEditor& SubobjectEditor) override;
	virtual bool SelectDefaultSubobject(SSubobjectEditor& SubobjectEditor) override;

private:
	TSharedPtr<FActorDetailsSCSEditorUICustomization> ParentCustomization;
	
	bool bHideAddComponentButton = true;
	bool bHideBlueprintButton = true;
};

bool SActorDetails::FInternalSubobjectCustomization::HideAddComponentButton(TArrayView<UObject*> Context) const
{
	if (const bool bParentHides = ParentCustomization && ParentCustomization->HideAddComponentButton(Context))
	{
		return bParentHides;
	}
	
	return bHideAddComponentButton;
}

bool SActorDetails::FInternalSubobjectCustomization::HideBlueprintButtons(TArrayView<UObject*> Context) const
{
	if (const bool bParentHides = ParentCustomization && ParentCustomization->HideBlueprintButtons(Context))
	{
		return bParentHides;
	}
	
	return bHideBlueprintButton;
}

bool SActorDetails::FInternalSubobjectCustomization::HideComponentsTree(TArrayView<UObject*> Context) const
{
	if (const bool bParentHides = ParentCustomization && ParentCustomization->HideComponentsTree(Context))
	{
		return bParentHides;
	}

	return false;
}

bool SActorDetails::FInternalSubobjectCustomization::HideComponentsFilterBox(TArrayView<UObject*> Context) const
{
	if (const bool bParentHides = ParentCustomization && ParentCustomization->HideComponentsFilterBox(Context))
	{
		return bParentHides;
	}

	return false;
}

const FSlateBrush* SActorDetails::FInternalSubobjectCustomization::GetIconBrush(const FSubobjectData& SubobjectData) const
{
	if (ParentCustomization)
	{
		return ParentCustomization->GetIconBrush(SubobjectData);
	}
	
	return ISCSEditorUICustomization::GetIconBrush(SubobjectData);
}

TSharedPtr<SWidget> SActorDetails::FInternalSubobjectCustomization::GetControlsWidget(TSharedRef<SSubobjectEditor>& InSubobjectEditor, const FSubobjectData& SubobjectData) const
{
	if (ParentCustomization)
	{
		return ParentCustomization->GetControlsWidget(InSubobjectEditor, SubobjectData);
	}

	return ISCSEditorUICustomization::GetControlsWidget(InSubobjectEditor, SubobjectData);
}

bool SActorDetails::FInternalSubobjectCustomization::OverrideDesiredVisibility(const UE::Editor::FSubobjectEditorContext& Context, EVisibility& VisibilityOut) const
{
	if (ParentCustomization)
	{
		if (ParentCustomization->OverrideDesiredVisibility(Context, VisibilityOut))
		{
			return true;
		}
	}
	return ISCSEditorUICustomization::OverrideDesiredVisibility(Context, VisibilityOut);
}

EChildActorComponentTreeViewVisualizationMode SActorDetails::FInternalSubobjectCustomization::GetChildActorVisualizationMode() const
{
	if (ParentCustomization)
	{
		return ParentCustomization->GetChildActorVisualizationMode();
	}

	return EChildActorComponentTreeViewVisualizationMode::UseDefault;
}

TSubclassOf<UActorComponent> SActorDetails::FInternalSubobjectCustomization::GetComponentTypeFilter(TArrayView<UObject*> Context) const
{
	if (ParentCustomization)
	{
		return ParentCustomization->GetComponentTypeFilter(Context);
	}

	return nullptr;
}

bool SActorDetails::FInternalSubobjectCustomization::FilterSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData) const
{
	if (ParentCustomization)
	{
		return ParentCustomization->FilterSubobjectData(SubobjectData);
	}
	return false;
}

bool SActorDetails::FInternalSubobjectCustomization::SortSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData)
{
	if (ParentCustomization)
	{
		return ParentCustomization->SortSubobjectData(SubobjectData);
	}
	return false;
}

bool SActorDetails::FInternalSubobjectCustomization::SupportsTreeViewOverlay(const SSubobjectEditor& Editor) const
{
	if (ParentCustomization)
	{
		return ParentCustomization->SupportsTreeViewOverlay(Editor);
	}
	return false;
}

TSharedRef<SWidget> SActorDetails::FInternalSubobjectCustomization::ConstructTreeViewOverlay(TWeakPtr<SSubobjectEditor> Editor) const
{
	if (ParentCustomization)
	{
		return ParentCustomization->ConstructTreeViewOverlay(Editor);
	}
	return SNullWidget::NullWidget;
}

void SActorDetails::FInternalSubobjectCustomization::PrePotentialTargetChange(SSubobjectEditor& InSubobjectEditor)
{
	if (ParentCustomization)
	{
		ParentCustomization->PrePotentialTargetChange(InSubobjectEditor);
	}
	ISCSEditorUICustomization::PrePotentialTargetChange(InSubobjectEditor);
}

bool SActorDetails::FInternalSubobjectCustomization::SelectDefaultSubobject(SSubobjectEditor& InSubobjectEditor)
{
	if (ParentCustomization)
	{
		return ParentCustomization->SelectDefaultSubobject(InSubobjectEditor);
	}
	return ISCSEditorUICustomization::SelectDefaultSubobject(InSubobjectEditor);
}

SActorDetails::FInternalSubobjectCustomization::FInternalSubobjectCustomization(
	TSharedPtr<FActorDetailsSCSEditorUICustomization>& InParentCustomization)
		: ParentCustomization(InParentCustomization)
{
}

void SActorDetails::FInternalSubobjectCustomization::SetHideAddComponentButton(bool bShouldHide)
{
	bHideAddComponentButton = bShouldHide;
}

void SActorDetails::FInternalSubobjectCustomization::SetHideBlueprintButton(bool bShouldHide)
{
	bHideBlueprintButton = bShouldHide;
}

namespace ActorDetails
{
	bool bCVarDisableComponentEditorButtonsAreaOnMultiSelect = false;
	FAutoConsoleVariableRef CVarLegacyAlwaysDisableComponentEditorButtonsAreaOnMultiSelect(
		TEXT("Editor.LevelEditor.LegacyAlwaysDisableComponentEditorButtonsAreaOnMultiSelect"),
		bCVarDisableComponentEditorButtonsAreaOnMultiSelect,
		TEXT("Disables all component editor buttons in a level editor details panel.  CVar used for backwards compatibility. Please update extension widgets to change visibility based on selected subobject nodes."),
		ECVF_Default);
}

void SActorDetails::Construct(
	const FArguments& InArgs,
	UTypedElementSelectionSet* InSelectionSet,
	const FName TabIdentifier,
	TSharedPtr<FUICommandList> InCommandList,
	TSharedPtr<FTabManager> InTabManager,
	TWeakPtr<FActorDetailsSCSEditorUICustomization> InLevelEditorSubobjectEditorCustomization)
{
	SelectionSet = InSelectionSet;
	checkf(SelectionSet, TEXT("SActorDetails must be constructed with a valid selection set!"));

	FCoreUObjectDelegates::OnObjectsReinstanced.AddRaw(this, &SActorDetails::OnObjectsReinstanced);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &SActorDetails::OnObjectPropertyChanged);

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnComponentsEdited().AddRaw(this, &SActorDetails::OnComponentsEditedInWorld);

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = true;
	DetailsViewArgs.bLockable = true;
	DetailsViewArgs.bAllowFavoriteSystem = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea | FDetailsViewArgs::ComponentsAndActorsUseNameArea;
	DetailsViewArgs.NotifyHook = GUnrealEd;
	DetailsViewArgs.ViewIdentifier = TabIdentifier;
	DetailsViewArgs.bCustomNameAreaLocation = true;
	DetailsViewArgs.bCustomFilterAreaLocation = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsViewArgs.HostCommandList = InCommandList;
	DetailsViewArgs.HostTabManager = InTabManager;
	DetailsViewArgs.bShowSectionSelector = true;
	DetailsViewArgs.RightColumnMinWidth = 44.0f;

	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropPlugin.CreateDetailView(DetailsViewArgs);

	auto IsPropertyVisible = [](const FPropertyAndParent& PropertyAndParent)
	{
		// For details views in the level editor all properties are the instanced versions
		if(PropertyAndParent.Property.HasAllPropertyFlags(CPF_DisableEditOnInstance))
		{
			return false;
		}

		return true;
	};

	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda(IsPropertyVisible));
	DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SActorDetails::IsPropertyReadOnly));
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SActorDetails::IsPropertyEditingEnabled));

	// Set up a delegate to call to add generic details to the view
	DetailsView->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FLevelEditorGenericDetails::MakeInstance));

	GEditor->RegisterForUndo(this);

	ComponentsBox = SNew(SBox)
		.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
		.Visibility(this, &SActorDetails::GetComponentEditorVisibility);

	FModuleManager::LoadModuleChecked<FSubobjectEditorModule>("SubobjectEditor");

	LevelEditorSubobjectEditorCustomization = InLevelEditorSubobjectEditorCustomization.Pin();
	
	TSharedPtr<ISCSEditorUICustomization> SubobjectEditorCustomization = nullptr;
	if (ActorDetails::bCVarDisableComponentEditorButtonsAreaOnMultiSelect)
	{
		SubobjectEditorCustomization = LevelEditorSubobjectEditorCustomization;
	}
	else
	{
		InternalSubobjectEditorCustomization = MakeShared<FInternalSubobjectCustomization>(LevelEditorSubobjectEditorCustomization);
		SubobjectEditorCustomization = InternalSubobjectEditorCustomization;
	}
	
	SubobjectEditor = SNew(SSubobjectInstanceEditor)
		.ObjectContext(this, &SActorDetails::GetObjectContext)
		.AllowEditing(this, &SActorDetails::GetAllowComponentTreeEditing)
		.OnSelectionUpdated(this, &SActorDetails::OnSubobjectEditorTreeViewSelectionChanged)
		.OnItemDoubleClicked(this, &SActorDetails::OnSubobjectEditorTreeViewItemDoubleClicked)
		.SCSEditorUICustomization(SubobjectEditorCustomization);
	
	SubobjectEditor->SetSelectionSet(SelectionSet);

	ComponentsBox->SetContent(SubobjectEditor.ToSharedRef());

	TSharedRef<SWidget> SubobjectEditorButtonBox = SubobjectEditor->GetToolButtonsBox().ToSharedRef();
	
	if (ActorDetails::bCVarDisableComponentEditorButtonsAreaOnMultiSelect)
	{
		SubobjectEditorButtonBox->SetVisibility(MakeAttributeSP(this, &SActorDetails::GetComponentEditorButtonsVisibility));
	}

	TFunction<TArray<TTypedElement<ITypedElementDetailsInterface>>()> GetDetailsHandles = [this]()
	{
		TArray<TTypedElement<ITypedElementDetailsInterface>> DetailsElements;

		// Regenerate the details handles
		if (bHasSelectionOverride)
		{
			UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
			DetailsElements.Reserve(SelectionOverrideActors.Num());

			for (AActor* Actor : SelectionOverrideActors)
			{
				if (FTypedElementHandle ActorElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
				{
					if (TTypedElement<ITypedElementDetailsInterface> ActorDetailsHandle = Registry->GetElement<ITypedElementDetailsInterface>(ActorElementHandle))
					{
						// Check if the actor element does to provide a details object
						if (TUniquePtr<ITypedElementDetailsObject> ElementDetailsObject = ActorDetailsHandle.GetDetailsObject())
						{
							DetailsElements.Add(ActorDetailsHandle);
						}
					}
				}
			}
		}
		else
		{
			DetailsElements.Reserve(SelectionSet->GetNumSelectedElements());
			SelectionSet->ForEachSelectedElement<ITypedElementDetailsInterface>([&DetailsElements](const TTypedElement<ITypedElementDetailsInterface>& InDetailsElement)
				{
					// Check if the element does to provide a details object
					if (TUniquePtr<ITypedElementDetailsObject> ElementDetailsObject = InDetailsElement.GetDetailsObject())
					{
						DetailsElements.Add(InDetailsElement);
					}
					return true;
				});
		}

		return DetailsElements;
	};


	TSharedRef<SWidget> ButtonBox = SAssignNew(ElementSelectionDetailsButtons, UE::LevelEditor::Private::SElementSelectionDetailsButtons, SubobjectEditorButtonBox, MoveTemp(GetDetailsHandles));
	DetailsView->SetNameAreaCustomContent(ButtonBox);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(10.f, 4.f, 0.f, 0.f)
		.AutoHeight()
		[
			DetailsView->GetNameAreaWidget().ToSharedRef()
		]
		+SVerticalBox::Slot()
		[
			SAssignNew(DetailsSplitter, SSplitter)
			.MinimumSlotHeight(80.0f)
			.Orientation(Orient_Vertical)
			.Style(FAppStyle::Get(), "SplitterDark")
			.PhysicalSplitterHandleSize(2.0f)
			+ SSplitter::Slot()
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( FMargin(0.f,0.f,0.f,1.f) )
				[
					SNew(SActorDetailsUneditableComponentWarning)
					.Visibility(this, &SActorDetails::GetUCSComponentWarningVisibility)
					.WarningText(NSLOCTEXT("SActorDetails", "BlueprintUCSComponentWarning", "Components created by the User Construction Script can only be edited in the <a id=\"HyperlinkDecorator\" style=\"DetailsView.BPMessageHyperlinkStyle\">Blueprint</>"))
					.OnHyperlinkClicked(this, &SActorDetails::OnBlueprintedComponentWarningHyperlinkClicked)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( FMargin(0.f,0.f,0.f,1.f) )
				[
					SNew(SActorDetailsUneditableComponentWarning)
					.Visibility(this, &SActorDetails::GetInheritedBlueprintComponentWarningVisibility)
					.WarningText(NSLOCTEXT("SActorDetails", "BlueprintUneditableInheritedComponentWarning", "Components flagged as not editable when inherited must be edited in the <a id=\"HyperlinkDecorator\" style=\"DetailsView.BPMessageHyperlinkStyle\">Blueprint</>"))
					.OnHyperlinkClicked(this, &SActorDetails::OnBlueprintedComponentWarningHyperlinkClicked)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( FMargin(0.f,0.f,0.f,1.f) )
				[
					SNew(SActorDetailsUneditableComponentWarning)
					.Visibility(this, &SActorDetails::GetNativeComponentWarningVisibility)
					.WarningText(NSLOCTEXT("SActorDetails", "UneditableNativeComponentWarning", "Native components are editable when declared as a FProperty in <a id=\"HyperlinkDecorator\" style=\"DetailsView.BPMessageHyperlinkStyle\">C++</>"))
					.OnHyperlinkClicked(this, &SActorDetails::OnNativeComponentWarningHyperlinkClicked)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					DetailsView->GetFilterAreaWidget().ToSharedRef()
				]
				+ SVerticalBox::Slot()
				[
					DetailsView.ToSharedRef()
				]
			]
		]
	];

	DetailsSplitter->AddSlot(0)
	.Value(.2f)
	[
		ComponentsBox.ToSharedRef()
	];

	// Immediately update (otherwise we will appear empty)
	RefreshSelection(/*bForceRefresh*/true);
}

SActorDetails::~SActorDetails()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}

	FCoreUObjectDelegates::OnObjectsReinstanced.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	RemoveBPComponentCompileEventDelegate();

	FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
	if (LevelEditor != nullptr)
	{
		LevelEditor->OnComponentsEdited().RemoveAll(this);
	}
}

bool SActorDetails::IsObservingSelectionSet(const UTypedElementSelectionSet* InSelectionSet) const
{
	return SelectionSet == InSelectionSet;
}

void SActorDetails::RefreshSelection(const bool bForceRefresh)
{
	if (bSelectionGuard)
	{
		return;
	}

	TArray<TTypedElement<ITypedElementDetailsInterface>> DetailsElements;
	DetailsElements.Reserve(SelectionSet->GetNumSelectedElements());
	SelectionSet->ForEachSelectedElement<ITypedElementDetailsInterface>([&DetailsElements](const TTypedElement<ITypedElementDetailsInterface>& InDetailsElement)
	{
		DetailsElements.Add(InDetailsElement);
		return true;
	});

	bHasSelectionOverride = false;
	SelectionOverrideActors.Reset();

	RefreshTopLevelElements(DetailsElements, bForceRefresh, /*bOverrideLock*/false);
}

void SActorDetails::OverrideSelection(const TArray<AActor*>& InActors, const bool bForceRefresh)
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	TArray<TTypedElement<ITypedElementDetailsInterface>> DetailsElements;
	DetailsElements.Reserve(InActors.Num());
	for (AActor* Actor : InActors)
	{
		if (FTypedElementHandle ActorElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
		{
			if (TTypedElement<ITypedElementDetailsInterface> ActorDetailsHandle = Registry->GetElement<ITypedElementDetailsInterface>(ActorElementHandle))
			{
				DetailsElements.Add(MoveTemp(ActorDetailsHandle));
			}
		}
	}

	bHasSelectionOverride = true;
	SelectionOverrideActors = InActors;

	RefreshTopLevelElements(DetailsElements, bForceRefresh, /*bOverrideLock*/false);
}

void SActorDetails::RefreshTopLevelElements(TArrayView<const TTypedElement<ITypedElementDetailsInterface>> InDetailsElements, const bool bForceRefresh, const bool bOverrideLock)
{
	// Nothing to do if this view is locked!
	if (DetailsView->IsLocked() && !bOverrideLock)
	{
		return;
	}

	// Build the array of top-level elements to edit
	TopLevelElements.Reset(InDetailsElements.Num());
	for (const TTypedElement<ITypedElementDetailsInterface>& DetailsElement : InDetailsElements)
	{
		if (DetailsElement.IsTopLevelElement())
		{
			if (TUniquePtr<ITypedElementDetailsObject> ElementDetailsObject = DetailsElement.GetDetailsObject())
			{
				TopLevelElements.Add(MoveTemp(ElementDetailsObject));
			}
		}
	}

	if (InternalSubobjectEditorCustomization)
	{
		const bool bHideButtons = TopLevelElements.Num() != 1;
		InternalSubobjectEditorCustomization->SetHideAddComponentButton(bHideButtons);
		InternalSubobjectEditorCustomization->SetHideBlueprintButton(bHideButtons);
	}

	// Update the underlying details view and the Elements buttons
	SetElementDetailsObjects(TopLevelElements, bForceRefresh, bOverrideLock, &InDetailsElements);

	// Update the Subobject tree
	{
		// Enable the selection guard to prevent OnTreeSelectionChanged() from altering the editor's component selection
		TGuardValue<bool> SelectionGuard(bSelectionGuard, true);
		SubobjectEditor->PrePotentialTargetChange();
		// Make our subobject tree correspond to our selected top-level elements (e.g. rebuild from current actor)
		SubobjectEditor->UpdateTree();
		// Adjust the selection of subobjects based on editor selection (e.g. based on actor component selection)
		UpdateComponentTreeFromEditorSelection();
	}

	// Draw attention to this tab if needed
	if (TSharedPtr<FTabManager> TabManager = DetailsView->GetHostTabManager())
	{
		TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(DetailsView->GetIdentifier());
		if (Tab.IsValid() && !Tab->IsForeground())
		{
			Tab->FlashTab();
		}
	}
}

void SActorDetails::RefreshSubobjectTreeElements(TArrayView<const FSubobjectEditorTreeNodePtrType> InSelectedNodes, const bool bForceRefresh, const bool bOverrideLock)
{
	// Nothing to do if this view is locked!
	if (DetailsView->IsLocked() && !bOverrideLock)
	{
		return;
	}

	// Does the Subobject tree have components selected?
	TArray<FTypedElementHandle> Components;
	if (const AActor* Actor = GetActorContext())
	{
		for (const FSubobjectEditorTreeNodePtrType& SelectedNode : InSelectedNodes)
		{
			if (SelectedNode)
			{
				if(const FSubobjectData* Data = SelectedNode->GetDataSource())
				{
					if(Data->IsRootActor())
					{
						// If the actor node is selected then we ignore the component selection
						Components.Reset();
						break;
					}
					
					if (Data->IsComponent())
					{
						if (const UActorComponent* Component = Data->FindComponentInstanceInActor(Actor))
						{
							if (FTypedElementHandle ComponentElementHandle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component))
							{
								Components.Add(ComponentElementHandle);
							}
						}
					}
				}
			}
		}
	}
	else
	{
		using namespace UE::Editor::DataStorage;
		const ICoreProvider* CoreProvider = GetDataStorageFeature<ICoreProvider>(StorageFeatureName);
		for (const FSubobjectEditorTreeNodePtrType& SelectedNode : InSelectedNodes)
		{
			if (SelectedNode)
			{
				if(const FSubobjectData* Data = SelectedNode->GetDataSource())
				{
					if(Data->IsRootActor()) // Not an actor, using IsRootActor to mean IsRootNode
					{
						// If the root node is selected then we ignore the component selection
						Components.Reset();
						break;
					}
					
					if (Data->IsComponent())
					{
						const FSubobjectDataSubsystemContextData_TedsRow* TedsRowAdapter = Data->GetContextData().GetPtr<FSubobjectDataSubsystemContextData_TedsRow>();
						if (TedsRowAdapter)
						{
							RowHandle ComponentRowHandle = TedsRowAdapter->GetRow();
							
							if (const FSubobjectDataSubsystemRowReferenceCollection* Collection = CoreProvider->GetColumn<FSubobjectDataSubsystemRowReferenceCollection>(ComponentRowHandle))
							{
								for (UE::Editor::DataStorage::RowHandle Row : Collection->Rows)
								{
									if (const Compatibility::FTypedElementColumn* TypedElementColumn = CoreProvider->GetColumn<UE::Editor::DataStorage::Compatibility::FTypedElementColumn>(Row))
									{
										Components.Add(TypedElementColumn->Handle);
									}
								}
							}
							
						}
					}
				}
			}
		}
	}

	SubobjectTreeElements.Reset(Components.Num());
	if (Components.Num() > 0)
	{
		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		for (const FTypedElementHandle& ComponentElementHandle : Components)
		{
			if (TTypedElement<ITypedElementDetailsInterface> ComponentDetailsHandle = Registry->GetElement<ITypedElementDetailsInterface>(ComponentElementHandle))
			{
				if (TUniquePtr<ITypedElementDetailsObject> ElementDetailsObject = ComponentDetailsHandle.GetDetailsObject())
				{
					SubobjectTreeElements.Add(MoveTemp(ElementDetailsObject));
				}
			}
		}

		// Use the component elements
		SetElementDetailsObjects(SubobjectTreeElements, bForceRefresh, bOverrideLock);
	}
	else
	{
		// Use the top-level elements
		SetElementDetailsObjects(TopLevelElements, bForceRefresh, bOverrideLock);
	}
}

void SActorDetails::SetElementDetailsObjects(TArrayView<const TUniquePtr<ITypedElementDetailsObject>> InElementDetailsObjects, const bool bForceRefresh, const bool bOverrideLock, TArrayView<const TTypedElement<ITypedElementDetailsInterface>>* InDetailsElementsPtr)
{
	TArray<UObject*> DetailsObjects;
	DetailsObjects.Reserve(InElementDetailsObjects.Num());
	for (const TUniquePtr<ITypedElementDetailsObject>& ElementDetailsObject : InElementDetailsObjects)
	{
		if (UObject* DetailsObject = ElementDetailsObject->GetObject())
		{
			DetailsObjects.Add(DetailsObject);
		}
	}
	DetailsView->SetObjects(DetailsObjects, bForceRefresh, bOverrideLock);
	ElementSelectionDetailsButtons->UpdateSelection(InDetailsElementsPtr);
}

void SActorDetails::PostUndo(bool bSuccess)
{
	// Enable the selection guard to prevent OnTreeSelectionChanged() from altering the editor's component selection
	TGuardValue<bool> SelectionGuard(bSelectionGuard, true);
	
	// Refresh the tree and update the selection to match the world
	SubobjectEditor->UpdateTree();
	UpdateComponentTreeFromEditorSelection();
}

void SActorDetails::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void SActorDetails::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (const TUniquePtr<ITypedElementDetailsObject>& TopLevelElement : TopLevelElements)
	{
		TopLevelElement->AddReferencedObjects(Collector);
	}
}

FString SActorDetails::GetReferencerName() const
{
	return TEXT("SActorDetails");
}

	void SActorDetails::SetActorDetailsRootCustomization(TSharedPtr<FDetailsViewObjectFilter> InActorDetailsObjectFilter, TSharedPtr<IDetailRootObjectCustomization> ActorDetailsRootCustomization)
{
	if (InActorDetailsObjectFilter.IsValid())
	{
		DisplayManager = InActorDetailsObjectFilter->GetDisplayManager();
	}
	
	DetailsView->SetObjectFilter(InActorDetailsObjectFilter);
	DetailsView->SetRootObjectCustomizationInstance(ActorDetailsRootCustomization);
	DetailsView->ForceRefresh();
}

void SActorDetails::OnComponentsEditedInWorld()
{
	if (AActor* Actor = GetActorContext())
	{
		if (SelectionSet->IsElementSelected(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor), FTypedElementIsSelectedOptions()))
		{
			// The component composition of the observed actor has changed, so rebuild the node tree
			TGuardValue<bool> SelectionGuard(bSelectionGuard, true);

			// Refresh the tree and update the selection to match the world
			SubobjectEditor->UpdateTree();
			DetailsView->ForceRefresh();
		}
	}
}

bool SActorDetails::GetAllowComponentTreeEditing() const
{
	return GEditor->PlayWorld == nullptr;
}

AActor* SActorDetails::GetActorContext() const
{
	return TopLevelElements.Num() == 1
		? Cast<AActor>(TopLevelElements[0]->GetObject())
		: nullptr;
}

UObject* SActorDetails::GetObjectContext() const
{
	return TopLevelElements.Num() == 1
		? TopLevelElements[0]->GetObject()
		: nullptr;
}

void SActorDetails::OnSubobjectEditorTreeViewSelectionChanged(const TArray<FSubobjectEditorTreeNodePtrType>& SelectedNodes)
{
	if (bSelectionGuard)
	{
		// Preventing selection changes from having an effect...
		return;
	}

	if (SelectedNodes.Num() == 0)
	{
		// Don't respond to de-selecting everything...
		return;
	}
	
	if (AActor* ActorContext = GetActorContext())
	{
		if (SelectedNodes.Num() > 1 && SelectedBPComponentBlueprint.IsValid())
		{
			// Remove the compilation delegate if we are no longer displaying the full details for a single blueprint component.
			RemoveBPComponentCompileEventDelegate();
		}
		else if (SelectedNodes.Num() == 1 && SelectedNodes[0]->IsComponentNode())
		{
			// Add delegate to monitor blueprint component compilation if we have a full details view ( i.e. single selection )
			FSubobjectData* SelectedData = SelectedNodes[0]->GetDataSource();
			if (UActorComponent* Component = const_cast<UActorComponent*>(SelectedData->FindComponentInstanceInActor(ActorContext)))
			{
				if (UBlueprintGeneratedClass* ComponentBPGC = Cast<UBlueprintGeneratedClass>(Component->GetClass()))
				{
					if (UBlueprint* ComponentBlueprint = Cast<UBlueprint>(ComponentBPGC->ClassGeneratedBy))
					{
						AddBPComponentCompileEventDelegate(ComponentBlueprint);
					}
				}
			}
		}
		
		// We only actually update the editor selection state if we're not locked
		if (!DetailsView->IsLocked())
		{
			TArray<FTypedElementHandle> NewEditorSelection;
			NewEditorSelection.Add(UEngineElementsLibrary::AcquireEditorActorElementHandle(ActorContext));

			for (const FSubobjectEditorTreeNodePtrType& SelectedNode : SelectedNodes)
			{
				if (SelectedNode)
				{
					if(FSubobjectData* Data = SelectedNode->GetDataSource())
					{
						if(Data->IsRootActor())
						{
							// If the actor node is selected then we ignore the component selection
							NewEditorSelection.Reset();
							NewEditorSelection.Add(UEngineElementsLibrary::AcquireEditorActorElementHandle(ActorContext));
							break;
						}
					
						if (Data->IsComponent())
						{
							if (const UActorComponent* Component = Data->FindComponentInstanceInActor(ActorContext))
							{
								NewEditorSelection.Add(UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component));
							}
						}
					}
				}
			}

			// Note: this transaction should not take place if we are in the middle of executing an undo or redo because it would clear the top of the transaction stack.
			const bool bShouldActuallyTransact = !GIsTransacting;
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnComponentInTree", "Clicking on Component (tree view)"), bShouldActuallyTransact);

			// Enable the selection guard to prevent OnEditorSelectionChanged() from altering the contents of the SubobjectTreeWidget
			TGuardValue<bool> SelectionGuard(bSelectionGuard, true);

			// Make sure we clear component visualizers if active. This ensures things like the viewport gizmos reflect the actual selection.
			if (GUnrealEd && GUnrealEd->ComponentVisManager.IsActive())
			{
				GUnrealEd->ComponentVisManager.ClearActiveComponentVis();
			}

			SelectionSet->SetSelection(NewEditorSelection, FTypedElementSelectionOptions());
			SelectionSet->NotifyPendingChanges(); // Fire while still under the selection guard
		}
		// Update the underlying details view
		RefreshSubobjectTreeElements(SelectedNodes, /*bForceRefresh*/false, DetailsView->IsLocked());
	}
	else if (!TopLevelElements.IsEmpty())
	{
		RefreshSubobjectTreeElements(SelectedNodes, /*bForceRefresh*/false, DetailsView->IsLocked());
	}
}

void SActorDetails::OnSubobjectEditorTreeViewItemDoubleClicked(const FSubobjectEditorTreeNodePtrType ClickedNode)
{
	if (ClickedNode && ClickedNode->IsComponentNode())
	{
		if (const USceneComponent* SceneComponent = Cast<USceneComponent>(ClickedNode->GetComponentTemplate()))
		{
			const bool bActiveViewportOnly = false;
			GEditor->MoveViewportCamerasToComponent(SceneComponent, bActiveViewportOnly);
		}
	}
}

void SActorDetails::UpdateComponentTreeFromEditorSelection()
{
	if (DetailsView->IsLocked())
	{
		return;
	}

	// Enable the selection guard to prevent OnTreeSelectionChanged() from altering the editor's component selection
	TGuardValue<bool> SelectionGuard(bSelectionGuard, true);

	TSharedPtr<SSubobjectEditorDragDropTree> TreeWidget = SubobjectEditor->GetDragDropTree();

	// Update the tree selection to match the level editor component selection
	SubobjectEditor->ClearSelection();
	SelectionSet->ForEachSelectedObject<UActorComponent>([this, &TreeWidget](UActorComponent* InComponent)
	{
		FSubobjectEditorTreeNodePtrType TreeNode = SubobjectEditor->FindSlateNodeForObject(InComponent, false);
		if (TreeNode && TreeNode->GetComponentTemplate())
		{
			TreeWidget->RequestScrollIntoView(TreeNode);
			TreeWidget->SetItemSelection(TreeNode, true);
			check(InComponent == TreeNode->GetComponentTemplate() || InComponent->GetArchetype() == TreeNode->GetComponentTemplate());
		}
		return true;
	});

	TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = SubobjectEditor->GetSelectedNodes();
	if (SelectedNodes.Num() == 0)
	{
		SubobjectEditor->SelectDefaultSubobject();
		SelectedNodes = SubobjectEditor->GetSelectedNodes();
	}

	// Update the underlying details view
	RefreshSubobjectTreeElements(SelectedNodes, bSelectedComponentRecompiled, /*bOverrideLock*/false);
}

bool SActorDetails::IsPropertyReadOnly(const FPropertyAndParent& PropertyAndParent) const
{
	bool bIsReadOnly = false;
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		const UActorComponent* Component = Node->GetComponentTemplate();
		if (Component && Component->CreationMethod == EComponentCreationMethod::SimpleConstructionScript)
		{
			TSet<const FProperty*> UCSModifiedProperties;
			Component->GetUCSModifiedProperties(UCSModifiedProperties);
			if (UCSModifiedProperties.Contains(&PropertyAndParent.Property) || 
				(PropertyAndParent.ParentProperties.Num() > 0 && UCSModifiedProperties.Contains(PropertyAndParent.ParentProperties[0])))
			{
				bIsReadOnly = true;
				break;
			}
		}
	}
	return bIsReadOnly;
}

bool SActorDetails::IsPropertyEditingEnabled() const
{
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (!LevelEditor.AreObjectsEditable(DetailsView->GetSelectedObjects()))
	{
		return false;
	}

	bool bIsEditable = true;
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		if(const FSubobjectData* Data = Node->GetDataSource())
		{
			bIsEditable = Data->CanEdit();
			if (!bIsEditable)
			{
				break;
			}
		}
	}
	return bIsEditable;
}

void SActorDetails::OnBlueprintedComponentWarningHyperlinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	UBlueprint* Blueprint = SubobjectEditor->GetBlueprint();
	if (Blueprint)
	{
		// Open the blueprint
		GEditor->EditObject(Blueprint);
	}
}

void SActorDetails::OnNativeComponentWarningHyperlinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	// Find the closest native parent
	UBlueprint* Blueprint = SubobjectEditor->GetBlueprint();
	UClass* ParentClass = Blueprint ? *Blueprint->ParentClass : GetActorContext()->GetClass();
	while (ParentClass && !ParentClass->HasAllClassFlags(CLASS_Native))
	{
		ParentClass = ParentClass->GetSuperClass();
	}

	if (ParentClass)
	{
		FString NativeParentClassHeaderPath;
		const bool bFileFound = FSourceCodeNavigation::FindClassHeaderPath(ParentClass, NativeParentClassHeaderPath)
			&& ( IFileManager::Get().FileSize(*NativeParentClassHeaderPath) != INDEX_NONE );
		if (bFileFound)
		{
			const FString AbsoluteHeaderPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*NativeParentClassHeaderPath);
			FSourceCodeNavigation::OpenSourceFileAsync(AbsoluteHeaderPath);
		}
	}
}

EVisibility SActorDetails::GetComponentEditorVisibility() const
{
	// see if we need to hide the editor due to the current object display 
	const bool bHideEditorFromDetailsView = (DisplayManager.IsValid() &&
							                 DisplayManager->ShouldHideComponentEditor() );
	if (bHideEditorFromDetailsView)
	{
		return EVisibility::Collapsed;
	}

	return SubobjectEditor.IsValid() ? SubobjectEditor->GetDesiredVisibility() : EVisibility::Collapsed;
}

EVisibility SActorDetails::GetComponentEditorButtonsVisibility() const
{
	return GetObjectContext() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SActorDetails::GetUCSComponentWarningVisibility() const
{
	bool bIsUneditableBlueprintComponent = false;

	// Check to see if any selected components are inherited from blueprint
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		if(const FSubobjectData* Data = Node->GetDataSource())
		{
			if (!Data->IsNativeComponent())
			{
				const UActorComponent* Component = Data->GetComponentTemplate();
				bIsUneditableBlueprintComponent = Component ? Component->CreationMethod == EComponentCreationMethod::UserConstructionScript : false;
				if (bIsUneditableBlueprintComponent)
				{
					break;
				}
			}
		}
		
	}

	return bIsUneditableBlueprintComponent ? EVisibility::Visible : EVisibility::Collapsed;
}

bool NotEditableSetByBlueprint(const UActorComponent* Component)
{
	// Determine if it is locked out from a blueprint or from the native
	UActorComponent* Archetype = CastChecked<UActorComponent>(Component->GetArchetype());
	while (Archetype)
	{
		if (Archetype->GetOuter()->IsA<UBlueprintGeneratedClass>() || Archetype->GetOuter()->GetClass()->HasAllClassFlags(CLASS_CompiledFromBlueprint))
		{
			if (!Archetype->bEditableWhenInherited)
			{
				return true;
			}

			Archetype = CastChecked<UActorComponent>(Archetype->GetArchetype());
		}
		else
		{
			Archetype = nullptr;
		}
	}

	return false;
}

EVisibility SActorDetails::GetInheritedBlueprintComponentWarningVisibility() const
{
	bool bIsUneditableBlueprintComponent = false;

	// Check to see if any selected components are inherited from blueprint
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		if(const FSubobjectData* Data = Node->GetDataSource())
		{
			if (!Data->IsNativeComponent())
			{
				if (const UActorComponent* Component = Data->GetComponentTemplate())
				{
					if (!Component->IsEditableWhenInherited() && Component->CreationMethod == EComponentCreationMethod::SimpleConstructionScript)
					{
						bIsUneditableBlueprintComponent = true;
						break;
					}
				}
			}
			else if (!Data->CanEdit() && NotEditableSetByBlueprint(Data->GetComponentTemplate()))
			{
				bIsUneditableBlueprintComponent = true;
				break;
			}
		}	
	}

	return bIsUneditableBlueprintComponent ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SActorDetails::GetNativeComponentWarningVisibility() const
{
	bool bIsUneditableNative = false;
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		if(const FSubobjectData* Data = Node->GetDataSource())
		{
			// Check to see if the component is native and not editable
			if (Data->IsNativeComponent() && !Data->CanEdit() && !NotEditableSetByBlueprint(Data->GetComponentTemplate()))
			{
				bIsUneditableNative = true;
				break;
			}
		}

	}
	
	return bIsUneditableNative ? EVisibility::Visible : EVisibility::Collapsed;
}

void SActorDetails::AddBPComponentCompileEventDelegate(UBlueprint* ComponentBlueprint)
{
	if(SelectedBPComponentBlueprint.Get() != ComponentBlueprint)
	{
		RemoveBPComponentCompileEventDelegate();
		SelectedBPComponentBlueprint = ComponentBlueprint;
		// Add blueprint component compilation event delegate
		if(!ComponentBlueprint->OnCompiled().IsBoundToObject(this))
		{
			ComponentBlueprint->OnCompiled().AddSP(this, &SActorDetails::OnBlueprintComponentCompiled);
		}
	}
}

void SActorDetails::RemoveBPComponentCompileEventDelegate()
{
	// Remove blueprint component compilation event delegate
	if(SelectedBPComponentBlueprint.IsValid())
	{
		SelectedBPComponentBlueprint.Get()->OnCompiled().RemoveAll(this);
		SelectedBPComponentBlueprint.Reset();
		bSelectedComponentRecompiled = false;
	}
}

void SActorDetails::OnBlueprintComponentCompiled(UBlueprint* ComponentBlueprint)
{
	TGuardValue<bool> SelectedComponentRecompiledGuard(bSelectedComponentRecompiled, true);
	UpdateComponentTreeFromEditorSelection();
}

void SActorDetails::OnObjectsReinstanced(const TMap<UObject*, UObject*>& InReplacementObjects)
{
	if (bHasSelectionOverride && SelectionOverrideActors.Num() > 0)
	{
		bool bHasChanges = false;

		for (auto It = SelectionOverrideActors.CreateIterator(); It; ++It)
		{
			AActor*& Actor = *It;

			if (UObject* const* ReplacementObjectPtr = InReplacementObjects.Find(Actor))
			{
				bHasChanges = true;

				AActor* ReplacementActor = Cast<AActor>(*ReplacementObjectPtr);
				if (ReplacementActor)
				{
					Actor = ReplacementActor;
				}
				else
				{
					It.RemoveCurrent();
				}
			}
		}

		if (bHasChanges)
		{
			TArray<AActor*> NewSelection = SelectionOverrideActors;
			OverrideSelection(NewSelection);
		}
	}
	else
	{
		// Enable the selection guard to prevent OnTreeSelectionChanged() from altering the editor's component selection
		TGuardValue<bool> SelectionGuard(bSelectionGuard, true);
		SubobjectEditor->UpdateTree();
	}
}

void SActorDetails::OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	// Listen for archetype changes to properties that are editable on the instance
	if (!ObjectBeingModified
		|| !ObjectBeingModified->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject)
		|| !PropertyChangedEvent.Property
		|| !PropertyChangedEvent.Property->HasAnyPropertyFlags(CPF_Edit)
		|| PropertyChangedEvent.Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance))
	{
		return;
	}

	// If the object that changes matches the archetype of an instance that's selected to the property view,
	// invalidate the cached state so that things like the "reset to default" icon is synced to the archetype.
	for (const TWeakObjectPtr<>& SelectedObject : DetailsView->GetSelectedObjects())
	{
		if (SelectedObject.IsValid()
			&& SelectedObject->GetArchetype() == ObjectBeingModified)
		{
			DetailsView->InvalidateCachedState();
			break;
		}
	}
}

FReply SActorDetails::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (DetailsView.IsValid())
    {
		// This leaves a chance to events coming from the whole Details tab that do not come directly from Detail view to be handled
		FReply Reply = DetailsView->OnKeyDown(InGeometry, InKeyEvent);
		if (Reply.IsEventHandled())
		{
			return Reply;
		}
    }

	return SCompoundWidget::OnKeyDown(InGeometry, InKeyEvent);
}

#undef LOCTEXT_NAMESPACE
