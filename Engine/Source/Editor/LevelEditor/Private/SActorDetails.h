// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Containers/ArrayView.h"
#include "EditorUndoClient.h"
#include "Elements/Interfaces/TypedElementDetailsInterface.h"

class FActorDetailsSCSEditorUICustomization;
class AActor;
class FSCSEditorTreeNode;
class FTabManager;
class FUICommandList;
class IDetailsView;
class SBox;
class SSplitter;
class UBlueprint;
class FDetailsViewObjectFilter;
class UTypedElementSelectionSet;
class SSubobjectEditor;
class ISCSEditorUICustomization;
class FSubobjectEditorTreeNode;
struct FPropertyChangedEvent;
class FDetailsDisplayManager;

namespace UE::LevelEditor::Private
{
	class SElementSelectionDetailsButtons;
}

/**
 * Wraps a details panel customized for the level editor
 */
class SActorDetails : public SCompoundWidget, public FEditorUndoClient, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SActorDetails) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs,
		UTypedElementSelectionSet* InSelectionSet,
		const FName TabIdentifier,
		TSharedPtr<FUICommandList> InCommandList,
		TSharedPtr<FTabManager> InTabManager,
		TWeakPtr<FActorDetailsSCSEditorUICustomization> InLevelEditorSubobjectEditorCustomization);
	virtual ~SActorDetails() override;

	/**
	 * Return true if this details panel is observing the given selection set.
	 */
	bool IsObservingSelectionSet(const UTypedElementSelectionSet* InSelectionSet) const;

	/**
	 * Update the view based on our observed selection set.
	 */
	void RefreshSelection(const bool bForceRefresh = false);

	/**
	 * Update the view based on the given set of actors.
	 */
	void OverrideSelection(const TArray<AActor*>& InActors, const bool bForceRefresh = false);

	/** FEditorUndoClient Interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	
	//~ FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	/**
	 * Sets the filter that should be used to filter incoming actors in or out of the details panel
	 *
	 * @param InFilter	The filter to use or nullptr to remove the active filter
	 */
	void SetActorDetailsRootCustomization(TSharedPtr<FDetailsViewObjectFilter> InActorDetailsObjectFilter, TSharedPtr<class IDetailRootObjectCustomization> ActorDetailsRootCustomization);

protected:
	//~ Begin SWidget
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

private:
	/**
	 * A customization that the ActorDetails widget manages that maintains the visibility of the
	 * add component and convert to blueprint buttons.
	 */
	class FInternalSubobjectCustomization;
	
	void RefreshTopLevelElements(TArrayView<const TTypedElement<ITypedElementDetailsInterface>> InDetailsElements, const bool bForceRefresh, const bool bOverrideLock);
	void RefreshSubobjectTreeElements(TArrayView<const TSharedPtr<FSubobjectEditorTreeNode>> InSelectedNodes, const bool bForceRefresh, const bool bOverrideLock);
	void SetElementDetailsObjects(TArrayView<const TUniquePtr<ITypedElementDetailsObject>> InElementDetailsObjects, const bool bForceRefresh, const bool bOverrideLock, TArrayView<const TTypedElement<ITypedElementDetailsInterface>>* InDetailsElementsPtr = nullptr);

	AActor* GetActorContext() const;
	UObject* GetObjectContext() const;
	bool GetAllowComponentTreeEditing() const;

	/**
	 * Gets the Visibility of the top of the panel buttons in the details panel
	 */
	EVisibility GetComponentEditorButtonsVisibility() const;

	void OnComponentsEditedInWorld();
	void OnSubobjectEditorTreeViewSelectionChanged(const TArray<TSharedPtr<FSubobjectEditorTreeNode> >& SelectedNodes);
	void OnSubobjectEditorTreeViewItemDoubleClicked(const TSharedPtr<FSubobjectEditorTreeNode> ClickedNode);
	void UpdateComponentTreeFromEditorSelection();

	bool IsPropertyReadOnly(const struct FPropertyAndParent& PropertyAndParent) const;
	bool IsPropertyEditingEnabled() const;
	EVisibility GetComponentEditorVisibility() const;
	EVisibility GetUCSComponentWarningVisibility() const;
	EVisibility GetInheritedBlueprintComponentWarningVisibility() const;
	EVisibility GetNativeComponentWarningVisibility() const;
	void OnBlueprintedComponentWarningHyperlinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata);
	void OnNativeComponentWarningHyperlinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata);

	void AddBPComponentCompileEventDelegate(UBlueprint* ComponentBlueprint);
	void RemoveBPComponentCompileEventDelegate();
	void OnBlueprintComponentCompiled(UBlueprint* ComponentBlueprint);
	void OnObjectsReinstanced(const TMap<UObject*, UObject*>& InReinstancedObjects);
	void OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	TSharedPtr<SSplitter> DetailsSplitter;
	TSharedPtr<class IDetailsView> DetailsView;
	TSharedPtr<SBox> ComponentsBox;

	// Subobject Editor customization that hosts other customizations. Scoped to the level editor
	TSharedPtr<FActorDetailsSCSEditorUICustomization> LevelEditorSubobjectEditorCustomization;
	// Subobject Editor customization that is maintained as last priority in the LevelEditorCustomization
	// and is specific to the SActorDetails
	TSharedPtr<FInternalSubobjectCustomization> InternalSubobjectEditorCustomization;
	/** The subobject editor provides a tree widget that allows for editing of subobjects */
	TSharedPtr<SSubobjectEditor> SubobjectEditor;

	TSharedPtr<UE::LevelEditor::Private::SElementSelectionDetailsButtons> ElementSelectionDetailsButtons;

	// The selection set this details panel is observing
	UTypedElementSelectionSet* SelectionSet = nullptr;

	// The selection override, if any
	bool bHasSelectionOverride = false;
	TArray<AActor*> SelectionOverrideActors;

	// Array of top-level elements that are currently being edited
	TArray<TUniquePtr<ITypedElementDetailsObject>> TopLevelElements;

	// Array of component elements that are being edited from the Subobject tree selection
	TArray<TUniquePtr<ITypedElementDetailsObject>> SubobjectTreeElements;
	
	// The current component blueprint selection
	TWeakObjectPtr<UBlueprint> SelectedBPComponentBlueprint;
	bool bSelectedComponentRecompiled = false;

	// Used to prevent reentrant changes
	bool bSelectionGuard = false;

	/** A @code TSharedPtr @endcode to the @code FDetailsDisplayManager @endcode to manage various aspects of the
	 *details view */
	TSharedPtr<FDetailsDisplayManager> DisplayManager;
};
