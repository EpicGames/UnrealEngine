// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActorPilotViewportToolbar.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "LevelEditorViewport.h"
#include "LevelViewportActions.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SActorPilotViewportToolbar"

void SActorPilotViewportToolbar::Construct(const FArguments& InArgs)
{
	SViewportToolBar::Construct(SViewportToolBar::FArguments());

	Viewport = InArgs._Viewport;

	FLevelViewportCommands& ViewportCommands = FLevelViewportCommands::Get();

	TSharedPtr<const FUICommandList> CommandList = InArgs._Viewport->GetCommandList();
	FSlimHorizontalToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None);

	// Use a custom style
	FName ToolBarStyle = "EditorViewportToolBar";
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), ToolBarStyle);
	ToolbarBuilder.SetIsFocusable(false);

	ToolbarBuilder.BeginSection("ActorPilot");
	{
		ToolbarBuilder.BeginBlockGroup();
		{
			static const FName PilotedActorName = "PilotedActor";
			ToolbarBuilder.AddToolBarButton(
				ViewportCommands.SelectPilotedActor,
				NAME_None,
				TAttribute<FText>::Create([this]()
					{
						return GetActiveText();
					}),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelViewport.PilotSelectedActor"),
				PilotedActorName);
	
			static const FName EjectActorPilotName = "EjectActorPilot";
			ToolbarBuilder.AddToolBarButton(
				ViewportCommands.EjectActorPilot,
				NAME_None,
				FText::GetEmpty(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(),
				EjectActorPilotName);
		
			static FName ToggleActorPilotCameraViewName = "ToggleActorPilotCameraView";
			ToolbarBuilder.AddToolBarButton(
				ViewportCommands.ToggleActorPilotCameraView,
				NAME_None,
				FText::GetEmpty(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(),
				ToggleActorPilotCameraViewName);
		}
		ToolbarBuilder.EndBlockGroup();
	}
	ToolbarBuilder.EndSection();

	ChildSlot
	[
		SNew( SBorder )
		.BorderImage( FAppStyle::GetBrush("NoBorder") )
		.Padding(FMargin(4.f, 0.f))
		[
			ToolbarBuilder.MakeWidget()
		]
	];
}

FText SActorPilotViewportToolbar::GetActiveText() const
{
	const AActor* Pilot = nullptr;
	const TSharedPtr<SLevelViewport> ViewportPtr = Viewport.Pin();
	if (ViewportPtr.IsValid())
	{
		Pilot = ViewportPtr->GetLevelViewportClient().GetActiveActorLock().Get();
	}

	return Pilot ? FText::Format(LOCTEXT("ActiveText", "Pilot Actor: {0} "), FText::FromString(Pilot->GetActorLabel())) : FText();
}

EVisibility SActorPilotViewportToolbar::GetLockedTextVisibility() const
{
	const AActor* Pilot = nullptr;
	const TSharedPtr<SLevelViewport> ViewportPtr = Viewport.Pin();
	if (ViewportPtr.IsValid())
	{
		Pilot = ViewportPtr->GetLevelViewportClient().GetActiveActorLock().Get();
	}

	return Pilot && Pilot->IsLockLocation() ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
