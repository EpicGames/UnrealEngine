// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SLevelViewport.h"
#include "SViewportToolBar.h"

class SActorPilotViewportToolbar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(SActorPilotViewportToolbar){}
		SLATE_ARGUMENT(TSharedPtr<SLevelViewport>, Viewport)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FText GetActiveText() const;

	EVisibility GetLockedTextVisibility() const;

private:

	/** The viewport that we are in */
	TWeakPtr<SLevelViewport> Viewport;
};
