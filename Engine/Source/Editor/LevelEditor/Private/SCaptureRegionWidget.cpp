// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCaptureRegionWidget.h"

#include "Fonts/SlateFontInfo.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "HighResScreenshot.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Visibility.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Slate/SceneViewport.h"
#include "Styling/CoreStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Widgets/Text/STextBlock.h"

void SCaptureRegionWidget::Construct( const FArguments& InArgs )
{
	this->ChildSlot
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 20))
			.ShadowOffset(FVector2D(1, 1))
			.Text(NSLOCTEXT("CaptureRegion", "SpecifyRectangleToCaptureMessage", "Please specify capture rectangle"))
		];

	Deactivate(true);

//	OnCaptureRegionChanged = InArgs._OnCaptureRegionChanged;
//	OnCaptureRegionCompleted = InArgs._OnCaptureRegionCompleted;

	CurrentState = State_Inactive;
	PotentialInteraction = PI_DrawNewCaptureRegion;
	bIgnoreExistingCaptureRegion = false;
}

void SCaptureRegionWidget::Activate(bool bCurrentCaptureRegionIsFullViewport)
{
	SetVisibility(EVisibility::Visible);

	OriginalCaptureRegion = GetHighResScreenshotConfig().UnscaledCaptureRegion;
	bIgnoreExistingCaptureRegion = bCurrentCaptureRegionIsFullViewport;
}

void SCaptureRegionWidget::Deactivate(bool bKeepChanges)
{
	if (GetVisibility() != EVisibility::Hidden)
	{
		SetVisibility(EVisibility::Hidden);

		bIgnoreExistingCaptureRegion = false;

		if (!bKeepChanges)
		{
			GetHighResScreenshotConfig().UnscaledCaptureRegion = OriginalCaptureRegion;
		}
	}
}

void SCaptureRegionWidget::Reset()
{
	bIgnoreExistingCaptureRegion = true;
}

FReply SCaptureRegionWidget::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (IsEnabled() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FVector2D ViewportPosition = MouseEvent.GetScreenSpacePosition() - FVector2D(MyGeometry.AbsolutePosition);
		FIntRect& CurrentCaptureRegion = GetHighResScreenshotConfig().UnscaledCaptureRegion;

		switch (PotentialInteraction)
		{
		case PI_DrawNewCaptureRegion:
			{
				DragStartPosition = MouseEvent.GetScreenSpacePosition() - FVector2D(MyGeometry.AbsolutePosition);
				BuildNewCaptureRegion(DragStartPosition, DragStartPosition);
				CurrentState = State_Dragging;
				break;
			}
		case PI_ResizeBL:
			{
				DragStartPosition = FVector2D(CurrentCaptureRegion.Max.X, CurrentCaptureRegion.Min.Y);
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				CurrentState = State_Dragging;
				break;
			}

		case PI_ResizeTL:
			{
				DragStartPosition = FVector2D(CurrentCaptureRegion.Max.X, CurrentCaptureRegion.Max.Y);
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				CurrentState = State_Dragging;
				break;
			}
		case PI_ResizeBR:
			{
				DragStartPosition = FVector2D(CurrentCaptureRegion.Min.X, CurrentCaptureRegion.Min.Y);
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				CurrentState = State_Dragging;
				break;
			}

		case PI_ResizeTR:
			{
				DragStartPosition = FVector2D(CurrentCaptureRegion.Min.X, CurrentCaptureRegion.Max.Y);
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				CurrentState = State_Dragging;
				break;
			}

		case PI_ResizeBottom:
			{
				DragStartPosition = FVector2D(CurrentCaptureRegion.Min.X, CurrentCaptureRegion.Min.Y);
				CurrentState = State_YAxisResize;
				break;
			}

		case PI_ResizeTop:
			{
				DragStartPosition = FVector2D(CurrentCaptureRegion.Min.X, CurrentCaptureRegion.Max.Y);
				CurrentState = State_YAxisResize;
				break;
			}

		case PI_ResizeLeft:
			{
				DragStartPosition = FVector2D(CurrentCaptureRegion.Max.X, CurrentCaptureRegion.Min.Y);
				CurrentState = State_XAxisResize;
				break;
			}

		case PI_ResizeRight:
			{
				DragStartPosition = FVector2D(CurrentCaptureRegion.Min.X, CurrentCaptureRegion.Min.Y);
				CurrentState = State_XAxisResize;
				break;
			}

		case PI_MoveExistingRegion:
			{
				DragStartPosition = ViewportPosition;
				CurrentState = State_Moving;
				break;
			}
		}
		
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SCaptureRegionWidget::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (IsEnabled() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FVector2D ViewportPosition = MouseEvent.GetScreenSpacePosition() - FVector2D(MyGeometry.AbsolutePosition);
		FIntRect& CurrentCaptureRegion = GetHighResScreenshotConfig().UnscaledCaptureRegion;

		switch (CurrentState)
		{
		case State_Dragging:
			{
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				CurrentState = State_Inactive;
				bIgnoreExistingCaptureRegion = false;
				break;
			}

		case State_XAxisResize:
			{
				ViewportPosition.Y = CurrentCaptureRegion.Max.Y;
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				CurrentState = State_Inactive;
				break;
			}

		case State_YAxisResize:
			{
				ViewportPosition.X = CurrentCaptureRegion.Max.X;
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				CurrentState = State_Inactive;
				break;
			}

		case State_Moving:
			{
				CurrentState = State_Inactive;
				//OnCaptureRegionChanged.ExecuteIfBound(CurrentCaptureRegion);
				SendUpdatedCaptureRegion();
				break;
			}

		case State_Inactive:
			{
				break;
			}

		default:
			{
				check(false);
				break;
			}
		}

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SCaptureRegionWidget::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (IsEnabled())
	{
		FVector2D ViewportPosition = MouseEvent.GetScreenSpacePosition() - FVector2D(MyGeometry.AbsolutePosition);
		FIntRect& CurrentCaptureRegion = GetHighResScreenshotConfig().UnscaledCaptureRegion;

		switch (CurrentState)
		{
		case State_Dragging:
			{
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				break;
			}

		case State_Moving:
			{
				FVector2D Delta = ViewportPosition - DragStartPosition;
				DragStartPosition = ViewportPosition;
				CurrentCaptureRegion += FIntPoint((int32)Delta.X, (int32)Delta.Y);
				//OnCaptureRegionChanged.ExecuteIfBound(CurrentCaptureRegion);
				SendUpdatedCaptureRegion();
				break;
			}

		case State_XAxisResize:
			{
				ViewportPosition.Y = CurrentCaptureRegion.Max.Y;
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				break;
			}

		case State_YAxisResize:
			{
				ViewportPosition.X = CurrentCaptureRegion.Max.X;
				BuildNewCaptureRegion(ViewportPosition, DragStartPosition);
				break;
			}
		
		case State_Inactive:
			{
				if (CurrentCaptureRegion.Area() > 0 && !bIgnoreExistingCaptureRegion)
				{
					// Common tests
					bool bWithinXRangeOfExistingRegion = ViewportPosition.X >= CurrentCaptureRegion.Min.X && ViewportPosition.X <= CurrentCaptureRegion.Max.X;
					bool bWithinYRangeOfExistingRegion = ViewportPosition.Y >= CurrentCaptureRegion.Min.Y && ViewportPosition.Y <= CurrentCaptureRegion.Max.Y;

					// Distance away from an edge which we consider intersecting
					const float EdgeDistance = 5;

					// Check if we're over the corners
					if ((ViewportPosition - FVector2D((float)CurrentCaptureRegion.Min.X, CurrentCaptureRegion.Min.Y)).Size() < EdgeDistance)
					{
						SetCursor(EMouseCursor::ResizeSouthEast);
						PotentialInteraction = PI_ResizeTL;
					}
					else if ((ViewportPosition - FVector2D((float)CurrentCaptureRegion.Min.X, CurrentCaptureRegion.Max.Y)).Size() < EdgeDistance)
					{
						SetCursor(EMouseCursor::ResizeSouthWest);
						PotentialInteraction = PI_ResizeBL;
					}
					else if ((ViewportPosition - FVector2D((float)CurrentCaptureRegion.Max.X, CurrentCaptureRegion.Min.Y)).Size() < EdgeDistance)
					{
						SetCursor(EMouseCursor::ResizeSouthWest);
						PotentialInteraction = PI_ResizeTR;
					}
					else if ((ViewportPosition - FVector2D((float)CurrentCaptureRegion.Max.X, CurrentCaptureRegion.Max.Y)).Size() < EdgeDistance)
					{
						SetCursor(EMouseCursor::ResizeSouthEast);
						PotentialInteraction = PI_ResizeBR;
					}
					else if (FMath::Abs((float)CurrentCaptureRegion.Min.X - ViewportPosition.X) < EdgeDistance && bWithinYRangeOfExistingRegion)
					{
						SetCursor(EMouseCursor::ResizeLeftRight);
						PotentialInteraction = PI_ResizeLeft;
					}
					else if (FMath::Abs((float)CurrentCaptureRegion.Max.X - ViewportPosition.X) < EdgeDistance && bWithinYRangeOfExistingRegion)
					{
						SetCursor(EMouseCursor::ResizeLeftRight);
						PotentialInteraction = PI_ResizeRight;
					}
					else if (FMath::Abs((float)CurrentCaptureRegion.Min.Y - ViewportPosition.Y) < EdgeDistance && bWithinXRangeOfExistingRegion)
					{
						SetCursor(EMouseCursor::ResizeUpDown);
						PotentialInteraction = PI_ResizeTop;
					}
					else if (FMath::Abs((float)CurrentCaptureRegion.Max.Y - ViewportPosition.Y) < EdgeDistance && bWithinXRangeOfExistingRegion)
					{
						SetCursor(EMouseCursor::ResizeUpDown);
						PotentialInteraction = PI_ResizeBottom;
					}
					else if (CurrentCaptureRegion.Contains(FIntPoint((int32)ViewportPosition.X, (int32)ViewportPosition.Y)))
					{
						SetCursor(EMouseCursor::CardinalCross);
						PotentialInteraction = PI_MoveExistingRegion;
					}
					else
					{
						SetCursor(EMouseCursor::Crosshairs);
						PotentialInteraction = PI_DrawNewCaptureRegion;
					}
				}
				else
				{
					SetCursor(EMouseCursor::Crosshairs);
					PotentialInteraction = PI_DrawNewCaptureRegion;
				}

				break;
			}

		default:
			{
				check(false);
				break;
			}
		}

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

void SCaptureRegionWidget::BuildNewCaptureRegion(const FVector2D& InPointA, const FVector2D& InPointB)
{
	FIntRect& CurrentCaptureRegion = GetHighResScreenshotConfig().UnscaledCaptureRegion;
	CurrentCaptureRegion.Min = FIntPoint(FMath::Min((int32)InPointA.X, (int32)InPointB.X), FMath::Min((int32)InPointA.Y, (int32)InPointB.Y));
	CurrentCaptureRegion.Max = FIntPoint(FMath::Max((int32)InPointA.X, (int32)InPointB.X), FMath::Max((int32)InPointA.Y, (int32)InPointB.Y));
	//OnCaptureRegionChanged.ExecuteIfBound(CurrentCaptureRegion);
	SendUpdatedCaptureRegion();
}

void SCaptureRegionWidget::SendUpdatedCaptureRegion()
{
	auto& Config = GetHighResScreenshotConfig();
	auto ConfigViewport = Config.TargetViewport.Pin();
	if (ConfigViewport.IsValid())
	{
		ConfigViewport->Invalidate();
	}
}
