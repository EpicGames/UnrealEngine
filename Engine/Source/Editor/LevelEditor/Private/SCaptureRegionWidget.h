// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "Input/Reply.h"
#include "Math/IntRect.h"
#include "Math/MathFwd.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FGeometry;
struct FPointerEvent;

DECLARE_DELEGATE_OneParam( FOnCaptureRegionChanged, const FIntRect& )
DECLARE_DELEGATE_OneParam( FOnCaptureRegionCompleted, bool )

class SCaptureRegionWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SCaptureRegionWidget )
		: _Enabled()
		{}

		/** Whether the control is enabled */
		SLATE_ATTRIBUTE( bool, Enabled )

		/** Event that fires when the widget changes the capture region **/
		//SLATE_EVENT( FOnCaptureRegionChanged, OnCaptureRegionChanged )

		/** Event that fires when capture region is completed **/
		//SLATE_EVENT( FOnCaptureRegionCompleted, OnCaptureRegionCompleted )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );
	void Activate(bool bCurrentCaptureRegionIsFullViewport);
	void Deactivate(bool bKeepChanges);
	void Reset();

	FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

private:

	enum EState
	{
		State_Inactive,
		State_Dragging,
		State_Moving,
		State_XAxisResize,
		State_YAxisResize,
	};

	enum EPotentialInteraction
	{
		PI_DrawNewCaptureRegion,
		PI_ResizeTL,
		PI_ResizeTR,
		PI_ResizeBL,
		PI_ResizeBR,
		PI_ResizeLeft,
		PI_ResizeRight,
		PI_ResizeTop,
		PI_ResizeBottom,
		PI_MoveExistingRegion,
	};

	EState CurrentState;
	FVector2D DragStartPosition;
	//FIntRect CurrentCaptureRegion;
	FIntRect OriginalCaptureRegion;
	EPotentialInteraction PotentialInteraction;
	bool bIgnoreExistingCaptureRegion; // Don't allow manipulation of the current region. Will be true after first activation, if the original capture region is the size of the viewport

	/** Delegate to invoke when selection changes. */
	//FOnCaptureRegionChanged OnCaptureRegionChanged;

	/** Delegate to invoke when selection has completed. */
	//FOnCaptureRegionCompleted OnCaptureRegionCompleted;

	void BuildNewCaptureRegion(const FVector2D& InPointA, const FVector2D& InPointB);

	void SendUpdatedCaptureRegion();
};
