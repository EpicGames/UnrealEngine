// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInViewportDetails.h"
#include "Widgets/SBoxPanel.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "Editor.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Selection.h"
#include "UnrealEdGlobals.h"
#include "LevelEditor.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "LevelEditorGenericDetails.h"
#include "ScopedTransaction.h"
#include "SourceCodeNavigation.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "DetailsViewObjectFilter.h"
#include "IDetailRootObjectCustomization.h"
#include "IPropertyRowGenerator.h"
#include "IDetailTreeNode.h"
#include "IDetailPropertyRow.h"
#include "Widgets/Layout/SBackgroundBlur.h"
#include "PropertyCustomizationHelpers.h"
#include "Input/DragAndDrop.h"
#include "SEditorViewport.h"
#include "SResetToDefaultPropertyEditor.h"
#include "ToolMenus.h"
#include "Editor/EditorEngine.h"
#include "LevelEditorMenuContext.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Viewports/InViewportUIDragOperation.h"

#define LOCTEXT_NAMESPACE "InViewportDetails"


class SInViewportDetailsRow : public STableRow< TSharedPtr<IDetailTreeNode> >
{
public:

	SLATE_BEGIN_ARGS(SInViewportDetailsRow)
	{}

	/** The item content. */
	SLATE_ARGUMENT(TSharedPtr<IDetailTreeNode>, InNode)
	SLATE_ARGUMENT(TSharedPtr<SInViewportDetails>, InDetailsView)
	SLATE_END_ARGS()


	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		ParentDetailsView = InArgs._InDetailsView;
		FDetailColumnSizeData& ColumnSizeData = InArgs._InDetailsView->GetColumnSizeData();

		TSharedPtr<IDetailPropertyRow> DetailPropertyRow = InArgs._InNode->GetRow();
		FDetailWidgetRow Row;
		TSharedPtr< SWidget > NameWidget;
		TSharedPtr< SWidget > ValueWidget;
		DetailPropertyRow->GetDefaultWidgets(NameWidget, ValueWidget, Row, true);
		TSharedPtr< SWidget > ResetWidget = SNew(SResetToDefaultPropertyEditor, InArgs._InNode->CreatePropertyHandle());
 		TSharedPtr<SWidget> RowWidget = SNullWidget::NullWidget;
		{
 
			RowWidget = SNew(SSplitter)
				.Style(FAppStyle::Get(), "PropertyTable.InViewport.Splitter")
				.PhysicalSplitterHandleSize(1.0f)
 				.HitDetectionSplitterHandleSize(5.0f)
 				+ SSplitter::Slot()
 				.Value(ColumnSizeData.GetNameColumnWidth())
				.OnSlotResized(ColumnSizeData.GetOnNameColumnResized())
 				[
					SNew(SBox)
					.HAlign(HAlign_Left)
 					.Padding(2.0f)
					.Clipping(EWidgetClipping::OnDemand)
 					[
						NameWidget.ToSharedRef()
					]
 				]
 				+ SSplitter::Slot()
 				.Value(ColumnSizeData.GetValueColumnWidth())
 				.OnSlotResized(ColumnSizeData.GetOnValueColumnResized())
 				[
 					SNew(SBox)
 					.Padding(2.0f)
 					[
 						ValueWidget.ToSharedRef()
 					]
				]
				+ SSplitter::Slot()
				.Value(ColumnSizeData.GetRightColumnWidth())
				.OnSlotResized(ColumnSizeData.GetOnRightColumnResized())
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Padding(2.0f)
					[
						ResetWidget.ToSharedRef()
					]
				];
 		}

		ChildSlot
			[
				SNew(SBox)
				.MinDesiredWidth(250.0f)
				.Padding(FMargin(14.f, 0.f, 10.f, 0.f))
				[
					RowWidget.ToSharedRef()
				]
			];

		STableRow< TSharedPtr<IDetailTreeNode> >::ConstructInternal(
			STableRow::FArguments()
			.Style(FAppStyle::Get(), "PropertyTable.InViewport.Row")
			.ShowSelection(false),
			InOwnerTableView
		);
	}

	TWeakPtr<SInViewportDetails> ParentDetailsView;

};

void SInViewportDetails::Construct(const FArguments& InArgs)
{
	OwningViewport = InArgs._InOwningViewport;
	ParentLevelEditor = InArgs._InOwningLevelEditor;
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FPropertyRowGeneratorArgs Args;
	Args.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	Args.NotifyHook = GUnrealEd;
	ColumnSizeData.SetValueColumnWidth(0.5f);
	PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(Args);

	USelection::SelectionChangedEvent.AddRaw(this, &SInViewportDetails::OnEditorSelectionChanged);

	GEditor->RegisterForUndo(this);

	GenerateWidget();

}

void SInViewportDetails::GenerateWidget()
{
	if (AActor* SelectedActor = GetSelectedActorInEditor())
	{
		// Don't show this menu in PIE 
		if (SelectedActor->GetWorld()->WorldType != EWorldType::Editor)
		{
			return;
		}

		FString NameString;
		if (GEditor->GetSelectedActors()->Num() > 1)
		{
			NameString = LOCTEXT("SelectedObjects", "Selected Objects").ToString();
		}
		else
		{
			// Use the actor label because that's the friendly name used in other editor UI.
			NameString = SelectedActor->GetActorLabel();
		}

		// Get the common base class of the selected objects
		UClass* BaseClass = NULL;
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			AActor* Actor = Cast<AActor>(*It);
			if (Actor)
			{
				UClass* ActorClass = Actor->GetClass();

				if (!BaseClass)
				{
					BaseClass = ActorClass;
				}

				while (!ActorClass->IsChildOf(BaseClass))
				{
					BaseClass = BaseClass->GetSuperClass();
				}
			}
		}
		const FSlateBrush* ActorIcon = FSlateIconFinder::FindIconBrushForClass(BaseClass);
		ChildSlot
			[
				SNew(SBackgroundBlur)
				.BlurStrength(4)
				.Padding(0.0f)
				.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("PropertyTable.InViewport.Background"))
					.Visibility(this, &SInViewportDetails::GetHeaderVisibility)
					.Padding(0.0f)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f)
						[
							SNew(SInViewportDetailsHeader)
							.Parent(SharedThis(this))
							.Content()
							[		
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(0.f)
								[
									SNew(SImage)
									.Image(ActorIcon)
									.ColorAndOpacity(FSlateColor::UseForeground())
								]
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.Padding(4.0f, 0.0f)
								[
									SNew(STextBlock)
									.Text(FText::FromString(NameString))
									.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
								]
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f)
						[
							SNew(SInViewportDetailsToolbar)
							.Parent(SharedThis(this))
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(FMargin(0.0f, 5.0f))
						[
							MakeDetailsWidget()
						]
					]
				]
			];
	}
}

EVisibility SInViewportDetails::GetHeaderVisibility() const
{
	return Nodes.Num() ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SInViewportDetails::MakeDetailsWidget()
{
	TSharedRef<SWidget> DetailWidget = SNullWidget::NullWidget;
	static const FName Name_ShouldShowInViewport("ShouldShowInViewport");
	if (PropertyRowGenerator.IsValid())
	{
		Nodes.Empty();
		
		for (TSharedRef<IDetailTreeNode> RootNode : PropertyRowGenerator->GetRootTreeNodes())
		{
			TArray<TSharedRef<IDetailTreeNode>> Children;
			RootNode->GetChildren(Children);
			
			for (TSharedRef<IDetailTreeNode> Child : Children)
			{
				bool bShowChild = false;
				{
					TSharedPtr<IPropertyHandle> NodePropertyHandle = Child->CreatePropertyHandle();
					if (NodePropertyHandle && NodePropertyHandle->GetProperty() && NodePropertyHandle->GetProperty()->HasAllPropertyFlags(CPF_DisableEditOnInstance))
					{
						continue;
					}
					if (NodePropertyHandle && NodePropertyHandle->GetProperty() && NodePropertyHandle->GetProperty()->GetBoolMetaData(Name_ShouldShowInViewport))
					{
						bShowChild = true;
					}
					if (bShowChild)
					{
						Nodes.Add(Child);
					}	
				}

			}
			
		}
	}
	if (Nodes.Num())
	{
		NodeList = SNew(SListView< TSharedPtr<IDetailTreeNode> >)
			.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("PropertyTable.InViewport.ListView"))
			.ListItemsSource(&Nodes)
			.OnGenerateRow(this, &SInViewportDetails::GenerateListRow);

		
		DetailWidget = NodeList.ToSharedRef();
	}
	return DetailWidget;
}

SInViewportDetails::~SInViewportDetails()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
	USelection::SelectionChangedEvent.RemoveAll(this);
}


void SInViewportDetails::SetObjects(const TArray<UObject*>& InObjects, bool bForceRefresh)
{
	if (PropertyRowGenerator)
	{
		PropertyRowGenerator->SetObjects(InObjects);
		GenerateWidget();
		if (!Nodes.Num())
		{
			TWeakPtr<SInViewportDetails> ThisWeak = SharedThis(this);

			// Do this on a delay so that we are not caught in a loop of creating and hiding the menu
			GEditor->GetTimerManager()->SetTimerForNextTick([ThisWeak]()
				{
					if (TSharedPtr<SInViewportDetails> This = ThisWeak.Pin())
					{
						if (TSharedPtr<SEditorViewport> Viewport = This->OwningViewport.Pin())
						{
							Viewport->HideInViewportContextMenu();
						}
					}
				});
		}
	}
}

void SInViewportDetails::PostUndo(bool bSuccess)
{

}

void SInViewportDetails::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}



void SInViewportDetails::OnEditorSelectionChanged(UObject* Object)
{
	if (Object && Object->GetWorld() && Object->GetWorld()->WorldType != EWorldType::Editor)
	{
		return;
	}
	TArray<UObject*> SelectedActors;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		AActor* Actor = static_cast<AActor*>(*It);
		checkSlow(Actor->IsA(AActor::StaticClass()));

		if (IsValidChecked(Actor))
		{
			SelectedActors.Add(Actor);
		}
	}
	SetObjects(SelectedActors);
}

AActor* SInViewportDetails::GetSelectedActorInEditor() const
{
	//@todo this doesn't work w/ multi-select
	return GEditor->GetSelectedActors()->GetTop<AActor>();
}

UToolMenu* SInViewportDetails::GetGeneratedToolbarMenu() const
{
	return GeneratedToolbarMenu.IsValid() ? GeneratedToolbarMenu.Get() : nullptr;
}

AActor* SInViewportDetails::GetActorContext() const
{
	AActor* SelectedActorInEditor = GetSelectedActorInEditor();
	
	return SelectedActorInEditor;
}

TSharedRef<ITableRow> SInViewportDetails::GenerateListRow(TSharedPtr<IDetailTreeNode> InItem, const TSharedRef<STableViewBase>& InOwningTable)
{
	return SNew(SInViewportDetailsRow, InOwningTable)
		.InNode(InItem)
		.InDetailsView(SharedThis(this));

}

FReply SInViewportDetails::StartDraggingDetails(FVector2D InTabGrabScreenSpaceOffset, const FPointerEvent& MouseEvent)
{
	FOnInViewportUIDropped OnUIDropped = FOnInViewportUIDropped::CreateSP(this, &SInViewportDetails::FinishDraggingDetails);
	// Start dragging.
	TSharedRef<FInViewportUIDragOperation> DragDropOperation =
		FInViewportUIDragOperation::New(
			SharedThis(this),
			InTabGrabScreenSpaceOffset,
			GetDesiredSize(),
			OnUIDropped
		);
	if (OwningViewport.IsValid())
	{
		OwningViewport.Pin()->ToggleInViewportContextMenu();
	}
	return FReply::Handled().BeginDragDrop(DragDropOperation);
}

void SInViewportDetails::FinishDraggingDetails(const FVector2D InLocation)
{
	if (OwningViewport.IsValid())
	{
		OwningViewport.Pin()->UpdateInViewportMenuLocation(InLocation);
		OwningViewport.Pin()->ToggleInViewportContextMenu();
	}
}

void SInViewportDetailsHeader::Construct(const FArguments& InArgs)
{
	ParentPtr = InArgs._Parent;
	ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("PropertyTable.InViewport.Header"))
			.Padding(FMargin(8.0f, 5.0f))
			[
				InArgs._Content.Widget
			]
		];
}

FReply SInViewportDetailsHeader::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Need to remember where within a tab we grabbed
	const FVector2D TabGrabScreenSpaceOffset = MouseEvent.GetScreenSpacePosition() - MyGeometry.GetAbsolutePosition();

	TSharedPtr<SInViewportDetails> PinnedParent = ParentPtr.Pin();
	if (PinnedParent.IsValid())
	{
		return PinnedParent->StartDraggingDetails(TabGrabScreenSpaceOffset, MouseEvent);
	}

	return FReply::Unhandled();
}

TSharedPtr<class FDragDropOperation> SInViewportDetailsHeader::CreateDragDropOperation()
{
	TSharedPtr<FDragDropOperation> Operation = MakeShareable(new FDragDropOperation());

	return Operation;
}

void SInViewportDetailsToolbar::Construct(const FArguments& InArgs)
{
	TSharedPtr<SInViewportDetails> Parent = InArgs._Parent;
	if (!Parent.IsValid())
	{
		return;
	}
	const FName ToolBarName = GetQuickActionMenuName(Parent->GetSelectedActorInEditor()->GetClass());
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* FoundMenu = ToolMenus->FindMenu(ToolBarName);

	if (!FoundMenu || !FoundMenu->IsRegistered())
	{
		FoundMenu = ToolMenus->RegisterMenu(ToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
	}

	FToolMenuContext MenuContext;

	UQuickActionMenuContext* ToolbarMenuContext = NewObject<UQuickActionMenuContext>(FoundMenu);
	TWeakPtr<ILevelEditor> ParentLevelEditor = Parent->ParentLevelEditor;
	if (ParentLevelEditor.IsValid())
	{
		ToolbarMenuContext->CurrentSelection = ParentLevelEditor.Pin()->GetElementSelectionSet();
	}
	MenuContext.AddObject(ToolbarMenuContext);
	Parent->GeneratedToolbarMenu = ToolMenus->GenerateMenu(ToolBarName, MenuContext);

	// Move this to the GenerateMenu API
	Parent->GeneratedToolbarMenu->StyleName = "InViewportToolbar";
	Parent->GeneratedToolbarMenu->bToolBarIsFocusable = false;
	Parent->GeneratedToolbarMenu->bToolBarForceSmallIcons = true;
	TSharedRef< class SWidget > ToolBarWidget = ToolMenus->GenerateWidget(Parent->GeneratedToolbarMenu.Get());

	ChildSlot
	[
		ToolBarWidget
	];
}

FName SInViewportDetailsToolbar::GetQuickActionMenuName(UClass* InClass)
{
	return FName("LevelEditor.InViewportPanel");
}

#undef LOCTEXT_NAMESPACE