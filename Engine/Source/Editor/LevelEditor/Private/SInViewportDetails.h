// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailColumnSizeData.h"
#include "EditorUndoClient.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Layout/Visibility.h"
#include "ToolMenuContext.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class AActor;
class FSCSEditorTreeNode;
class FTabManager;
class FUICommandList;
class IDetailsView;
class SBox;
class SSplitter;
class UBlueprint;
class FDetailsViewObjectFilter;
class IDetailTreeNode;
class IPropertyRowGenerator;
class ILevelEditor;
class UToolMenu;

/**
 * Wraps a details panel customized for viewing actors
 */
class SInViewportDetails : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SInViewportDetails) {}
	SLATE_ARGUMENT(TSharedPtr<class SEditorViewport>, InOwningViewport)
	SLATE_ARGUMENT(TSharedPtr<ILevelEditor>, InOwningLevelEditor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void GenerateWidget();

	EVisibility GetHeaderVisibility() const;
	TSharedRef<SWidget> MakeDetailsWidget();
	~SInViewportDetails();

	/**
	 * Sets the objects to be viewed by the details panel
	 *
	 * @param InObjects	The objects to set
	 */
	void SetObjects(const TArray<UObject*>& InObjects, bool bForceRefresh = false);

	/** FEditorUndoClient Interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	FReply StartDraggingDetails(FVector2D InTabGrabScreenSpaceOffset, const FPointerEvent& MouseEvent);
	void FinishDraggingDetails(const FVector2D InLocation);
	FDetailColumnSizeData& GetColumnSizeData() { return ColumnSizeData; }
	AActor* GetSelectedActorInEditor() const;
	UToolMenu* GetGeneratedToolbarMenu() const;

	friend class SInViewportDetailsToolbar;

private:
	AActor* GetActorContext() const;
	TSharedRef<ITableRow> GenerateListRow(TSharedPtr<IDetailTreeNode> InItem, const TSharedRef<STableViewBase>& InOwningTable);
	void OnEditorSelectionChanged(UObject* Object);

private:
	TSharedPtr<SSplitter> DetailsSplitter;
	TSharedPtr<SListView<TSharedPtr<class IDetailTreeNode>>> NodeList;
	TArray<TSharedPtr<class IDetailTreeNode>> Nodes;
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;
	FDetailColumnSizeData ColumnSizeData;
	TWeakPtr<class SEditorViewport> OwningViewport;
	TWeakPtr<ILevelEditor> ParentLevelEditor;
	TWeakObjectPtr<UToolMenu> GeneratedToolbarMenu;
};

class SInViewportDetailsHeader : public SCompoundWidget
{

public:
	SLATE_BEGIN_ARGS(SInViewportDetailsHeader) {}
	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_ARGUMENT(TSharedPtr<SInViewportDetails>, Parent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	};


	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TSharedPtr<class FDragDropOperation> CreateDragDropOperation();

	/** The parent in-viewport details */
	TWeakPtr<SInViewportDetails> ParentPtr;
};

class SInViewportDetailsToolbar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInViewportDetailsToolbar) {}
	SLATE_ARGUMENT(TSharedPtr<SInViewportDetails>, Parent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	FName GetQuickActionMenuName(UClass* InClass);

};