// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLevelEditor.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Components/StaticMeshComponent.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Materials/MaterialInterface.h"
#include "ToolMenus.h"
#include "Framework/Docking/LayoutService.h"
#include "EditorModeRegistry.h"
#include "EdMode.h"
#include "Engine/Selection.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "LevelEditorViewport.h"
#include "EditorModes.h"
#include "UnrealEdGlobals.h"
#include "LevelEditor.h"
#include "LevelEditorMenu.h"
#include "IDetailsView.h"
#include "LevelEditorActions.h"
#include "LevelEditorModesActions.h"
#include "LevelEditorContextMenu.h"
#include "LevelEditorToolBar.h"
#include "SLevelEditorToolBox.h"
#include "SLevelEditorBuildAndSubmit.h"
#include "Kismet2/DebuggerCommands.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"
#include "SceneView.h"
#include "LayersModule.h"
#include "WorldBrowserModule.h"
#include "DataLayerEditorModule.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "Toolkits/ToolkitManager.h"
#include "PropertyEditorModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "StatsViewerModule.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "TutorialMetaData.h"
#include "Widgets/Docking/SDockTab.h"
#include "SActorDetails.h"
#include "GameFramework/WorldSettings.h"
#include "Framework/Docking/LayoutExtender.h"
#include "HierarchicalLODOutlinerModule.h"
#include "EditorViewportCommands.h"
#include "IPlacementModeModule.h"
#include "StatusBarSubsystem.h"
#include "Widgets/Colors/SColorPicker.h"
#include "SourceCodeNavigation.h"
#include "EnvironmentLightingModule.h"
#include "Misc/MessageDialog.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "DataStorage/Features.h"
#include "Columns/SceneOutlinerColumns.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/TypedElementCommonActions.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Elements/Interfaces/TypedElementDetailsInterface.h"
#include "Elements/Actor/ActorElementLevelEditorSelectionCustomization.h"
#include "Elements/Actor/ActorElementLevelEditorCommonActionsCustomization.h"
#include "Elements/Component/ComponentElementLevelEditorSelectionCustomization.h"
#include "Elements/Component/ComponentElementLevelEditorCommonActionsCustomization.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/SMInstance/SMInstanceElementId.h"
#include "Elements/SMInstance/SMInstanceElementLevelEditorSelectionCustomization.h"
#include "DerivedDataEditor.h"
#include "EditorInteractiveGizmoManager.h"
#include "ZenEditor.h"
#include "EditorModeManager.h"
#include "EditorViewportLayout.h"
#include "LevelViewportTabContent.h"
#include "SLevelViewport.h"
#include "LevelEditorOutlinerSettings.h"
#include "IWorldHierarchy.h"
#include "LevelViewportActions.h"
#include "Containers/VersePath.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Framework/Commands/GenericCommands.h"
#include "ViewportInteractions/LevelViewportClickSelection.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportInteractionsUtilities.h"
#include "ActorFolders/TypedElements/ActorFolderTypedElementSupport.h"
#include "DataStorage/Scope/EditorDataScope.h"
#include "DataStorage/Scope/EditorDataScopeColumns.h"
#include "DataStorage/Features.h"
#include "Elements/ActorFolder/ActorFolderElementLevelEditorCommonActionsCustomization.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/CommonActions/LevelEditorCommonActions.h"

#define LOCTEXT_NAMESPACE "SLevelEditor"

class UViewportOrbitInteraction;
static const FName MainFrameModuleName("MainFrame");
static const FName LevelEditorModuleName("LevelEditor");
static const FName SceneOutlinerModuleName("SceneOutliner");
static const FName WorldBrowserHierarchyTab("WorldBrowserHierarchy");
static const FName WorldBrowserDetailsTab("WorldBrowserDetails");
static const FName WorldBrowserCompositionTab("WorldBrowserComposition");

TAutoConsoleVariable<FString> CVarSetOutlinerPurposeStr(
	TEXT("TEDS.UI.SetOutlinerPurpose"),
	TEXT("ActorOutliner"),
	TEXT("Set the purpose name to switch the current Outliner, ex:'ActorOutliner', 'TEDSOutliner'"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* /*CVar*/)
	{
		if (FModuleManager::Get().IsModuleLoaded(LevelEditorModuleName))
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModuleName);
			LevelEditorModule.RefreshLevelEditorOutliners();
		}
	}));

SLevelEditor::SLevelEditor()
	: World(nullptr)
	, bNeedsRefresh(false)
	// Ensure the Actor main menu has a default label before the first selection change.
	, CachedViewportContextMenuTitle(FLevelEditorContextMenu::GetContextMenuTitle(ELevelEditorMenuContext::MainMenu, nullptr))
{
	if (UModeManagerInteractiveToolsContext* const ModeManagerInteractiveToolsContext =
			SLevelEditor::GetEditorModeManager().GetInteractiveToolsContext())
	{
		ModeManagerInteractiveToolsContext->SetViewportInteractionsEnabled(true);

		ModeManagerInteractiveToolsContext->OnBuildViewportInteractions().AddRaw(this, &SLevelEditor::RegisterViewportInteractions);
	}
}

void SLevelEditor::BindCommands()
{
	LevelEditorCommands = MakeShareable( new FUICommandList );

	const FLevelEditorCommands& Actions = FLevelEditorCommands::Get();

	// Map UI commands to delegates that are executed when the command is handled by a keybinding or menu
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( LevelEditorModuleName );

	// Append the list of the level editor commands for this instance with the global list of commands for all instances.
	LevelEditorCommands->Append( LevelEditorModule.GetGlobalLevelEditorActions() );

	// Append the list of global PlayWorld commands
	LevelEditorCommands->Append( FPlayWorldCommands::GlobalPlayWorldActions.ToSharedRef() );

	LevelEditorCommands->MapAction( 
		Actions.EditAsset, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::EditAsset_Clicked, EToolkitMode::Standalone, TWeakPtr< SLevelEditor >( SharedThis( this ) ), true ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::EditAsset_CanExecute ) );

	LevelEditorCommands->MapAction( 
		Actions.EditAssetNoConfirmMultiple, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::EditAsset_Clicked, EToolkitMode::Standalone, TWeakPtr< SLevelEditor >( SharedThis( this ) ), false ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::EditAsset_CanExecute ) );

	LevelEditorCommands->MapAction(
		Actions.OpenSelectionInPropertyMatrix,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OpenSelectionInPropertyMatrix_Clicked ),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FLevelEditorActionCallbacks::OpenSelectionInPropertyMatrix_IsVisible));

	LevelEditorCommands->MapAction(
		Actions.CheckOutProjectSettingsConfig,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CheckOutProjectSettingsConfig ) );

	LevelEditorCommands->MapAction(
		Actions.OpenLevelBlueprint,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OpenLevelBlueprint, TWeakPtr< SLevelEditor >( SharedThis( this ) ) ) );
	
	LevelEditorCommands->MapAction(
		Actions.CreateBlankBlueprintClass,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CreateBlankBlueprintClass ) );

	LevelEditorCommands->MapAction(
		Actions.ConvertSelectionToBlueprint,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ConvertSelectedActorsIntoBlueprintClass ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::CanConvertSelectedActorsIntoBlueprintClass ) );

	LevelEditorCommands->MapAction(
		Actions.OpenPlaceActors,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OpenPlaceActors) );

	LevelEditorCommands->MapAction(
		Actions.OpenContentBrowser,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OpenContentBrowser ) );
	
	LevelEditorCommands->MapAction(
		Actions.ImportContent,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ImportContent));

	LevelEditorCommands->MapAction(
		Actions.ToggleVR,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ToggleVR ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ToggleVR_CanExecute ),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::ToggleVR_IsChecked ),
		FIsActionButtonVisible::CreateStatic(&FLevelEditorActionCallbacks::ToggleVR_IsButtonActive));

	LevelEditorCommands->MapAction(
		Actions.WorldProperties,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnShowWorldProperties, TWeakPtr< SLevelEditor >( SharedThis( this ) ) ) );
	
	LevelEditorCommands->MapAction( 
		FEditorViewportCommands::Get().FocusAllViewportsToSelection,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("CAMERA ALIGN") ) )
		);

	LevelEditorCommands->MapAction( 
		FEditorViewportCommands::Get().FocusViewportToSelection, 
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY") ) )
		);

	LevelEditorCommands->MapAction(
		FEditorViewportCommands::Get().FocusOutlinerToSelection,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnFocusOutlinerToSelection, TWeakPtr< SLevelEditor >( SharedThis( this ) ) )
		);

	LevelEditorCommands->MapAction(
		FEditorViewportCommands::Get().FocusOutlinerToContextFolder,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnFocusOutlinerToContextFolder, TWeakPtr< SLevelEditor >( SharedThis( this ) ) )
		);
		
	LevelEditorCommands->MapAction(
		FLevelViewportCommands::Get().ToggleViewportToolbar,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::OnToggleShowViewportToolbar ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &FLevelEditorActionCallbacks::IsViewportToolbarVisible )
		);

	if (FPlayWorldCommands::GlobalPlayWorldActions.IsValid())
	{
		FUICommandList& PlayWorldActionList = *FPlayWorldCommands::GlobalPlayWorldActions;
		PlayWorldActionList.MapAction(Actions.RecompileGameCode, *LevelEditorCommands->GetActionForCommand(Actions.RecompileGameCode));
	}
}

void SLevelEditor::RegisterMenus()
{
	FLevelEditorMenu::RegisterLevelEditorMenus();
	FLevelEditorToolBar::RegisterLevelEditorToolBar(LevelEditorCommands.ToSharedRef(), SharedThis(this));
	FLevelEditorToolBar::RegisterLevelEditorSecondaryModeToolbar();
}

void SLevelEditor::Construct( const SLevelEditor::FArguments& InArgs)
{
	ActorDetailsSCSEditorUICustomization = FActorDetailsSCSEditorUICustomization::NewInstance();
	
	// Important: We use raw bindings here because we are releasing our binding in our destructor (where a weak pointer would be invalid)
	// It's imperative that our delegate is removed in the destructor for the level editor module to play nicely with reloading.

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked< FLevelEditorModule >( LevelEditorModuleName );
	LevelEditorModule.OnTitleBarMessagesChanged().AddRaw( this, &SLevelEditor::ConstructTitleBarMessages );

	GetMutableDefault<UEditorExperimentalSettings>()->OnSettingChanged().AddRaw(this, &SLevelEditor::HandleExperimentalSettingChanged);

	BindCommands();

	// Rebuild the editor mode commands and their tab spawners before we restore the layout,
	// or there wont be any tab spawners for the modes.
	RefreshEditorModeCommands();

	RegisterMenus();

	// We need to register when modes list changes so that we can refresh the auto generated commands.
	if (GEditor != nullptr)
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->OnEditorModesChanged().AddRaw(this, &SLevelEditor::EditorModeCommandsChanged);
		}
	}

	// @todo This is a hack to get this working for now. This won't work with multiple worlds
	if (GEditor != nullptr)
	{
		GEditor->GetEditorWorldContext(true).AddRef(World);

		// Set the initial preview feature level.
		World->ShaderPlatformChanged(GEditor->GetActiveShaderPlatform());

		LevelActorDeletedHandle = GEditor->OnLevelActorDeleted().AddSP(this, &SLevelEditor::OnLevelActorDeleted);
		LevelActorOuterChangedHandle = GEditor->OnLevelActorOuterChanged().AddSP(this, &SLevelEditor::OnLevelActorOuterChanged);
	}

	PreviewPlatformChangedHandle = GEditor->OnPreviewShaderPlatformChanged().AddLambda([this](EShaderPlatform NewShaderPlatform)
		{
			// Do one recapture if atleast one ReflectionComponent is dirty
			// BuildReflectionCapturesOnly_Execute in LevelEditorActions relies on this happening on toggle between SM5->ES31. If you remove this, update that code!
			ERHIFeatureLevel::Type NewFeatureLevel = GetMaxSupportedFeatureLevel(NewShaderPlatform);
			if (World->NumUnbuiltReflectionCaptures >= 1 && NewFeatureLevel == ERHIFeatureLevel::ES3_1 && GEditor != nullptr)
			{
				GEditor->BuildReflectionCaptures();
			}
			World->ShaderPlatformChanged(NewShaderPlatform, true);
		});
	// Create the level editor scope row, parented to the current context (root)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Scope;
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			LevelEditorScopeRow = Storage->AddScopeRow(TEXT("Level Editor"));
			Storage->SetParentScope(LevelEditorScopeRow, GetCurrentScope());
		}
	}
	FEditorDelegates::MapChange.AddRaw(this, &SLevelEditor::HandleEditorMapChange);
	FEditorDelegates::OnAssetsDeleted.AddRaw(this, &SLevelEditor::HandleAssetsDeleted);

	HandleEditorMapChange(MapChangeEventFlags::NewMap);

	// Register the display names of the outliners
	SceneOutlinerDisplayNames.Add(LevelEditorTabIds::LevelEditorSceneOutliner, NSLOCTEXT("LevelEditorTabs", "LevelEditorSceneOutliner", "Outliner 1"));
	SceneOutlinerDisplayNames.Add(LevelEditorTabIds::LevelEditorSceneOutliner2, NSLOCTEXT("LevelEditorTabs", "LevelEditorSceneOutliner2", "Outliner 2"));
	SceneOutlinerDisplayNames.Add(LevelEditorTabIds::LevelEditorSceneOutliner3, NSLOCTEXT("LevelEditorTabs", "LevelEditorSceneOutliner3", "Outliner 3"));
	SceneOutlinerDisplayNames.Add(LevelEditorTabIds::LevelEditorSceneOutliner4, NSLOCTEXT("LevelEditorTabs", "LevelEditorSceneOutliner4", "Outliner 4"));
}

void SLevelEditor::Initialize( const TSharedRef<SDockTab>& OwnerTab, const TSharedRef<SWindow>& OwnerWindow )
{
	// Push the level editor context so child widgets constructed during Initialize see it
	UE::Editor::DataStorage::Scope::FPushScopeGuard PushScopeGuard(LevelEditorScopeRow);

	SelectedElements = NewObject<UTypedElementSelectionSet>(GetTransientPackage(), NAME_None, RF_Transactional);
	SelectedElements->AddToRoot();

	// Register the level editor specific selection behavior
	{
		TUniquePtr<FActorElementLevelEditorSelectionCustomization> ActorCustomization = MakeUnique<FActorElementLevelEditorSelectionCustomization>();
		ActorCustomization->SetToolkitHost(this);
		SelectedElements->RegisterInterfaceCustomizationByTypeName(NAME_Actor, MoveTemp(ActorCustomization));
	}
	{
		TUniquePtr<FComponentElementLevelEditorSelectionCustomization> ComponentCustomization = MakeUnique<FComponentElementLevelEditorSelectionCustomization>();
		ComponentCustomization->SetToolkitHost(this);
		SelectedElements->RegisterInterfaceCustomizationByTypeName(NAME_Components, MoveTemp(ComponentCustomization));
	}
	{
		TUniquePtr<FSMInstanceElementLevelEditorSelectionCustomization> SMInstanceCustomization = MakeUnique<FSMInstanceElementLevelEditorSelectionCustomization>();
		SMInstanceCustomization->SetToolkitHost(this);
		SelectedElements->RegisterInterfaceCustomizationByTypeName(NAME_SMInstance, MoveTemp(SMInstanceCustomization));
	}

	// Allow USelection to bridge to our selected element list
	GUnrealEd->GetSelectedActors()->SetElementSelectionSet(SelectedElements);
	GUnrealEd->GetSelectedComponents()->SetElementSelectionSet(SelectedElements);

	// Setup a callback to deselect instances of Instanced Static Meshes so there are no lingering references.
	FSMInstanceElementIdMap::Get().OnInstancePreRemoval().AddSP(this, &SLevelEditor::OnIsmInstanceRemoving);

	CommonActions = NewObject<ULevelEditorCommonActions>();
	CommonActions->AddToRoot();

	// Register the level editor specific selection behavior
	{
		TUniquePtr<FActorElementLevelEditorCommonActionsCustomization> ActorCustomization = MakeUnique<FActorElementLevelEditorCommonActionsCustomization>();
		ActorCustomization->SetToolkitHost(this);
		CommonActions->RegisterInterfaceCustomizationByTypeName(NAME_Actor, MoveTemp(ActorCustomization));
	}
	{
		TUniquePtr<FComponentElementLevelEditorCommonActionsCustomization> ComponentCustomization = MakeUnique<FComponentElementLevelEditorCommonActionsCustomization>();
		ComponentCustomization->SetToolkitHost(this);
		CommonActions->RegisterInterfaceCustomizationByTypeName(NAME_Components, MoveTemp(ComponentCustomization));
	}
	{
		TUniquePtr<FActorFolderElementLevelEditorCommonActionsCustomization> FolderCustomization = MakeUnique<FActorFolderElementLevelEditorCommonActionsCustomization>();
		FolderCustomization->SetToolkitHost(this);
		CommonActions->RegisterInterfaceCustomizationByTypeName(UE::Editor::ActorFolders::NAME_ActorFolder, MoveTemp(FolderCustomization));
	}

	// Bind the level editor tab's label to the currently loaded level name string in the main frame
	OwnerTab->SetLabel( TAttribute<FText>( this, &SLevelEditor::GetTabTitle) );
	OwnerTab->SetTabLabelSuffix(TAttribute<FText>(this, &SLevelEditor::GetTabSuffix));

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked< FLevelEditorModule >(LevelEditorModuleName);

	FModuleManager::Get().LoadModuleChecked("StatusBar");

	LevelEditorModule.OnElementSelectionChanged().AddSP(this, &SLevelEditor::OnElementSelectionChanged);
	LevelEditorModule.OnActorSelectionChanged().AddSP(this, &SLevelEditor::OnActorSelectionChanged);
	LevelEditorModule.OnOverridePropertyEditorSelection().AddSP(this, &SLevelEditor::OnOverridePropertyEditorSelection);

	TSharedRef<SWidget> ContentArea = RestoreContentArea( OwnerTab, OwnerWindow );

	FLevelEditorMenu::MakeLevelEditorMenu(LevelEditorCommands, SharedThis(this));


	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			ContentArea
		]
		+SVerticalBox::Slot()
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		.AutoHeight()
		[
			GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->MakeStatusBarWidget(GetStatusBarName(), OwnerTab)
		]
	];
	
	RegisterStatusBarTools();

	TtileBarMessageBox = SNew(SHorizontalBox);

	ConstructTitleBarMessages();

	OnLayoutHasChanged();
}

void SLevelEditor::ConstructTitleBarMessages()
{
	TtileBarMessageBox->ClearChildren();

	const IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked<IMainFrameModule>(MainFrameModuleName);
	const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked< FLevelEditorModule >(LevelEditorModuleName);
	
	TArray<FMainFrameDeveloperTool> Tools;
	for (const TPair<FName, FLevelEditorModule::FTitleBarItem>& Item : LevelEditorModule.GetTitleBarItems())
	{
		Tools.Add({Item.Value.Visibility, Item.Value.Label, Item.Value.Value});
	}

	TtileBarMessageBox->AddSlot()
		.AutoWidth()
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			MainFrameModule.MakeDeveloperTools( Tools )
		];
}


SLevelEditor::~SLevelEditor()
{
	// Clean up the scope row
	{
		using namespace UE::Editor::DataStorage;
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			Storage->RemoveScopeRow(LevelEditorScopeRow);
		}
		LevelEditorScopeRow = InvalidRowHandle;
	}

	ActorDetailsSCSEditorUICustomization = nullptr;

	// We're going away now, so make sure all toolkits that are hosted within this level editor are shut down
	FToolkitManager::Get().OnToolkitHostDestroyed( this );
	HostedToolkits.Reset();

	// Deactivate any active modes, and reset back to the default mode.
	GetEditorModeManager().SetDefaultMode(FBuiltinEditorModes::EM_Default);
	GetEditorModeManager().DeactivateAllModes();
	
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked< FLevelEditorModule >( LevelEditorModuleName );
	LevelEditorModule.OnTitleBarMessagesChanged().RemoveAll( this );
	
	if(UObjectInitialized())
	{
		GetMutableDefault<UEditorExperimentalSettings>()->OnSettingChanged().RemoveAll(this);
		GetMutableDefault<UEditorPerProjectUserSettings>()->OnUserSettingChanged().RemoveAll(this);
	}

	FEditorDelegates::OnAssetsDeleted.RemoveAll(this);
	FEditorDelegates::MapChange.RemoveAll(this);

	if (GEngine)
	{
		CastChecked<UEditorEngine>(GEngine)->OnPreviewShaderPlatformChanged().Remove(PreviewPlatformChangedHandle);
	}

	if (GEditor)
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->OnEditorModesChanged().RemoveAll(this);
		}
		GEditor->OnLevelActorDeleted().Remove(LevelActorDeletedHandle);
		GEditor->OnLevelActorOuterChanged().Remove(LevelActorOuterChangedHandle);
		GEditor->GetEditorWorldContext(true).RemoveRef(World);
	}

	// Clear USelection from using our selected element list
	if (UObjectInitialized())
	{
		if (!SelectedElements->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
			SelectedElements->ClearSelection(FTypedElementSelectionOptions());
		}
		SelectedElements->OnPreChange().Clear();
		SelectedElements->OnChanged().Clear();
		SelectedElements->RemoveFromRoot();
		SelectedElements = nullptr;

		CommonActions->RemoveFromRoot();
		CommonActions = nullptr;
	}
	if (GUnrealEd)
	{
		GUnrealEd->GetSelectedActors()->SetElementSelectionSet(nullptr);
		GUnrealEd->GetSelectedComponents()->SetElementSelectionSet(nullptr);
	}

	if (UModeManagerInteractiveToolsContext* const ModeManagerInteractiveToolsContext =
			SLevelEditor::GetEditorModeManager().GetInteractiveToolsContext())
	{
		ModeManagerInteractiveToolsContext->OnBuildViewportInteractions().RemoveAll(this);
	}
}

FText SLevelEditor::GetTabTitle() const
{
	const IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked< IMainFrameModule >( MainFrameModuleName );

	return FText::FromString(MainFrameModule.GetLoadedLevelName());
}

FText SLevelEditor::GetTabSuffix() const
{
	const bool bDirtyState = World && World->GetCurrentLevel()->GetOutermost()->IsDirty();
	return bDirtyState ? FText::FromString(TEXT("*")) : FText::GetEmpty();
}

bool SLevelEditor::HasActivePlayInEditorViewport() const
{
	// Search through all current viewport layouts
	for( int32 TabIndex = 0; TabIndex < ViewportTabs.Num(); ++TabIndex )
	{
		TWeakPtr<FLevelViewportTabContent> ViewportTab = ViewportTabs[ TabIndex ];

		if (ViewportTab.IsValid())
		{
			// Get all the viewports in the layout
			const TMap< FName, TSharedPtr< IEditorViewportLayoutEntity > >* LevelViewports = ViewportTab.Pin()->GetViewports();

			if (LevelViewports != NULL)
			{
				// Search for a viewport with a pie session
				for (auto& Pair : *LevelViewports)
				{
					TSharedPtr<ILevelViewportLayoutEntity> ViewportEntity = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Pair.Value);
					if (ViewportEntity.IsValid() && ViewportEntity->IsPlayInEditorViewportActive())
					{
						return true;
					}
				}
			}
		}
	}

	// Also check standalone viewports
	for( const TWeakPtr< SLevelViewport >& StandaloneViewportWeakPtr : StandaloneViewports )
	{
		const TSharedPtr< SLevelViewport > Viewport = StandaloneViewportWeakPtr.Pin();
		if( Viewport.IsValid() )
		{
			if( Viewport->IsPlayInEditorViewportActive() )
			{
				return true;
			}
		}
	}

	return false;
}

void SLevelEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Push the level editor context so child widget ticks see it via TLS
	UE::Editor::DataStorage::Scope::FPushScopeGuard PushScopeGuard(LevelEditorScopeRow);

	// Update the ActiveViewport
	TSharedPtr<SLevelViewport> LastActiveViewport = CachedActiveViewport.Pin();
	if (!LastActiveViewport || &LastActiveViewport->GetLevelViewportClient() != GCurrentLevelEditingViewportClient)
	{
		TSharedPtr<SLevelViewport> NewActiveViewport = GetActiveViewport();
		CachedActiveViewport = NewActiveViewport;
		OnActiveViewportChanged().Broadcast(LastActiveViewport, NewActiveViewport);
	}
}

TSharedPtr<SLevelViewport> SLevelEditor::GetActiveViewport() const
{
	// The first visible viewport
	TSharedPtr<SLevelViewport> FirstVisibleViewport;

	// Search through all current viewport tabs
	for( int32 TabIndex = 0; TabIndex < ViewportTabs.Num(); ++TabIndex )
	{
		TSharedPtr<FLevelViewportTabContent> ViewportTab = ViewportTabs[ TabIndex ].Pin();

		if (ViewportTab.IsValid())
		{
			// Only check the viewports in the tab if its visible
			if( ViewportTab->IsVisible() )
			{
				const TMap< FName, TSharedPtr< IEditorViewportLayoutEntity > >* LevelViewports = ViewportTab->GetViewports();

				if (LevelViewports != nullptr)
				{
					for(auto& Pair : *LevelViewports)
					{
						TSharedPtr<SLevelViewport> Viewport = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Pair.Value)->AsLevelViewport();
						if( Viewport.IsValid() && Viewport->IsInForegroundTab() )
						{
							if( &Viewport->GetLevelViewportClient() == GCurrentLevelEditingViewportClient )
							{
								// If the viewport is visible and is also the current level editing viewport client
								// return it as the active viewport
								return Viewport;
							}
							else if( !FirstVisibleViewport.IsValid() )
							{
								// If there is no current first visible viewport set it now
								// We will return this viewport if the current level editing viewport client is not visible
								FirstVisibleViewport = Viewport;
							}
						}
					}
				}
			}
		}
	}

	// Also check standalone viewports
	for( const TWeakPtr< SLevelViewport >& StandaloneViewportWeakPtr : StandaloneViewports )
	{
		const TSharedPtr< SLevelViewport > Viewport = StandaloneViewportWeakPtr.Pin();
		if( Viewport.IsValid() )
		{
			if( &Viewport->GetLevelViewportClient() == GCurrentLevelEditingViewportClient )
			{
				// If the viewport is visible and is also the current level editing viewport client
				// return it as the active viewport
				return Viewport;
			}
			else if( !FirstVisibleViewport.IsValid() )
			{
				// If there is no current first visible viewport set it now
				// We will return this viewport if the current level editing viewport client is not visible
				FirstVisibleViewport = Viewport;
			}
		}
	}
	
	// Return the first visible viewport if we found one.  This can be null if we didn't find any visible viewports
	return FirstVisibleViewport;
}


TSharedRef< SWidget > SLevelEditor::GetParentWidget()
{
	return AsShared();
}

void SLevelEditor::BringToFront()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( LevelEditorModuleName );
	TSharedPtr<SDockTab> LevelEditorTab = LevelEditorModule.GetLevelEditorInstanceTab().Pin();
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	if (LevelEditorTabManager.IsValid() && LevelEditorTab.IsValid())
	{
		LevelEditorTabManager->DrawAttention( LevelEditorTab.ToSharedRef() );
	}
}

void SLevelEditor::OnToolkitHostingStarted( const TSharedRef< class IToolkit >& Toolkit )
{
	// @todo toolkit minor: We should consider only allowing a single toolkit for a specific asset editor type hosted
	//   at once.  OR, we allow multiple to be hosted, but we only show tabs for one at a time (fast switching.)
	//   Otherwise, it's going to be a huge cluster trying to distinguish tabs for different assets of the same type
	//   of editor
	TSharedPtr<FLevelEditorModeUILayer> ModeUILayer = MakeShareable(new FLevelEditorModeUILayer(this));
	ModeUILayer->SetSecondaryModeToolbarName(FLevelEditorToolBar::GetSecondaryModeToolbarName());
	HostedToolkits.Add( Toolkit );
	ModeUILayer->OnToolkitHostingStarted(Toolkit);

	ModeUILayers.Add(Toolkit->GetToolkitFName(), ModeUILayer);
	
	SecondaryModeToolbarWidget->SetContent(FLevelEditorToolBar::MakeLevelEditorSecondaryModeToolbar(LevelEditorCommands.ToSharedRef(), ModeUILayers));
}

void SLevelEditor::OnToolkitHostingFinished( const TSharedRef< class IToolkit >& Toolkit )
{
	TSharedPtr<FLevelEditorModeUILayer> ModeUILayer; 
	ModeUILayers.RemoveAndCopyValue(Toolkit->GetToolkitFName(), ModeUILayer);
	if(ModeUILayer)
	{
		ModeUILayer->OnToolkitHostingFinished(Toolkit);
	}
	HostedToolkits.Remove( Toolkit );

	SecondaryModeToolbarWidget->SetContent(FLevelEditorToolBar::MakeLevelEditorSecondaryModeToolbar(LevelEditorCommands.ToSharedRef(), ModeUILayers));

	// @todo toolkit minor: If user clicks X on all opened world-centric toolkit tabs, should we exit that toolkit automatically?
	//   Feel 50/50 about this.  It's totally valid to use the "Save" menu even after closing tabs, etc.  Plus, you can spawn the tabs back up using the tab area down-down menu.
}

TSharedPtr<FTabManager> SLevelEditor::GetTabManager() const
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( LevelEditorModuleName );
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	return LevelEditorTabManager;
}

TSharedPtr<SDockTab> SLevelEditor::AttachSequencer( TSharedPtr<SWidget> SequencerWidget, TSharedPtr<IAssetEditorInstance> NewSequencerAssetEditor )
{
	struct Local
	{
		static void OnSequencerClosed( TSharedRef<SDockTab> DockTab, TWeakPtr<IAssetEditorInstance> InSequencerAssetEditor )
		{
			TSharedPtr<IAssetEditorInstance> AssetEditorInstance = InSequencerAssetEditor.Pin();

			if (AssetEditorInstance.IsValid())
			{
				InSequencerAssetEditor.Pin()->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
			}
		}
	};

	static bool bIsReentrant = false;

	if( !bIsReentrant )
	{
		TSharedPtr<SDockTab> Tab = TryInvokeTab(LevelEditorTabIds::Sequencer);
		if(Tab.IsValid())
		{
			// Close the sequence editor after invoking a sequencer tab instead of before so that the existing asset editor doesn't refer to a stale sequencer.
			if (SequencerAssetEditor.IsValid())
			{
				// Closing the window will invoke this method again but we are handling reopening with a new movie scene ourselves
				TGuardValue<bool> ReentrantGuard(bIsReentrant, true);
				// Shutdown cleanly
				SequencerAssetEditor.Pin()->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
			}

			if (!FGlobalTabmanager::Get()->OnOverrideDockableAreaRestore_Handler.IsBound())
			{
				// Don't allow standard tab closing behavior when the override is active
				Tab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateStatic(&Local::OnSequencerClosed, TWeakPtr<IAssetEditorInstance>(NewSequencerAssetEditor)));
			}
			if (SequencerWidget.IsValid() && NewSequencerAssetEditor.IsValid())
			{
				Tab->SetContent(SequencerWidget.ToSharedRef());
				SequencerWidgetPtr = SequencerWidget;
				SequencerAssetEditor = NewSequencerAssetEditor;
				if (FGlobalTabmanager::Get()->OnOverrideDockableAreaRestore_Handler.IsBound())
				{
					// @todo vreditor: more general vr editor tab manager should handle windows instead
					// Close the original tab so we just work with the override window
					Tab->RequestCloseTab();
				}
			}
			else
			{
				Tab->SetContent(SNullWidget::NullWidget);
				Tab->RequestCloseTab();
				SequencerAssetEditor.Reset();
			}
		}

		return Tab;
	}
	
	return nullptr;
}

void SLevelEditor::GetSceneOutlinerInitializationOptions(const FName TabIdentifier, FSceneOutlinerInitializationOptions& OutInitOptions)
{
	OutInitOptions.bShowTransient = true;

	/**
	 * Pass level editor commands to allow scene outliner tab to handle them properly
	 * even if outliner scene tab is undocked and events are not processed by SLevelEditor since they do not share the same window anymore
	 */
	OutInitOptions.CommandList = LevelEditorCommands;

	{
		static const FName MenuName = "LevelEditor.LevelEditorSceneOutliner.ContextMenu";
		FLevelEditorContextMenu::RegisterAsDerivedMenu(MenuName, "SceneOutliner.DefaultContextMenuBase", 
			FToolMenuInsert("MainSection", EToolMenuInsertType::Before));

		TWeakPtr<SLevelEditor> WeakLevelEditor = SharedThis(this);
		OutInitOptions.ModifyContextMenu.BindLambda([=](FName& OutMenuName, FToolMenuContext& MenuContext)
			{
				OutMenuName = MenuName;

				if (WeakLevelEditor.IsValid())
				{
					FLevelEditorContextMenu::InitMenuContext(MenuContext, WeakLevelEditor, ELevelEditorMenuContext::SceneOutliner);
				}
			});
	}

	OutInitOptions.OutlinerIdentifier = TabIdentifier;
	OutInitOptions.FilterBarOptions.bHasFilterBar = true;
	
	// All level editor outliners share their custom text filters
	OutInitOptions.FilterBarOptions.bUseSharedSettings = true;

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModuleName);
	
	TSharedPtr<FLevelEditorOutlinerSettings> OutlinerSettings = LevelEditorModule.GetLevelEditorOutlinerSettings();
	OutlinerSettings->GetOutlinerFilters(OutInitOptions.FilterBarOptions);
	OutInitOptions.FilterBarOptions.CategoryToExpand = OutlinerSettings->GetFilterCategory(FLevelEditorOutlinerBuiltInCategories::Common());
}

TArray<TWeakPtr<ISceneOutliner>> SLevelEditor::GetAllSceneOutliners() const
{
	TArray<TWeakPtr<ISceneOutliner>> OutValueArray;
	SceneOutliners.GenerateValueArray(OutValueArray);
	return OutValueArray;
}

void SLevelEditor::SetMostRecentlyUsedSceneOutliner(FName OutlinerIdentifier)
{
	TWeakPtr<ISceneOutliner> *Outliner = SceneOutliners.Find(OutlinerIdentifier);

	if(Outliner)
	{
		if (TSharedPtr<ISceneOutliner> OutlinerPtr = Outliner->Pin())
		{
			SceneOutlinerPtr = OutlinerPtr;
		}
	}
}

void SLevelEditor::ResetMostRecentOutliner()
{
	// Pick the first available outliner as the most recent outliner
	for(const TPair<FName, TWeakPtr<ISceneOutliner>>& Outliner : SceneOutliners)
	{
		if(TSharedPtr<ISceneOutliner> OutlinerPin = Outliner.Value.Pin())
		{
			SceneOutlinerPtr = OutlinerPin;
			return;
		}
	}

	// No outliners are open currently
	SceneOutlinerPtr.Reset();
}

void SLevelEditor::RegisterViewportInteractions(UViewportInteractionsBehaviorSource* Source) const
{
	UE::Editor::ViewportInteractions::AddDefaultCameraMovementInteractions(Source);
	UE::Editor::ViewportInteractions::AddDefaultDragToolsInteractions(Source);

	// Left Mouse Click
	Source->AddInteraction<ULevelViewportClickSelection>();

	// Right Mouse Click - summons context menu
	Source->AddInteraction<ULevelViewportClickContextMenu>();
	
	// Middle Mouse Click
	Source->AddInteraction<ULevelViewportSetPivot>();

	// Double click
	Source->AddInteraction<ULevelViewportDoubleClick>();
}

TSharedPtr<ISceneOutliner> SLevelEditor::GetMostRecentlyUsedSceneOutliner()
{
	// If the outliner marked as "most recent" is still open, simply return it
	if(TSharedPtr<ISceneOutliner> SceneOutlinerPin = SceneOutlinerPtr.Pin())
	{
		return SceneOutlinerPin;
	}

	// Otherwise, pick the first available outliner to be the most recent one
	ResetMostRecentOutliner();

	return SceneOutlinerPtr.Pin();
}

TSharedRef<SDockTab> SLevelEditor::SummonDetailsPanel( FName TabIdentifier )
{
	TSharedRef<SActorDetails> ActorDetails = StaticCastSharedRef<SActorDetails>( CreateActorDetails( TabIdentifier ) );

	TWeakPtr<SActorDetails> ActorDetailsWeakPtr = ActorDetails;

	const FText Label = NSLOCTEXT( "LevelEditor", "DetailsTabTitle", "Details" );

	TSharedRef<SDockTab> DocTab = SNew(SDockTab)
		.Label( Label )
		.ToolTip( IDocumentation::Get()->CreateToolTip( Label, nullptr, "Shared/LevelEditor", "DetailsTab" ) )
		.OnTabDrawerClosed_Lambda([ActorDetailsWeakPtr]()
			{
				// Close the color picker if a details panel is put into a drawer since you cant visually see the properties anymore
				if (TSharedPtr<SColorPicker> ColorPicker = GetColorPicker())
				{
					if (ColorPicker->GetOptionalOwningDetailsView().IsValid())
					{
						DestroyColorPicker();
					}
				}
			})
		[
			SNew( SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.AddMetaData<FTutorialMetaData>(FTutorialMetaData(TEXT("ActorDetails"), TEXT("LevelEditorSelectionDetails")))
			[
				ActorDetails
			]
		];

	return DocTab;
}

TSharedRef<ISceneOutliner> SLevelEditor::CreateSceneOutliner(FName TabIdentifier)
{
	using namespace UE::Editor::DataStorage;
	
	TSharedPtr<ISceneOutliner> NewSceneOutlinerPtr;

	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	IUiProvider* StorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
	if (Storage && StorageUi)
	{
		IUiProvider::FWidgetConstructorPtr OutSceneOutlinerConstructorPtr;

		auto AssignWidgetToColumn = [&OutSceneOutlinerConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			OutSceneOutlinerConstructorPtr = MoveTemp(WidgetConstructor);
			return false;
		};

		const FName TEDSOutlinerId = FName(*CVarSetOutlinerPurposeStr->GetString());
		RowHandle WidgetPurposeRow = StorageUi->FindPurpose(
			IUiProvider::FPurposeInfo(LevelEditorModuleName, SceneOutlinerModuleName, TEDSOutlinerId).GeneratePurposeID());
		
		// If the String given by the CVar isn't a valid purpose, fallback to the base outliner and let the user know
		if (!ensureAlwaysMsgf(Storage->IsRowAvailable(WidgetPurposeRow),
			TEXT("The given Outliner purpose was not found, falling back to using the base Non-TEDS Outliner")))
		{
			WidgetPurposeRow = StorageUi->FindPurpose(
				IUiProvider::FPurposeInfo(LevelEditorModuleName, SceneOutlinerModuleName, "ActorOutliner").GeneratePurposeID());
		}
		StorageUi->CreateWidgetConstructors(WidgetPurposeRow, FMetaDataView(), AssignWidgetToColumn);

		if (OutSceneOutlinerConstructorPtr)
		{
			const FName NamePurposeTableName = TEXT("Editor_WidgetTable");
			
			if (RowHandle RowHandle = Storage->AddRow(Storage->FindTable(NamePurposeTableName));
				Storage->IsRowAvailable(RowHandle))
			{
				UE::Editor::DataStorage::FMetaData OutlinerMetaData;
				OutlinerMetaData.AddImmutableData(TEXT("OutlinerIdentifier"), TabIdentifier.ToString());

				// "Pin" the Outliner Widget so that the WeakPtr in FSceneOutlinerColumn will be valid
				const TSharedPtr<SWidget> OutlinerWidget = StorageUi->ConstructWidget(RowHandle, *OutSceneOutlinerConstructorPtr, FGenericMetaDataView(OutlinerMetaData));
				if (OutlinerWidget)
				{
					// Scene Outliner constructors should assign the FSceneOutlinerColumn to signify that a valid ISceneOutliner was created
					const FSceneOutlinerColumn* OutlinerColumn = Storage->GetColumn<FSceneOutlinerColumn>(RowHandle);
					if (OutlinerColumn && OutlinerColumn->Outliner.IsValid())
					{
						NewSceneOutlinerPtr = OutlinerColumn->Outliner.Pin();
					}
				}
			}
		}
	}

	// Always fallback to the standard way of creating the Outliner if TEDS isn't available or widget construction failed
	if (!NewSceneOutlinerPtr)
	{
		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>(SceneOutlinerModuleName);
		FSceneOutlinerInitializationOptions InitOptions;
		GetSceneOutlinerInitializationOptions(TabIdentifier, InitOptions);
		
		NewSceneOutlinerPtr = SceneOutlinerModule.CreateActorBrowser(InitOptions);
	}
	
	// Add this to the map of all outliners
	SceneOutliners.Add(TabIdentifier, NewSceneOutlinerPtr);

	// Update the most recently used outliner
	SetMostRecentlyUsedSceneOutliner(TabIdentifier);

	return NewSceneOutlinerPtr.ToSharedRef();
}

void SLevelEditor::OnExtendSceneOutlinerTabContextMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("LevelEditorOutlinerContextMenu", LOCTEXT("LevelEditorOutlinerContextMenuOptions", "Outliner Tabs"));

	for (const TPair<FName, FText>& OutlinerTabName : SceneOutlinerDisplayNames)
	{
		InMenuBuilder.AddMenuEntry(OutlinerTabName.Value,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([OutlinerTabName]()
					{
						FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModuleName);
						TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

						// Close the tab if it is already open, otherwise open it
						if (TSharedPtr<SDockTab> LevelEditorOutlinerTab = LevelEditorTabManager->FindExistingLiveTab(OutlinerTabName.Key))
						{
							LevelEditorOutlinerTab->RequestCloseTab();
						}
						else
						{
							LevelEditorTabManager->TryInvokeTab(OutlinerTabName.Key);
						}
					}
				),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([OutlinerTabName]()
					{
						FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModuleName);
						TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

						return LevelEditorTabManager->FindExistingLiveTab(OutlinerTabName.Key) != nullptr;
					}
				)),
			NAME_None,
			EUserInterfaceActionType::Check);
	}

	InMenuBuilder.EndSection();
}

FText SLevelEditor::GetSceneOutlinerLabel(FName SceneOutlinerTabIdentifier)
{
	int32 NumActiveOutliners = 0;
	for (const TPair<FName, TWeakPtr<SDockTab>>& OutlinerTab : SceneOutlinerTabs)
	{
		if (OutlinerTab.Value.IsValid())
		{
			++NumActiveOutliners;
		}
	}

	// If there is only one outliner open, we don't need to call it "Outliner 1"
	if (NumActiveOutliners == 1)
	{
		return NSLOCTEXT("LevelEditor", "SceneOutlinerTabTitle", "Outliner");
	}

	return SceneOutlinerDisplayNames[SceneOutlinerTabIdentifier];
}

/** Method to call when a tab needs to be spawned by the FLayoutService */
TSharedRef<SDockTab> SLevelEditor::SpawnLevelEditorTab( const FSpawnTabArgs& Args, FName TabIdentifier, FString InitializationPayload )
{
	UE::Editor::DataStorage::Scope::FPushScopeGuard PushScopeGuard(LevelEditorScopeRow);
	
	if( TabIdentifier == LevelEditorTabIds::LevelEditorViewport)
	{
		return this->BuildViewportTab( NSLOCTEXT("LevelViewportTypes", "LevelEditorViewport", "Viewport 1"), TEXT("Viewport 1"), InitializationPayload );
	}
	else if( TabIdentifier == LevelEditorTabIds::LevelEditorViewport_Clone1)
	{
		return this->BuildViewportTab( NSLOCTEXT("LevelViewportTypes", "LevelEditorViewport_Clone1", "Viewport 2"), TEXT("Viewport 2"), InitializationPayload );
	}
	else if( TabIdentifier == LevelEditorTabIds::LevelEditorViewport_Clone2)
	{
		return this->BuildViewportTab( NSLOCTEXT("LevelViewportTypes", "LevelEditorViewport_Clone2", "Viewport 3"), TEXT("Viewport 3"), InitializationPayload );
	}
	else if( TabIdentifier == LevelEditorTabIds::LevelEditorViewport_Clone3)
	{
		return this->BuildViewportTab( NSLOCTEXT("LevelViewportTypes", "LevelEditorViewport_Clone3", "Viewport 4"), TEXT("Viewport 4"), InitializationPayload );
	}
	else if( TabIdentifier == TEXT("LevelEditorSelectionDetails") || TabIdentifier == TEXT("LevelEditorSelectionDetails2") || TabIdentifier == TEXT("LevelEditorSelectionDetails3") || TabIdentifier == TEXT("LevelEditorSelectionDetails4") )
	{
		return SummonDetailsPanel( TabIdentifier );
	}
	else if (TabIdentifier == LevelEditorTabIds::PlacementBrowser)
	{
		TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.Label(NSLOCTEXT("LevelEditor", "PlacementBrowserTitle", "Place Actors"))
			.AddMetaData<FTutorialMetaData>(FTutorialMetaData(TEXT("PlacementBrowser"), TEXT("PlacementBrowser")));

			DockTab->SetContent(IPlacementModeModule::Get().CreatePlacementModeBrowser(DockTab));

		return DockTab;
	}
	else if( TabIdentifier == LevelEditorTabIds::LevelEditorBuildAndSubmit )
	{
		TSharedRef<SLevelEditorBuildAndSubmit> NewBuildAndSubmit = SNew( SLevelEditorBuildAndSubmit, SharedThis( this ) );

		TSharedRef<SDockTab> NewTab = SNew( SDockTab )
			.Label( NSLOCTEXT("LevelEditor", "BuildAndSubmitTabTitle", "Build and Submit") )
			[
				NewBuildAndSubmit
			];

		NewBuildAndSubmit->SetDockableTab(NewTab);

		return NewTab;
	}
	else if (TabIdentifier == LevelEditorTabIds::LevelEditorSceneOutliner || TabIdentifier == LevelEditorTabIds::LevelEditorSceneOutliner2 || TabIdentifier == LevelEditorTabIds::LevelEditorSceneOutliner3 || TabIdentifier == LevelEditorTabIds::LevelEditorSceneOutliner4)
	{
		TSharedRef<ISceneOutliner> SceneOutlinerRef = CreateSceneOutliner(TabIdentifier);

		TAttribute<FText> Label = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &SLevelEditor::GetSceneOutlinerLabel, TabIdentifier));
		
		TSharedRef<SDockTab> SceneOutlinerTab = SNew( SDockTab )
			.Label( Label )
			.ToolTip( IDocumentation::Get()->CreateToolTip(NSLOCTEXT("LevelEditor", "OutlinerTabTooltip", "The Outliner lists all actors in the level by name, organized in a tree view."), nullptr, "Shared/LevelEditor", "SceneOutlinerTab" ) )
			.OnExtendContextMenu(this, &SLevelEditor::OnExtendSceneOutlinerTabContextMenu)
			[
				SNew(SBorder)
				.Padding( 0.f )
				.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
				.AddMetaData<FTutorialMetaData>(FTutorialMetaData(TEXT("SceneOutliner"), TEXT("LevelEditorSceneOutliner")))
				[
					SceneOutlinerRef
				]
			];

		// Add it to the map of all the outliner tabs
		SceneOutlinerTabs.Add(TabIdentifier, SceneOutlinerTab);

		return SceneOutlinerTab;
	}
	else if(TabIdentifier == LevelEditorTabIds::LevelEditorLayerBrowser)
	{
		FLayersModule& LayersModule = FModuleManager::LoadModuleChecked<FLayersModule>( "Layers" );
		return SNew( SDockTab )
			.Label( NSLOCTEXT("LevelEditor", "LayersTabTitle", "Layers") )
			[
				SNew(SBorder)
				.Padding( 0.f )
				.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
				.AddMetaData<FTutorialMetaData>(FTutorialMetaData(TEXT("LayerBrowser"), TEXT("LevelEditorLayerBrowser")))
				[
					LayersModule.CreateLayerBrowser()
				]
			];
	}
	else if (TabIdentifier == LevelEditorTabIds::LevelEditorHierarchicalLODOutliner)
	{
		FText Label = NSLOCTEXT("LevelEditor", "HLODOutlinerTabTitle", "Hierarchical LOD Outliner");

		FHierarchicalLODOutlinerModule& HLODModule = FModuleManager::LoadModuleChecked<FHierarchicalLODOutlinerModule>("HierarchicalLODOutliner");
		return SNew(SDockTab)
			.Label(Label)
			.ToolTip(IDocumentation::Get()->CreateToolTip(Label, nullptr, "Shared/Editor/HLOD", "main"))
			[
				HLODModule.CreateHLODOutlinerWidget()
			];
	}
	else if( TabIdentifier == LevelEditorTabIds::WorldBrowserHierarchy)
	{
		FWorldBrowserModule& WorldBrowserModule = FModuleManager::LoadModuleChecked<FWorldBrowserModule>( "WorldBrowser" );
		const TSharedRef<UE::WorldHierarchy::IWorldHierarchy> LevelHierarchyWidget = WorldBrowserModule.CreateWorldBrowserHierarchyWidget();
		WorldHierarchy = LevelHierarchyWidget;
		return SNew( SDockTab )
			.Label( NSLOCTEXT("LevelEditor", "WorldBrowserHierarchyTabTitle", "Levels") )
			[
				LevelHierarchyWidget->GetWidget()
			];
	}
	else if( TabIdentifier == LevelEditorTabIds::WorldBrowserDetails)
	{
		FWorldBrowserModule& WorldBrowserModule = FModuleManager::LoadModuleChecked<FWorldBrowserModule>( "WorldBrowser" );
		return SNew( SDockTab )
			.Label( NSLOCTEXT("LevelEditor", "WorldBrowserDetailsTabTitle", "Level Details") )
			[
				WorldBrowserModule.CreateWorldBrowserDetails()
			];
	}
	else if( TabIdentifier == LevelEditorTabIds::WorldBrowserComposition )
	{
		FWorldBrowserModule& WorldBrowserModule = FModuleManager::LoadModuleChecked<FWorldBrowserModule>( "WorldBrowser" );
		return SNew( SDockTab )
			.Label( NSLOCTEXT("LevelEditor", "WorldBrowserCompositionTabTitle", "World Composition") )
			[
				WorldBrowserModule.CreateWorldBrowserComposition()
			];
	}
	else if (TabIdentifier == LevelEditorTabIds::LevelEditorDataLayerBrowser)
	{
		FDataLayerEditorModule& DataLayerEditorModule = FModuleManager::LoadModuleChecked<FDataLayerEditorModule>( "DataLayerEditor" );
		return SNew(SDockTab)
			.Label(NSLOCTEXT("LevelEditor", "DataLayersTabTitle", "Data Layers"))
			[
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.AddMetaData<FTutorialMetaData>(FTutorialMetaData(TEXT("DataLayerBrowser"), TEXT("LevelEditorDataLayerBrowser")))
				[
					DataLayerEditorModule.CreateDataLayerBrowser()
				]
			];
	}
	else if( TabIdentifier == TEXT("Sequencer") )
	{
		// If the Sequencer tab already exists, draw attention to it and return
		TSharedPtr<FTabManager> LevelEditorTabManager = GetTabManager();
		if (LevelEditorTabManager.IsValid())
		{
			TSharedPtr<SDockTab> Tab = LevelEditorTabManager->FindExistingLiveTab(LevelEditorTabIds::Sequencer);
			if (Tab.IsValid())
			{
				LevelEditorTabManager->DrawAttention(Tab.ToSharedRef());
				return Tab.ToSharedRef();
			}
		}
		
		// Otherwise, open a null sequencer widget
		if (FSlateStyleRegistry::FindSlateStyle("LevelSequenceEditorStyle"))
		{
			// @todo sequencer: remove when world-centric mode is added
			return SNew(SDockTab)
				.Label(NSLOCTEXT("Sequencer", "SequencerMainTitle", "Sequencer"))
				[
					SNullWidget::NullWidget
				];
		}
	}
	else if(TabIdentifier == LevelEditorTabIds::SequencerGraphEditor )
	{
		// @todo sequencer: remove when world-centric mode is added
		return SNew(SDockTab)
			.Label(NSLOCTEXT("Sequencer", "SequencerMainGraphEditorTitle", "Sequencer Curves"))
			[
				SNullWidget::NullWidget
			];
	}
	else if( TabIdentifier == LevelEditorTabIds::LevelEditorStatsViewer )
	{
		FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>( "StatsViewer" );
		return SNew( SDockTab )
			.Label( NSLOCTEXT("LevelEditor", "StatsViewerTabTitle", "Statistics") )
			[
				StatsViewerModule.CreateStatsViewer()
			];
	}
	else if (TabIdentifier == LevelEditorTabIds::WorldSettings)
	{
		FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.NotifyHook = GUnrealEd;
		DetailsViewArgs.ColumnWidth = 0.5f;

		WorldSettingsView = PropPlugin.CreateDetailView( DetailsViewArgs );

		if (GetWorld() != nullptr)
		{
			WorldSettingsView->SetObject(GetWorld()->GetWorldSettings());
		}

		return SNew( SDockTab )
			.Label( NSLOCTEXT("LevelEditor", "WorldSettingsTabTitle", "World Settings") )
			.AddMetaData<FTutorialMetaData>(FTutorialMetaData(TEXT("WorldSettings"), TEXT("WorldSettingsTab")))
			[
				WorldSettingsView.ToSharedRef()
			];
	}
	else if( TabIdentifier == LevelEditorTabIds::LevelEditorEnvironmentLightingViewer)
	{
		FEnvironmentLightingViewerModule& EnvironmentLightingViewerModule = FModuleManager::Get().LoadModuleChecked<FEnvironmentLightingViewerModule>( "EnvironmentLightingViewer" );
		return SNew(SDockTab)
			.Label(NSLOCTEXT("LevelEditor", "EnvironmentLightingViewerTitle", "Env. Light Mixer"))
			[
				EnvironmentLightingViewerModule.CreateEnvironmentLightingViewer()
			];
	}
	
	return SNew(SDockTab);
}


bool SLevelEditor::CanSpawnLevelEditorTab(const FSpawnTabArgs& Args, FName TabIdentifier)
{
	// HLOD Outliner not yet supported with World Partition
	if (TabIdentifier == LevelEditorTabIds::LevelEditorHierarchicalLODOutliner && ensure(World) && World->IsPartitionedWorld())
	{
		return false;
	}

	return true;
}

TSharedPtr<SDockTab> SLevelEditor::TryInvokeTab( FName TabID )
{
	TSharedPtr<FTabManager> LevelEditorTabManager = GetTabManager();
	return LevelEditorTabManager->TryInvokeTab(TabID);
}

void SLevelEditor::SyncDetailsToSelection()
{
	static const FName DetailsTabIdentifiers[] = { 
		LevelEditorTabIds::LevelEditorSelectionDetails, 
		LevelEditorTabIds::LevelEditorSelectionDetails2, 
		LevelEditorTabIds::LevelEditorSelectionDetails3, 
		LevelEditorTabIds::LevelEditorSelectionDetails4 };

	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FName FirstClosedDetailsTabIdentifier;

	// First see if there is an already open details view that can handle the request
	// For instance, if "Details 3" is open, we don't want to open "Details 2" to handle this
	for(const FName& DetailsTabIdentifier : DetailsTabIdentifiers)
	{
		TSharedPtr<IDetailsView> DetailsView = PropPlugin.FindDetailView(DetailsTabIdentifier);

		if(!DetailsView.IsValid())
		{
			// Track the first closed details view in case no currently open ones can handle our request
			if(FirstClosedDetailsTabIdentifier.IsNone())
			{
				FirstClosedDetailsTabIdentifier = DetailsTabIdentifier;
			}
			continue;
		}

		if(DetailsView->IsUpdatable() && !DetailsView->IsLocked())
		{
			TryInvokeTab(DetailsTabIdentifier);
			return;
		}
	}

	// If we got this far then there were no open details views, so open the first available one
	if(!FirstClosedDetailsTabIdentifier.IsNone())
	{
		TryInvokeTab(FirstClosedDetailsTabIdentifier);
	}
}

/** Builds a viewport tab. */
TSharedRef<SDockTab> SLevelEditor::BuildViewportTab( const FText& Label, const FString LayoutId, const FString& InitializationPayload )
{
	// The tab must be created before the viewport layout because the layout needs them
	TSharedRef< SDockTab > DockableTab =
		SNew(SDockTab)
		.Label(Label)
		.OnTabClosed(this, &SLevelEditor::OnViewportTabClosed);
		
	// Create a new tab
	TSharedRef<FLevelViewportTabContent> ViewportTabContent = MakeShareable(new FLevelViewportTabContent());

	// Track the viewport
	CleanupPointerArray(ViewportTabs);
	ViewportTabs.Add(ViewportTabContent);

	auto MakeLevelViewportFunc = [this](const FAssetEditorViewportConstructionArgs& InConstructionArgs)
	{
		return SNew(SLevelViewport, InConstructionArgs)
			.ParentLevelEditor(SharedThis(this));
	};
	ViewportTabContent->Initialize(MakeLevelViewportFunc, DockableTab, LayoutId);

	// Restore transient camera position
	RestoreViewportTabInfo(ViewportTabContent);

	return DockableTab;
}

void SLevelEditor::OnViewportTabClosed(TSharedRef<SDockTab> ClosedTab)
{
	TWeakPtr<FLevelViewportTabContent>* const ClosedTabContent = ViewportTabs.FindByPredicate([&ClosedTab](TWeakPtr<FLevelViewportTabContent>& InPotentialElement) -> bool
	{
		TSharedPtr<FLevelViewportTabContent> ViewportTabContent = InPotentialElement.Pin();
		return ViewportTabContent.IsValid() && ViewportTabContent->BelongsToTab(ClosedTab);
	});

	if(ClosedTabContent)
	{
		TSharedPtr<FLevelViewportTabContent> ClosedTabContentPin = ClosedTabContent->Pin();
		if(ClosedTabContentPin.IsValid())
		{
			SaveViewportTabInfo(ClosedTabContentPin.ToSharedRef());

			// Untrack the viewport
			ViewportTabs.Remove(ClosedTabContentPin);
			CleanupPointerArray(ViewportTabs);
		}
	}
}

void SLevelEditor::OnToolboxTabClosed(TSharedRef<SDockTab> ClosedTab)
{
	GetEditorModeManager().ActivateDefaultMode();
}


void SLevelEditor::SaveViewportTabInfo(TSharedRef<const FLevelViewportTabContent> ViewportTabContent)
{
	const TMap<FName, TSharedPtr<IEditorViewportLayoutEntity>>* const Viewports = ViewportTabContent->GetViewports();
	if(Viewports)
	{
		const FString& LayoutId = ViewportTabContent->GetLayoutString();
		for (auto& Pair : *Viewports)
		{
			TSharedPtr<SLevelViewport> Viewport = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Pair.Value)->AsLevelViewport();

			if( !Viewport.IsValid() )
			{
				continue;
			}

			//@todo there could potentially be more than one of the same viewport type.  This effectively takes the last one of a specific type
			const FLevelEditorViewportClient& LevelViewportClient = Viewport->GetLevelViewportClient();
			const FString Key = FString::Printf(TEXT("%s[%d]"), *LayoutId, static_cast<int32>(LevelViewportClient.ViewportType));
			TransientEditorViews.Add(
				Key, FLevelViewportInfo( 
					LevelViewportClient.GetViewLocation(),
					LevelViewportClient.GetViewRotation(), 
					LevelViewportClient.GetOrthoZoom()
					)
				);
		}
	}
}

void SLevelEditor::RestoreViewportTabInfo(TSharedRef<FLevelViewportTabContent> ViewportTabContent) const
{
	const TMap<FName, TSharedPtr<IEditorViewportLayoutEntity>>* const Viewports = ViewportTabContent->GetViewports();
	if(Viewports)
	{
		const FString& LayoutId = ViewportTabContent->GetLayoutString();
		for (auto& Pair : *Viewports)
		{
			TSharedPtr<SLevelViewport> Viewport = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Pair.Value)->AsLevelViewport();
			if( !Viewport.IsValid() )
			{
				continue;
			}

			FLevelEditorViewportClient& LevelViewportClient = Viewport->GetLevelViewportClient();
			bool bInitializedOrthoViewport = false;
			for (int32 ViewportType = 0; ViewportType < LVT_MAX; ViewportType++)
			{
				if (ViewportType == LVT_Perspective || !bInitializedOrthoViewport)
				{
					const FString Key = FString::Printf(TEXT("%s[%d]"), *LayoutId, ViewportType);
					const FLevelViewportInfo* const TransientEditorView = TransientEditorViews.Find(Key);
					if (TransientEditorView)
					{
						LevelViewportClient.SetInitialViewTransform(
							static_cast<ELevelViewportType>(ViewportType),
							TransientEditorView->CamPosition,
							TransientEditorView->CamRotation,
							TransientEditorView->CamOrthoZoom
							);

						if (ViewportType != LVT_Perspective)
						{
							bInitializedOrthoViewport = true;
						}
					}
				}
			}
		}
	}
}

void SLevelEditor::ResetViewportTabInfo()
{
	TransientEditorViews.Reset();
}

TSharedRef<SWidget> SLevelEditor::RestoreContentArea( const TSharedRef<SDockTab>& OwnerTab, const TSharedRef<SWindow>& OwnerWindow )
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( LevelEditorModuleName );
	LevelEditorModule.SetLevelEditorTabManager(OwnerTab);
	
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

	// Register Level Editor tab spawners
	{
		{
			const FText ViewportTooltip = NSLOCTEXT("LevelEditorTabs", "LevelEditorViewportTooltip", "Open a Viewport tab. Use this to view and edit the current level.");
			const FSlateIcon ViewportIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports");

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorViewport, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorViewport, FString()))
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorViewport", "Viewport 1"))
				.SetTooltipText(ViewportTooltip)
				.SetGroup(MenuStructure.GetLevelEditorViewportsCategory())
				.SetIcon(ViewportIcon)
				.SetCanSidebarTab(false);

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorViewport_Clone1, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorViewport_Clone1, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorViewport_Clone1", "Viewport 2"))
				.SetTooltipText(ViewportTooltip)
				.SetGroup( MenuStructure.GetLevelEditorViewportsCategory() )
				.SetIcon(ViewportIcon);

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorViewport_Clone2, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorViewport_Clone2, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorViewport_Clone2", "Viewport 3"))
				.SetTooltipText(ViewportTooltip)
				.SetGroup( MenuStructure.GetLevelEditorViewportsCategory() )
				.SetIcon(ViewportIcon);

				LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorViewport_Clone3, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorViewport_Clone3, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorViewport_Clone3", "Viewport 4"))
				.SetTooltipText(ViewportTooltip)
				.SetGroup( MenuStructure.GetLevelEditorViewportsCategory() )
				.SetIcon(ViewportIcon);
		}
		{
			const FText DetailsTooltip = NSLOCTEXT("LevelEditorTabs", "LevelEditorSelectionDetailsTooltip", "Open a Details tab. Use this to view and edit properties of the selected object(s).");
			const FSlateIcon DetailsIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorSelectionDetails, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorSelectionDetails, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorSelectionDetails", "Details 1"))
				.SetTooltipText(DetailsTooltip)
				.SetGroup( MenuStructure.GetLevelEditorDetailsCategory() )
				.SetIcon( DetailsIcon );

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorSelectionDetails2, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorSelectionDetails2, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorSelectionDetails2", "Details 2"))
				.SetTooltipText(DetailsTooltip)
				.SetGroup( MenuStructure.GetLevelEditorDetailsCategory() )
				.SetIcon( DetailsIcon );

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorSelectionDetails3, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorSelectionDetails3, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorSelectionDetails3", "Details 3"))
				.SetTooltipText(DetailsTooltip)
				.SetGroup( MenuStructure.GetLevelEditorDetailsCategory() )
				.SetIcon( DetailsIcon );

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorSelectionDetails4, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorSelectionDetails4, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorSelectionDetails4", "Details 4"))
				.SetTooltipText(DetailsTooltip)
				.SetGroup( MenuStructure.GetLevelEditorDetailsCategory() )
				.SetIcon( DetailsIcon );
		}

		{
			const FSlateIcon ToolsIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.PlacementBrowser");
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::PlacementBrowser, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::PlacementBrowser, FString()))
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "PlacementBrowser", "Place Actors"))
				.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "PlacementBrowserTooltipText", "Actor Placement Browser"))
				.SetGroup(MenuStructure.GetLevelEditorCategory())
				.SetIcon(ToolsIcon);
		}

		{
			const FText OutlinerTooltip = NSLOCTEXT("LevelEditorTabs", "LevelEditorSceneOutlinerTooltipText", "Open the Outliner tab, which provides a searchable and filterable list of all actors in the world.");
			const FSlateIcon OutlinerIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner");

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorSceneOutliner, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorSceneOutliner, FString()))
				.SetDisplayName(SceneOutlinerDisplayNames[LevelEditorTabIds::LevelEditorSceneOutliner])
				.SetTooltipText(OutlinerTooltip)
				.SetGroup(MenuStructure.GetLevelEditorOutlinerCategory())
				.SetIcon(OutlinerIcon);

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorSceneOutliner2, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorSceneOutliner2, FString()))
				.SetDisplayName(SceneOutlinerDisplayNames[LevelEditorTabIds::LevelEditorSceneOutliner2])
				.SetTooltipText(OutlinerTooltip)
				.SetGroup(MenuStructure.GetLevelEditorOutlinerCategory())
				.SetIcon(OutlinerIcon);

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorSceneOutliner3, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorSceneOutliner3, FString()))
				.SetDisplayName(SceneOutlinerDisplayNames[LevelEditorTabIds::LevelEditorSceneOutliner3])
				.SetTooltipText(OutlinerTooltip)
				.SetGroup(MenuStructure.GetLevelEditorOutlinerCategory())
				.SetIcon(OutlinerIcon);

			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorSceneOutliner4, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorSceneOutliner4, FString()))
				.SetDisplayName(SceneOutlinerDisplayNames[LevelEditorTabIds::LevelEditorSceneOutliner4])
				.SetTooltipText(OutlinerTooltip)
				.SetGroup(MenuStructure.GetLevelEditorOutlinerCategory())
				.SetIcon(OutlinerIcon);
		}

		{
			const FSlateIcon LayersIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Layers");
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorLayerBrowser, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorLayerBrowser, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorLayerBrowser", "Layers"))
				.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "LevelEditorLayerBrowserTooltipText", "Open the Layers tab. Use this to manage which actors in the world belong to which layers."))
				.SetGroup( MenuStructure.GetLevelEditorCategory() )
				.SetIcon( LayersIcon );
		}

		{
			const FSlateIcon DataLayersIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.DataLayers");
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorDataLayerBrowser, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorDataLayerBrowser, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorDataLayerBrowser", "Data Layers Outliner"))
				.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "LevelEditorDataLayerBrowserTooltipText", "Open the Data Layers Outliner."))
				.SetGroup(MenuStructure.GetLevelEditorWorldPartitionCategory())
				.SetIcon(DataLayersIcon);


		}

		{
			const FSlateIcon HLODIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.HLOD");
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorHierarchicalLODOutliner, 
				FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorHierarchicalLODOutliner, FString()),
				FCanSpawnTab::CreateSP(this, &SLevelEditor::CanSpawnLevelEditorTab, LevelEditorTabIds::LevelEditorHierarchicalLODOutliner))
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorHierarchicalLODOutliner", "Hierarchical LOD Outliner"))
				.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "LevelEditorHierarchicalLODOutlinerTooltipText", "Open the Hierarchical LOD Outliner."))
				.SetGroup(MenuStructure.GetLevelEditorCategory())
				.SetIcon(HLODIcon);
		}
		
		{
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::WorldBrowserHierarchy, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::WorldBrowserHierarchy, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "WorldBrowserHierarchy", "Levels"))
				.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "WorldBrowserHierarchyTooltipText", "Open the Levels tab. Use this to manage the levels in the current project."))
				.SetGroup( WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory() )
				.SetIcon( FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.WorldBrowser") );
			
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::WorldBrowserDetails, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::WorldBrowserDetails, FString()) )
				.SetMenuType( ETabSpawnerMenuType::Hidden )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "WorldBrowserDetails", "Level Details"))
				.SetGroup( WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory() )
				.SetIcon( FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.WorldBrowserDetails") );
		
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::WorldBrowserComposition, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::WorldBrowserComposition, FString()) )
				.SetMenuType( ETabSpawnerMenuType::Hidden )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "WorldBrowserComposition", "World Composition"))
				.SetGroup( WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory() )
				.SetIcon( FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.WorldBrowserComposition") );
		}

		{
			const FSlateIcon StatsViewerIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.StatsViewer");
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorStatsViewer, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorStatsViewer, FString()))
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorStatsViewer", "Statistics"))
				.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "LevelEditorStatsViewerTooltipText", "Open the Statistics tab, in order to see data pertaining to lighting, textures and primitives."))
				.SetGroup(MenuStructure.GetDeveloperToolsAuditCategory())
				.SetIcon(StatsViewerIcon);
		}

		{
			// @todo remove when world-centric mode is added
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::Sequencer, 
				FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::Sequencer, FString()),
				FCanSpawnTab::CreateSP(this, &SLevelEditor::CanSpawnLevelEditorTab, LevelEditorTabIds::Sequencer))
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "Sequencer", "Sequencer"))
				.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "SequencerTooltipText", "Open the Sequencer tab to animate characters, cameras, and properties."))
				.SetGroup( MenuStructure.GetLevelEditorCinematicsCategory() )
				.SetIcon( FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Cinematics") );
		}

		{
			// @todo remove when world-centric mode is added
			const FSlateIcon SequencerGraphIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCurveEditor.TabIcon");
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::SequencerGraphEditor, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::SequencerGraphEditor, FString()))
				.SetMenuType(ETabSpawnerMenuType::Type::Hidden)
				.SetIcon(SequencerGraphIcon);
		}

		{
			const FSlateIcon WorldPropertiesIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.WorldProperties.Tab");
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::WorldSettings, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::WorldSettings, FString()) )
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "WorldSettings", "World Settings"))
				.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "WorldSettingsTooltipText", "Open the World Settings tab, in which global properties of the level can be viewed and edited."))
				.SetGroup( MenuStructure.GetLevelEditorCategory() )
				.SetIcon( WorldPropertiesIcon );
		}

		{
			const FSlateIcon EnvironmentLightingViewerIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.ReflectionOverrideMode");
			LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorEnvironmentLightingViewer, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorEnvironmentLightingViewer, FString()))
				.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "EnvironmentLightingViewer", "Env. Light Mixer"))
				.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "LevelEditorEnvironmentLightingViewerTooltipText", "Open the Environmment Lighting tab to edit all the entities important for world lighting."))
				.SetGroup(MenuStructure.GetLevelEditorCategory())
				.SetIcon(EnvironmentLightingViewerIcon);
		}

		FTabSpawnerEntry& BuildAndSubmitEntry = LevelEditorTabManager->RegisterTabSpawner(LevelEditorTabIds::LevelEditorBuildAndSubmit, FOnSpawnTab::CreateSP(this, &SLevelEditor::SpawnLevelEditorTab, LevelEditorTabIds::LevelEditorBuildAndSubmit, FString()));
		BuildAndSubmitEntry.SetAutoGenerateMenuEntry(false);

		LevelEditorModule.OnRegisterTabs().Broadcast(LevelEditorTabManager);
	}


	// IMPORTANT: If you want to change the default value of "LevelEditor_Layout_v1.8" or "UnrealEd_Layout_v1.4" (even if you only change their version numbers), these are the steps to follow:
	// 1. Check out Engine\Config\Layouts\DefaultLayout.ini in Perforce.
	// 2. Change the code below as you wish and compile the code.
	// 3. (Optional:) Save your current layout so you can load it later.
	// 4. Close the editor.
	// 5. Manually remove Engine\Saved\Config\WindowsEditor\EditorLayout.ini and Users\[UserName]\AppData\Local\UnrealEngine\Editor\EditorLayout.json
	// 6. Open the Editor, which will auto-regenerate a default EditorLayout.ini that uses your new code below.
	// 7. "Window" --> "Save Layout" --> "Save Layout As..."
	//     - Name: Default Editor Layout
	//     - Description: Default layout that the Unreal Editor automatically generates
	// 8. Either click on the toast generated by Unreal that would open the saving path or manually open Engine\Saved\Config\Layouts\ in your explorer
	// 9. Move and rename the new file (Engine\Saved\Config\Layouts\Default_Editor_Layout.ini) into Engine\Config\Layouts\DefaultLayout.ini. You might also have to modify:
	//     9.1. QAGame/Config/DefaultEditorLayout.ini
	//     9.2. Engine/Config/BaseEditorLayout.ini
	//     9.3. Etc
	// 10. Push the new "DefaultLayout.ini" together with your new code.
	// 11. Also update these instructions if you change the version number (e.g., from "UnrealEd_Layout_v1.6" to "UnrealEd_Layout_v1.7").
	const FName LayoutName = TEXT("LevelEditor_Layout_v1.8");
	const TSharedRef<FTabManager::FLayout> DefaultLayout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni,
		FTabManager::NewLayout( LayoutName )
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation( Orient_Horizontal )
			->SetExtensionId( "TopLevelArea" )
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation( Orient_Vertical )
				->SetSizeCoefficient( 1 )
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient( .75f )
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( .15f )
						->SetHideTabWell(true)
						->SetExtensionId("VerticalToolbar")
 					)
					->Split
					(
						FTabManager::NewSplitter()
						->SetSizeCoefficient(.3f)
						->SetOrientation(Orient_Vertical)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient( 0.5f )
							->AddTab(LevelEditorTabIds::PlacementBrowser, ETabState::ClosedTab)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->SetExtensionId("BottomLeftPanel")
						)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetHideTabWell(true)
						->SetSizeCoefficient( 1.0f )
						->AddTab(LevelEditorTabIds::LevelEditorViewport, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(.4f)
					->SetHideTabWell(false)
					->AddTab("ContentBrowserTab1", ETabState::ClosedTab)
					->AddTab(LevelEditorTabIds::Sequencer, ETabState::ClosedTab)
					->AddTab(LevelEditorTabIds::OutputLog, ETabState::ClosedTab)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.25f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.4f)
					->AddTab(LevelEditorTabIds::LevelEditorSceneOutliner, ETabState::OpenedTab)
					->AddTab(LevelEditorTabIds::LevelEditorLayerBrowser, ETabState::ClosedTab)

				)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(LevelEditorTabIds::LevelEditorSelectionDetails, ETabState::OpenedTab)
					->AddTab(LevelEditorTabIds::WorldSettings, ETabState::ClosedTab)
					->SetForegroundTab(LevelEditorTabIds::LevelEditorSelectionDetails)
				)
			)
		));

	FGlobalTabmanager::Get()->SetInitialLayoutSP(DefaultLayout);
	
	const EOutputCanBeNullptr OutputCanBeNullptr = EOutputCanBeNullptr::IfNoTabValid;
	TArray<FString> RemovedOlderLayoutVersions;
	const TSharedRef<FTabManager::FLayout> Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni,
		DefaultLayout, OutputCanBeNullptr, RemovedOlderLayoutVersions);

	// On startup, it should not show the dialog to avoid crashes. However, when clicking the "Load" button, it should also show the message dialog
	// Rather than adding a bool to this function (which would have to be sent from over 10 functions above...), we create a static atomic bool (which is a bit less elegant, but way simpler!), which
	// will have the same exact effect than sending the bool (and it is also thread safe)
	static std::atomic<bool> bIsBeingRecreated(false);
	// If older fields of the layout name (i.e., lower versions than "LevelEditor_Layout_v1.2") were found
	if (RemovedOlderLayoutVersions.Num() > 0)
	{
		// FMessageDialog - Notify the user that the layout version was updated and the current layout uses a deprecated one
		const FText WarningText = FText::Format(LOCTEXT("LevelEditorVersionErrorBody", "The expected Unreal Level Editor layout version is \"{0}\", while only version \"{1}\" was found. I.e., the current layout was created with a previous version of Unreal that is deprecated and no longer compatible.\n\nUnreal will continue with the default layout for its current version, the deprecated one has been removed.\n\nYou can create and save your custom layouts with \"Window\"->\"Save Layout\"->\"Save Layout As...\"."),
			FText::FromString(LayoutName.ToString()), FText::FromString(RemovedOlderLayoutVersions[0]));
		UE_LOGF(LogSlate, Warning, "%ls", *WarningText.ToString());
		// If user is trying to load a specific layout with "Load", also warn them with a message dialog
		if (bIsBeingRecreated)
		{
			const FText TextTitle = LOCTEXT("LevelEditorVersionErrorTitle", "Unreal Level Editor Layout Version Mismatch");
			FMessageDialog::Open(EAppMsgType::Ok, WarningText, TextTitle);
		}
	}
	bIsBeingRecreated = true; // For future loads

	FLayoutExtender LayoutExtender;

	LevelEditorModule.OnRegisterLayoutExtensions().Broadcast(LayoutExtender);
	Layout->ProcessExtensions(LayoutExtender);

	const bool bEmbedTitleAreaContent = false;
	TSharedPtr<SWidget> ContentAreaWidget = LevelEditorTabManager->RestoreFrom(Layout, OwnerWindow, bEmbedTitleAreaContent, OutputCanBeNullptr);
	// ContentAreaWidget will only be nullptr if its main area contains invalid tabs (probably some layout bug). If so, reset layout to avoid potential crashes
	if (!ContentAreaWidget.IsValid())
	{
		// Try to load default layout to avoid nullptr.ToSharedRef() crash
		ContentAreaWidget = LevelEditorTabManager->RestoreFrom(DefaultLayout, OwnerWindow, bEmbedTitleAreaContent, EOutputCanBeNullptr::Never);
		// Warn user/developer
		const FString WarningMessage = FString::Format(TEXT("Level editor layout could not be loaded from the config file {0}, trying to reset this config file to the default one."), { *GEditorLayoutIni });
		UE_LOGF(LogTemp, Warning, "%ls", *WarningMessage);
		ensureMsgf(false, TEXT("Some additional testing of that layout file should be done."));
	}
	check(ContentAreaWidget.IsValid());

	SecondaryModeToolbarWidget =
		SNew(SBorder)
		.Padding(0.f)
		.BorderImage(FAppStyle::Get().GetBrush("NoBorder"));

	SecondaryModeToolbarWidget->SetContent(FLevelEditorToolBar::MakeLevelEditorSecondaryModeToolbar(LevelEditorCommands.ToSharedRef(), ModeUILayers));

	TSharedRef<SWidget> LevelEditorContent = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
		.AutoHeight()
		[
			FLevelEditorToolBar::MakeLevelEditorToolBar(LevelEditorCommands.ToSharedRef(), SharedThis(this))
		]
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
		.AutoHeight()
		[
			SecondaryModeToolbarWidget.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.Padding(4.0f, 2.f, 4.f, 2.f)
		.FillHeight(1.0f)
		[
			ContentAreaWidget.ToSharedRef()
		];

	if (TSharedPtr<SWidget> PanelDrawer = LevelEditorTabManager->RestorePanelDrawer(LevelEditorContent, OwnerWindow))
	{
		LevelEditorContent = PanelDrawer.ToSharedRef();
	}

	return LevelEditorContent;
}

void SLevelEditor::HandleExperimentalSettingChanged(FName PropertyName)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	LevelEditorTabManager->UpdateMainMenu(nullptr, true);
}

FName SLevelEditor::GetEditorModeTabId( FEditorModeID ModeID )
{
	return FName(*(FString("EditorMode.Tab.") + ModeID.ToString()));
}

void SLevelEditor::ToggleEditorMode( FEditorModeID ModeID )
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

	// Abort viewport tracking when switching editor mode
	if (GCurrentLevelEditingViewportClient)
	{
		GCurrentLevelEditingViewportClient->AbortTracking();
	}
		
	// *Important* - activate the mode first since FEditorModeTools::DeactivateMode will
	// activate the default mode when the stack becomes empty, resulting in multiple active visible modes.
	GetEditorModeManager().ActivateMode( ModeID );
}

bool SLevelEditor::IsModeActive( FEditorModeID ModeID )
{
	return GetEditorModeManager().IsModeActive( ModeID );
}

bool SLevelEditor::ShouldShowModeInToolbar(FEditorModeID ModeID)
{
	if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		FEditorModeInfo ModeInfo;
		if (AssetEditorSubsystem->FindEditorModeInfo(ModeID, ModeInfo))
		{
			return ModeInfo.IsVisible();
		}
	}

	return false;
}

void SLevelEditor::EditorModeCommandsChanged()
{
	if (FLevelEditorModesCommands::IsRegistered())
	{
		FLevelEditorModesCommands::Unregister();
	}

	RefreshEditorModeCommands();
}

void SLevelEditor::RefreshEditorModeCommands()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( "LevelEditor" );

	if(!FLevelEditorModesCommands::IsRegistered())
	{
		FLevelEditorModesCommands::Register();
	}

	// We need to remap all the actions to commands.
	const FLevelEditorModesCommands& Commands = FLevelEditorModesCommands::Get();

	int32 CommandIndex = 0;
	if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		for (const FEditorModeInfo& Mode : AssetEditorSubsystem->GetEditorModeInfoOrderedByPriority())
		{
			FName EditorModeTabName = GetEditorModeTabId(Mode.ID);
			FName EditorModeCommandName = FName(*(FString("EditorMode.") + Mode.ID.ToString()));

			TSharedPtr<FUICommandInfo> EditorModeCommand =
				FInputBindingManager::Get().FindCommandInContext(Commands.GetContextName(), EditorModeCommandName);

			// If a command isn't yet registered for this mode, we need to register one.
			if (EditorModeCommand.IsValid() && Mode.IsVisible() && !LevelEditorCommands->IsActionMapped(Commands.EditorModeCommands[CommandIndex]))
			{
				LevelEditorCommands->MapAction(
					Commands.EditorModeCommands[CommandIndex],
					FExecuteAction::CreateSP(SharedThis(this), &SLevelEditor::ToggleEditorMode, Mode.ID),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(SharedThis(this), &SLevelEditor::IsModeActive, Mode.ID),
					FIsActionButtonVisible::CreateSP(SharedThis(this), &SLevelEditor::ShouldShowModeInToolbar, Mode.ID));
				CommandIndex++;
			}
		}
	}
}

FReply SLevelEditor::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	// Check to see if any of the actions for the level editor can be processed by the current event
	// If we are in debug mode do not process commands
	if (FSlateApplication::Get().IsNormalExecution())
	{
		for (const auto& ActiveToolkit : HostedToolkits)
		{
			// A toolkit is active, so direct all command processing to it
			if (ActiveToolkit->ProcessCommandBindings(InKeyEvent))
			{
				return FReply::Handled();
			}
		}

		// No toolkit processed the key, so let the level editor have a chance at the keystroke
		if (LevelEditorCommands->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}
	
	return FReply::Unhandled();
}

FReply SLevelEditor::OnKeyDownInViewport( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	// Check to see if any of the actions for the level editor can be processed by the current keyboard from a viewport
	if( LevelEditorCommands->ProcessCommandBindings( InKeyEvent ) )
	{
		return FReply::Handled();
	}

	// NOTE: Currently, we don't bother allowing toolkits to get a chance at viewport keys

	return FReply::Unhandled();
}

/** Callback for when the level editor layout has changed */
void SLevelEditor::OnLayoutHasChanged()
{
	// ...
}

const UTypedElementSelectionSet* SLevelEditor::GetElementSelectionSet() const
{
	return SelectedElements;
}

UTypedElementSelectionSet* SLevelEditor::GetMutableElementSelectionSet()
{
	return SelectedElements;
}

void SLevelEditor::SummonLevelViewportContextMenu(const FTypedElementHandle& HitProxyElement)
{
	FLevelEditorContextMenu::SummonMenu( SharedThis( this ), ELevelEditorMenuContext::Viewport, HitProxyElement);
}

FText SLevelEditor::GetLevelViewportContextMenuTitle() const
{
	return CachedViewportContextMenuTitle;
}

void SLevelEditor::SummonLevelViewportViewOptionMenu(const ELevelViewportType ViewOption)
{
	FLevelEditorContextMenu::SummonViewOptionMenu( SharedThis( this ), ViewOption);
}

const TArray< TSharedPtr< IToolkit > >& SLevelEditor::GetHostedToolkits() const
{
	return HostedToolkits;
}

TArray< TSharedPtr< SLevelViewport > > SLevelEditor::GetViewports() const
{
	TArray< TSharedPtr<SLevelViewport> > OutViewports;

	for( int32 TabIndex = 0; TabIndex < ViewportTabs.Num(); ++TabIndex )
	{
		TSharedPtr<FLevelViewportTabContent> ViewportTab = ViewportTabs[ TabIndex ].Pin();
		
		if (ViewportTab.IsValid())
		{
			const TMap< FName, TSharedPtr< IEditorViewportLayoutEntity > >* LevelViewports = ViewportTab->GetViewports();

			if (LevelViewports != NULL)
			{
				for (auto& Pair : *LevelViewports)
				{
					TSharedPtr<SLevelViewport> Viewport = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Pair.Value)->AsLevelViewport();
					if( Viewport.IsValid() )
					{
						OutViewports.Add(Viewport);
					}
				}
			}
		}
	}

	// Also add any standalone viewports
	{
		for( const TWeakPtr< SLevelViewport >& StandaloneViewportWeakPtr : StandaloneViewports )
		{
			const TSharedPtr< SLevelViewport > Viewport = StandaloneViewportWeakPtr.Pin();
			if( Viewport.IsValid() )
			{
				OutViewports.Add( Viewport );
			}
		}
	}

	return OutViewports;
}

TMap<int32, TArray<TSharedPtr<SLevelViewport>>> SLevelEditor::GetViewportsByTab() const
{
	constexpr int32 ExternalViewportIndex = 4;

	TMap< int32, TArray< TSharedPtr< SLevelViewport > > > OutViewports;
	OutViewports.Reserve(5); // 4 tabs and 1 standalone

	for (int32 TabIndex = 0; TabIndex < ViewportTabs.Num(); ++TabIndex)
	{
		TArray< TSharedPtr< SLevelViewport > > TabViewportList;
		TabViewportList.Reserve(4);

		TSharedPtr<FLevelViewportTabContent> ViewportTab = ViewportTabs[TabIndex].Pin();

		if (ViewportTab.IsValid())
		{
			const TMap< FName, TSharedPtr< IEditorViewportLayoutEntity > >* LevelViewports = ViewportTab->GetViewports();

			if (LevelViewports != NULL)
			{
				for (auto& Pair : *LevelViewports)
				{
					TSharedPtr<SLevelViewport> Viewport = StaticCastSharedPtr<ILevelViewportLayoutEntity>(Pair.Value)->AsLevelViewport();
					if (Viewport.IsValid())
					{
						TabViewportList.Add(Viewport);
					}
				}
			}
		}

		OutViewports.Add(TabIndex, TabViewportList);
	}

	// Also add any standalone viewports
	{
		TArray< TSharedPtr< SLevelViewport > > StandaloneViewportList;
		StandaloneViewportList.Reserve(StandaloneViewports.Num());

		for (const TWeakPtr< SLevelViewport >& StandaloneViewportWeakPtr : StandaloneViewports)
		{
			const TSharedPtr< SLevelViewport > Viewport = StandaloneViewportWeakPtr.Pin();
			if (Viewport.IsValid())
			{
				StandaloneViewportList.Add(Viewport);
			}
		}

		OutViewports.Add(ExternalViewportIndex, StandaloneViewportList);
	}

	return OutViewports;
}
 
TSharedPtr<SLevelViewport> SLevelEditor::GetActiveViewportInterface()
{
	return GetActiveViewport();
}

void SLevelEditor::AppendCommands( const TSharedRef<FUICommandList>& InCommandsToAppend )
{
	LevelEditorCommands->Append(InCommandsToAppend);
}

UWorld* SLevelEditor::GetWorld() const
{
	return World;
}

void SLevelEditor::HandleEditorMapChange( uint32 MapChangeFlags )
{
	ResetViewportTabInfo();

	if (WorldSettingsView.IsValid())
	{
		WorldSettingsView->SetObject(GetWorld()->GetWorldSettings(), true);
	}

	// Update the editing verse scope on the scope row when a new map is loaded
	if (MapChangeFlags == MapChangeEventFlags::NewMap)
	{
		using namespace UE::Editor::DataStorage;
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (Storage->IsRowAvailable(LevelEditorScopeRow))
			{
				FEditingVerseScope Scope;
				if (UWorld* EditorWorld = GetWorld())
				{
					UE::Core::FVersePath VersePath = EditorWorld->GetVersePath();
					Scope.EditingScopeVersePath = VersePath.ToString();
				}
				Storage->SetScopeData<FEditingVerseScope>(LevelEditorScopeRow, MoveTemp(Scope));
			}
		}
	}

	// We want to recreate all the Scene Outliners on loading a new map
	if (MapChangeFlags == MapChangeEventFlags::NewMap)
	{
		SceneOutliners.Empty();
		
		for (TPair<FName, TWeakPtr<SDockTab>> OutlinerTab : SceneOutlinerTabs)
		{
			if (TSharedPtr<SDockTab> SceneOutlinerTabPin = OutlinerTab.Value.Pin())
			{
				// While this is not strictly necessary since the old tab contents will be cleaned up on SetContent, it is a cleaner flow if we
				// first destruct the old Outliner and then construct the new one
				SceneOutlinerTabPin->SetContent(SNullWidget::NullWidget);
				
				// Create the new outliner
				TSharedRef<ISceneOutliner> SceneOutlinerRef = CreateSceneOutliner(OutlinerTab.Key);

				// Add it to the list of outliners
				SceneOutliners.Add(OutlinerTab.Key, SceneOutlinerRef);

				// Set the tabs content to the newly created outliner
				SceneOutlinerTabPin->SetContent(SceneOutlinerRef);
			}
		}
	}
}

void SLevelEditor::HandleAssetsDeleted(const TArray<UClass*>& DeletedClasses)
{
	bool bDeletedMaterials = false;
	for (UClass* AssetClass : DeletedClasses)
	{
		if (AssetClass == nullptr || AssetClass->IsChildOf<UMaterialInterface>())
		{
			bDeletedMaterials = true;
			break;
		}
	}

	if (bDeletedMaterials)
	{
		// If a material asset has been deleted, it may be being referenced by the BSP model.
		// In case this is the case, invalidate the surface and immediately commit it (rather than waiting until the next tick as is usual),
		// to ensure that it is rebuilt prior to the viewport being redrawn.
		GetWorld()->InvalidateModelSurface(false);
		GetWorld()->CommitModelSurfaces();
	}
}

void SLevelEditor::OnIsmInstanceRemoving(const FSMInstanceElementId& SMInstanceElementId, [[maybe_unused]] int32 InstanceIndex)
{
	SelectedElements->DeselectElement(
		UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(SMInstanceElementId), FTypedElementSelectionOptions{});
}

void SLevelEditor::OnElementSelectionChanged(const UTypedElementSelectionSet* SelectionSet, bool bForceRefresh)
{
	for (TSharedRef<SActorDetails> ActorDetails : GetAllActorDetails())
	{
		if (ActorDetails->IsObservingSelectionSet(SelectionSet))
		{
			ActorDetails->RefreshSelection(bForceRefresh || bNeedsRefresh);
		}
	}

	GetEditorModeManager().ElementSelectionChangeNotify();

	CachedViewportContextMenuTitle = FLevelEditorContextMenu::GetContextMenuTitle(ELevelEditorMenuContext::MainMenu, SelectionSet);

#if PLATFORM_MAC
	// The titles of the level editor's main menus can change when the selection changes (e.g. Action menu).
	// Since Mac uses a global menu bar, we need to force it to sync with the main menu.
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModuleName);
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	LevelEditorTabManager->UpdateMainMenu(nullptr, true);
#endif

	bNeedsRefresh = false;
}

void SLevelEditor::OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	GetEditorModeManager().ActorSelectionChangeNotify();
}

void SLevelEditor::OnOverridePropertyEditorSelection(const TArray<AActor*>& NewSelection, bool bForceRefresh)
{
	for (TSharedRef<SActorDetails> ActorDetails : GetAllActorDetails())
	{
		ActorDetails->OverrideSelection(NewSelection, bForceRefresh || bNeedsRefresh);
	}

	bNeedsRefresh = false;
}

void SLevelEditor::OnLevelActorDeleted(AActor* InActor)
{
	if (SelectedElements && InActor->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
	{
		// Deselect PIE actors when they are deleted, as these won't go through the normal editor flow that will make sure an element is deselected prior to being destroyed
		if (FTypedElementHandle ActorElement = UEngineElementsLibrary::AcquireEditorActorElementHandle(InActor, /*bAllowCreate*/false))
		{
			const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
				.SetAllowHidden(true)
				.SetAllowGroups(false)
				.SetAllowLegacyNotifications(false)
				.SetWarnIfLocked(false)
				.SetChildElementInclusionMethod(ETypedElementChildInclusionMethod::Recursive);

			SelectedElements->DeselectElement(ActorElement, SelectionOptions);
		}
	}
}

void SLevelEditor::OnLevelActorOuterChanged(AActor* InActor, UObject* InOldOuter)
{
	bNeedsRefresh = true;
}

void SLevelEditor::RegisterStatusBarTools()
{
#if WITH_LIVE_CODING
	if (!UToolMenus::Get()->IsMenuRegistered("LevelEditor.StatusBar.ToolBar.CompileComboButton"))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("LevelEditor.StatusBar.ToolBar.CompileComboButton");
		{
			FToolMenuSection& Section = Menu->AddSection("LiveCodingMode", LOCTEXT("LiveCodingMode", "General"));
			Section.AddMenuEntryWithCommandList(FLevelEditorCommands::Get().LiveCoding_Enable, LevelEditorCommands);
		}

		{
			FToolMenuSection& Section = Menu->AddSection("LiveCodingActions", LOCTEXT("LiveCodingActions", "Actions"));
			Section.AddMenuEntryWithCommandList(FLevelEditorCommands::Get().LiveCoding_StartSession, LevelEditorCommands);
			Section.AddMenuEntryWithCommandList(FLevelEditorCommands::Get().LiveCoding_ShowConsole, LevelEditorCommands);
			Section.AddMenuEntryWithCommandList(FLevelEditorCommands::Get().LiveCoding_Settings, LevelEditorCommands);
		}
	}

#endif

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.StatusBar.ToolBar");
	
	FToolMenuSection& CompileSection = Menu->AddSection("Compile", FText::GetEmpty(), FToolMenuInsert("SourceControl", EToolMenuInsertType::Before));

	CompileSection.AddDynamicEntry("CompilerAvailable",
		FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			// Only show the compile options on machines with the solution (assuming they can build it)
			if (FSourceCodeNavigation::IsCompilerAvailable())
			{
				// Since we can always add new code to the project, only hide these buttons if we haven't done so yet
				InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
					"CompileButton",
					FUIAction(
						FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::RecompileGameCode_Clicked),
						FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Recompile_CanExecute),
						FIsActionChecked(),
						FIsActionButtonVisible::CreateStatic(FLevelEditorActionCallbacks::CanShowSourceCodeActions)),
					FText::GetEmpty(),
					FLevelEditorCommands::Get().RecompileGameCode->GetDescription(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Recompile")
				));

#if WITH_LIVE_CODING
				InSection.AddEntry(FToolMenuEntry::InitComboButton(
					"CompileComboButton",
					FUIAction(
						FExecuteAction(),
						FCanExecuteAction(),
						FIsActionChecked(),
						FIsActionButtonVisible::CreateStatic(FLevelEditorActionCallbacks::CanShowSourceCodeActions)),
					FNewToolMenuChoice(),
					LOCTEXT("CompileCombo_Label", "Compile Options"),
					LOCTEXT("CompileComboToolTip", "Compile options menu"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Recompile"),
					true
				));
#endif
			}
		}));

	FZenEditor& ZenEditor = FModuleManager::LoadModuleChecked<FZenEditor>("ZenEditor");

	if (ZenEditor.IsZenEnabled())
	{
		// Only create the Zen Status Bar tool if it is enabled. This should be the default
		FToolMenuSection& ZenSection = Menu->AddSection("Zen", FText::GetEmpty(), FToolMenuInsert("Compile", EToolMenuInsertType::Before));

		ZenSection.AddEntry(
			FToolMenuEntry::InitWidget("ZentatusBar", ZenEditor.CreateStatusBarWidget(), FText::GetEmpty(), true, false)
		);
	}
	else
	{
		// Optionally create the Derived Data Status Bar tool
		FDerivedDataEditor& DerivedDataEditor = FModuleManager::LoadModuleChecked<FDerivedDataEditor>("DerivedDataEditor");

		FToolMenuSection& DerivedDataSection = Menu->AddSection("DDC", FText::GetEmpty(), FToolMenuInsert("Compile", EToolMenuInsertType::Before));

		DerivedDataSection.AddEntry(
			FToolMenuEntry::InitWidget("DerivedDatatatusBar", DerivedDataEditor.CreateStatusBarWidget(), FText::GetEmpty(), true, false)
		);
	}
}

void SLevelEditor::AddStandaloneLevelViewport( const TSharedRef<SLevelViewport>& LevelViewport )
{
	CleanupPointerArray( StandaloneViewports );
	StandaloneViewports.Add( LevelViewport );
}


TSharedRef<SWidget> SLevelEditor::CreateActorDetails( const FName TabIdentifier )
{
	TSharedRef<SActorDetails> ActorDetails = SNew( SActorDetails, GetMutableElementSelectionSet(), TabIdentifier, LevelEditorCommands, GetTabManager(), ActorDetailsSCSEditorUICustomization);

	ActorDetails->SetActorDetailsRootCustomization(ActorDetailsObjectFilter, ActorDetailsRootCustomization);

	AllActorDetailPanels.Add( ActorDetails );
	return ActorDetails;
}

TArray<TSharedRef<SActorDetails>> SLevelEditor::GetAllActorDetails() const
{
	TArray<TSharedRef<SActorDetails>> AllValidActorDetails;
	AllValidActorDetails.Reserve(AllActorDetailPanels.Num());

	for (TWeakPtr<SActorDetails> ActorDetails : AllActorDetailPanels)
	{
		if (TSharedPtr<SActorDetails> ActorDetailsPinned = ActorDetails.Pin())
		{
			AllValidActorDetails.Add(ActorDetailsPinned.ToSharedRef());
		}
	}

	if (AllActorDetailPanels.Num() > AllValidActorDetails.Num())
	{
		TArray<TWeakPtr<SActorDetails>>& AllActorDetailPanelsNonConst = const_cast<TArray<TWeakPtr<SActorDetails>>&>(AllActorDetailPanels);
		AllActorDetailPanelsNonConst.Reset(AllValidActorDetails.Num());
		for (const TSharedRef<SActorDetails>& ValidActorDetails : AllValidActorDetails)
		{
			AllActorDetailPanelsNonConst.Add(ValidActorDetails);
		}
	}

	return AllValidActorDetails;
}

void SLevelEditor::SetActorDetailsRootCustomization(TSharedPtr<FDetailsViewObjectFilter> InActorDetailsObjectFilter, TSharedPtr<IDetailRootObjectCustomization> InActorDetailsRootCustomization)
{
	ActorDetailsObjectFilter = InActorDetailsObjectFilter;
	ActorDetailsRootCustomization = InActorDetailsRootCustomization;

	for (TSharedRef<SActorDetails> ActorDetails : GetAllActorDetails())
	{
		ActorDetails->SetActorDetailsRootCustomization(ActorDetailsObjectFilter, ActorDetailsRootCustomization);
	}
}

void SLevelEditor::AddActorDetailsSCSEditorUICustomization(TSharedPtr<ISCSEditorUICustomization> InActorDetailsSCSEditorUICustomization)
{
	check(ActorDetailsSCSEditorUICustomization);
	ActorDetailsSCSEditorUICustomization->AddCustomization(InActorDetailsSCSEditorUICustomization);
}

void SLevelEditor::RemoveActorDetailsSCSEditorUICustomization(TSharedPtr<ISCSEditorUICustomization> InActorDetailsSCSEditorUICustomization)
{
	if (ActorDetailsSCSEditorUICustomization)
	{
		ActorDetailsSCSEditorUICustomization->RemoveCustomization(InActorDetailsSCSEditorUICustomization);
	}
}

FEditorModeTools& SLevelEditor::GetEditorModeManager() const
{
	return GLevelEditorModeTools();
}

UTypedElementCommonActions* SLevelEditor::GetCommonActions() const
{
	return CommonActions;
}

FName SLevelEditor::GetStatusBarName() const
{
	static const FName LevelEditorStatusBarName = "LevelEditor.StatusBar";
	return LevelEditorStatusBarName;
}

void SLevelEditor::AddViewportOverlayWidget(TSharedRef<SWidget> InWidget, int32 ZOrder, TSharedPtr<IAssetViewport> InViewport)
{
	if (InViewport != nullptr)
	{
		InViewport->AddOverlayWidget(InWidget, ZOrder);
	}
	else if (TSharedPtr<SLevelViewport> ActiveViewport = GetActiveViewport())
	{
		ActiveViewport->AddOverlayWidget(InWidget, ZOrder);
	}
}

void SLevelEditor::RemoveViewportOverlayWidget(TSharedRef<SWidget> InWidget, TSharedPtr<IAssetViewport> InViewport)
{
	if (InViewport != nullptr)
	{
		InViewport->RemoveOverlayWidget(InWidget);
	}
	else if (TSharedPtr<SLevelViewport> ActiveViewport = GetActiveViewport())
	{
		ActiveViewport->RemoveOverlayWidget(InWidget);
	}
}
	
FVector2f SLevelEditor::GetActiveViewportWidgetSize() const
{
	if (TSharedPtr<SLevelViewport> ActiveViewport = GetActiveViewport())
	{
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			ActiveViewport->GetActiveViewport(),
			ActiveViewport->GetViewportClient()->GetScene(),
			ActiveViewport->GetViewportClient()->EngineShowFlags)
			.SetRealtimeUpdate(ActiveViewport->IsRealtime()));
		// SceneView is deleted with the ViewFamily
		FSceneView* SceneView = ActiveViewport->GetViewportClient()->CalcSceneView(&ViewFamily);
		const float InvDpiScale = 1.0f / ActiveViewport->GetViewportClient()->GetDPIScale();
		const float MaxX = static_cast<float>(SceneView->UnscaledViewRect.Width()) * InvDpiScale;
		const float MaxY = static_cast<float>(SceneView->UnscaledViewRect.Height()) * InvDpiScale;
		return FVector2f(MaxX, MaxY);
	}
	return FVector2f::ZeroVector;
}

#undef LOCTEXT_NAMESPACE
