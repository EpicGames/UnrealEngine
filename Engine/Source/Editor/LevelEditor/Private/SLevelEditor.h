// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/World.h"
#include "Widgets/SWindow.h"
#include "SLevelViewport.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Toolkits/IToolkit.h"
#include "AssetThumbnail.h"
#include "ILevelEditor.h"
#include "LevelViewportTabContent.h"
#include "SLevelEditorToolBox.h"
#include "ActorDetailsSCSEditorUICustomization.h"
#include "DataStorage/Handles.h"

class IAssetEditorInstance;
class IDetailsView;
class SActorDetails;
class SBorder;
class SDockTab;
class SLevelEditorModeContent;
class SLevelEditorToolBox;
class UTypedElementSelectionSet;
struct FSceneOutlinerInitializationOptions;
namespace UE::WorldHierarchy { class IWorldHierarchy; }

/**
 * Unreal editor level editor Slate widget
 */
class SLevelEditor
	: public ILevelEditor
{

public:
	SLATE_BEGIN_ARGS( SLevelEditor ){}

	SLATE_END_ARGS()

	/**
	 * Constructor
	 */
	SLevelEditor();

	~SLevelEditor();

	/**
	 * Constructs this widget
	 *
	 * @param InArgs    Declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs );

	/**
	 * Initialize the newly constructed level editor UI, needed because restoring the layout could trigger showing tabs
	 * that immediately try to get a reference to the current level editor.
	 */
	void Initialize( const TSharedRef<SDockTab>& OwnerTab, const TSharedRef<SWindow>& OwnerWindow );

	/**
	 * Gets the currently active viewport in the level editor
	 * @todo Slate: Needs a better implementation 
	 *
	 * @return The active viewport.  If multiple are active it returns the first one               
	 */
	TSharedPtr<class SLevelViewport> GetActiveViewport() const;

	/** ILevelEditor interface */
	virtual const UTypedElementSelectionSet* GetElementSelectionSet() const override;
	virtual UTypedElementSelectionSet* GetMutableElementSelectionSet() override;
	virtual void SummonLevelViewportContextMenu(const FTypedElementHandle& HitProxyElement = FTypedElementHandle()) override;
	virtual FText GetLevelViewportContextMenuTitle() const override;
	virtual void SummonLevelViewportViewOptionMenu(const ELevelViewportType ViewOption) override;
	virtual const TArray< TSharedPtr< class IToolkit > >& GetHostedToolkits() const override;
	virtual TArray< TSharedPtr< SLevelViewport > > GetViewports() const override;
	virtual TMap< int32, TArray< TSharedPtr< SLevelViewport > > > GetViewportsByTab() const override;
	virtual TSharedPtr<SLevelViewport> GetActiveViewportInterface() override;
	virtual void AppendCommands( const TSharedRef<FUICommandList>& InCommandsToAppend ) override;
	virtual void AddStandaloneLevelViewport( const TSharedRef<SLevelViewport>& LevelViewport ) override;

	/**
	 * Given a tab ID, summons a new tab in the position saved in the current layout, or in a default position.
	 * @return the invoked tab
	 */
	TSharedPtr<SDockTab> TryInvokeTab( FName TabID );

	/**
	 * Sync the details panel to the current selection
	 * Will spawn a new details window if required (and possible) due to other details windows being locked
	 */
	void SyncDetailsToSelection();

	/**
	 * @return true if the level editor has a viewport currently being used for pie
	 */
	bool HasActivePlayInEditorViewport() const;

	/** @return	Returns the title to display in the level editor's tab label */
	FText GetTabTitle() const;

	/** @return	Returns the suffix to display in the level editor's after tab label */
	FText GetTabSuffix() const;

	/**
	 * Processes level editor keybindings using events made in a viewport
	 * 
	 * @param MyGeometry		Information about the size and position of the viewport widget
	 * @param InKeyEvent	The event which just occurred	
	 */
	virtual FReply OnKeyDownInViewport( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Returns the full action list for this level editor instance */
	virtual const TSharedPtr< FUICommandList >& GetLevelEditorActions() const override
	{
		return LevelEditorCommands;
	}

	/** IToolKitHost interface */
	virtual TSharedRef< class SWidget > GetParentWidget() override;
	virtual void BringToFront() override;
	virtual TSharedPtr<FTabManager> GetTabManager() const override;
	virtual void OnToolkitHostingStarted( const TSharedRef< class IToolkit >& Toolkit ) override;
	virtual void OnToolkitHostingFinished( const TSharedRef< class IToolkit >& Toolkit ) override;
	virtual UWorld* GetWorld() const override;
	virtual TSharedRef<SWidget> CreateActorDetails( const FName TabIdentifier ) override;
	virtual void SetActorDetailsRootCustomization(TSharedPtr<FDetailsViewObjectFilter> InActorDetailsObjectFilter, TSharedPtr<IDetailRootObjectCustomization> InActorDetailsRootCustomization) override;
	virtual void AddActorDetailsSCSEditorUICustomization(TSharedPtr<ISCSEditorUICustomization> ActorDetailsSCSEditorUICustomization) override;
	virtual void RemoveActorDetailsSCSEditorUICustomization(TSharedPtr<ISCSEditorUICustomization> ActorDetailsSCSEditorUICustomization) override;
	virtual FEditorModeTools& GetEditorModeManager() const override;
	virtual UTypedElementCommonActions* GetCommonActions() const override;
	virtual FName GetStatusBarName() const override;
	virtual FOnActiveViewportChanged& OnActiveViewportChanged() { return OnActiveViewportChangedDelegate; }
	virtual void AddViewportOverlayWidget(TSharedRef<SWidget>, int32 ZOrder = INDEX_NONE, TSharedPtr<IAssetViewport> InViewport = nullptr) override;
	virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget>, TSharedPtr<IAssetViewport> InViewport = nullptr) override; 


	virtual FVector2f GetActiveViewportWidgetSize() const override;
	/* Tick to check the ActiveViewport */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** SWidget overrides */
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}
	
	/** Attaches a sequencer asset editor used to animate objects in the level to this level editor */
	TSharedPtr<SDockTab> AttachSequencer( TSharedPtr<SWidget> SequencerWidget, TSharedPtr<IAssetEditorInstance> NewSequencerAssetEditor );

	/** Gets the default FSceneOutlinerInitializationOptions for this Level Editor's Scene Outliners */
	virtual void GetSceneOutlinerInitializationOptions(const FName TabIdentifier, FSceneOutlinerInitializationOptions& OutInitOptions) override;

	/** Get an array containing weak pointers to all 4 Scene Outliners which could be potentially active */
	virtual TArray<TWeakPtr<ISceneOutliner>> GetAllSceneOutliners() const override;

	/** Set the outliner with the given name as the most recently interacted with */
	virtual void SetMostRecentlyUsedSceneOutliner(FName OutlinerIdentifier) override;
	
	/** Return the most recently interacted with Outliner */
	virtual TSharedPtr<ISceneOutliner> GetMostRecentlyUsedSceneOutliner() override;

	virtual TWeakPtr<UE::WorldHierarchy::IWorldHierarchy> GetWorldHierarchy() override { return WorldHierarchy; }
	
	TSharedRef<SWidget> GetTitleBarMessageWidget() const { return TtileBarMessageBox.ToSharedRef(); }
private:
	
	TSharedRef<SDockTab> SpawnLevelEditorTab(const FSpawnTabArgs& Args, FName TabIdentifier, FString InitializationPayload);
	bool CanSpawnLevelEditorTab(const FSpawnTabArgs& Args, FName TabIdentifier);
	//TSharedRef<SDockTab> SpawnLevelEditorModeTab(const FSpawnTabArgs& Args, FEdMode* EditorMode);
	TSharedRef<SDockTab> SummonDetailsPanel( FName Identifier );

	/**
	 * Binds UI commands to actions for the level editor                   
	 */
	void BindCommands();

	/** Registers menus associated with level editor */
	void RegisterMenus();

	/**
	 * Fills the level editor with content, using the layout string, or the default if
	 * no layout string is passed in
	 */
	TSharedRef<SWidget> RestoreContentArea( const TSharedRef<SDockTab>& OwnerTab, const TSharedRef<SWindow>& OwnerWindow );

	/** Called when a property is changed */
	void HandleExperimentalSettingChanged(FName PropertyName);

	/** Rebuilds the command list for spawning editor modes, this is done when new modes are registered. */
	void RefreshEditorModeCommands();

	/** Editor mode has been added or removed, clears cached command list so it will be rebuilt */
	void EditorModeCommandsChanged();

	/** Gets the tabId mapping to an editor mode */
	static FName GetEditorModeTabId( FEditorModeID ModeID );

	/** Toggles the editor mode on and off, this is what the auto generated editor mode commands are mapped to. */
	void ToggleEditorMode( FEditorModeID ModeID );

	/** Checks if the editor mode is active for the auto-generated editor mode command. */
	bool IsModeActive(FEditorModeID ModeID);

	/** Checks if the editor mode is visible for the auto-generated editor mode command. */
	bool ShouldShowModeInToolbar(FEditorModeID ModeID);

	/**
	 * Processes keybindings on the level editor
	 * 
	 * @param MyGeometry		Information about the size and position of the level editor widget
	 * @param InKeyEvent	The event which just occurred	
	 */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Callback for when the level editor layout has changed */
	void OnLayoutHasChanged();
	
	/** Constructs the title bar message widget */
	void ConstructTitleBarMessages();

	/** Builds a viewport tab. */
	TSharedRef<SDockTab> BuildViewportTab( const FText& Label, const FString LayoutId, const FString& InitializationPayload );

	/** Called when a viewport tab is closed */
	void OnViewportTabClosed(TSharedRef<SDockTab> ClosedTab);

	/** Called when the toolbox tab is closed */
	void OnToolboxTabClosed(TSharedRef<SDockTab> ClosedTab);

	/** Save the information about the given viewport in the transient viewport information */
	void SaveViewportTabInfo(TSharedRef<const class FLevelViewportTabContent> ViewportTabContent);

	/** Restore the information about the given viewport from the transient viewport information */
	void RestoreViewportTabInfo(TSharedRef<FLevelViewportTabContent> ViewportTabContent) const;

	/** Reset the transient viewport information */
	void ResetViewportTabInfo();

	/** Handles Editor map changes */
	void HandleEditorMapChange( uint32 MapChangeFlags );

	/** Handles deletion of assets */
	void HandleAssetsDeleted(const TArray<UClass*>& DeletedClasses);

	/** Handles events where an instance of an Instanced Static Mesh is about to be removed. */
	void OnIsmInstanceRemoving(const FSMInstanceElementId& SMInstanceElementId, int32 InstanceIndex);

	/** Called when element selection changes */
	void OnElementSelectionChanged(const UTypedElementSelectionSet* SelectionSet, bool bForceRefresh = false);

	/** Called when actor selection changes */
	void OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);

	/** Called to set property editors to show the given actors, even if those actors aren't in the current selection set */
	void OnOverridePropertyEditorSelection(const TArray<AActor*>& NewSelection, bool bForceRefresh = false);

	/** Called when an actor is destroyed */
	void OnLevelActorDeleted(AActor* InActor);

	/** Called when an actor changes outer */
	void OnLevelActorOuterChanged(AActor* InActor = nullptr, UObject* InOldOuter = nullptr);

	/** Registers toolbar options for the level editor status bar */
	void RegisterStatusBarTools();

	/** @return All valid actor details panels */
	TArray<TSharedRef<SActorDetails>> GetAllActorDetails() const;

	/** Create a Scene Outliner for the Level Editor */
	TSharedRef<ISceneOutliner> CreateSceneOutliner(FName TabIdentifier);

	/** Function to extend the context menu for the outliner tab */
	void OnExtendSceneOutlinerTabContextMenu(FMenuBuilder& InMenuBuilder);

	/** Helper function to get the display name for an outliner */
	FText GetSceneOutlinerLabel(FName SceneOutlinerTabIdentifier);

	/** Sets the first available outliner as the most recent outliner */
	void ResetMostRecentOutliner();

	void RegisterViewportInteractions(UViewportInteractionsBehaviorSource* Source) const;

private:

	// Tracking the active viewports in this level editor.
	TArray< TWeakPtr<FLevelViewportTabContent> > ViewportTabs;

	// A list of any standalone editor viewports that aren't in tabs
	TArray< TWeakPtr<SLevelViewport> > StandaloneViewports;

	// The last known active viewport
	TWeakPtr<class SLevelViewport> CachedActiveViewport;

	// Border that hosts the document content for the level editor.
	TSharedPtr< SBorder > DocumentsAreaBorder;
	
	// The list of commands with bound delegates for the level editor.
	TSharedPtr<FUICommandList> LevelEditorCommands;

	// Weak reference to all toolbox panels this level editor has spawned.  May contain invalid entries for tabs that were closed.
	TArray< TWeakPtr< class SLevelEditorToolBox > > ToolBoxTabs;

	// List of all of the toolkits we're currently hosting.
	TArray< TSharedPtr< class IToolkit > > HostedToolkits;

	// The UWorld that this level editor is viewing and allowing the user to interact with through.
	UWorld* World;

	// The list of selected elements (also set on the global USelection for actors and components).
	UTypedElementSelectionSet* SelectedElements = nullptr;

	// The common actions implementation for the level editor
	UTypedElementCommonActions* CommonActions = nullptr;

	// The box that holds the title bar messages.
	TSharedPtr<SHorizontalBox> TtileBarMessageBox;

	// Holds the world settings details view.
	TSharedPtr<IDetailsView> WorldSettingsView;

	/** Transient editor viewport states - one for each view type. Key is "LayoutId[ELevelViewportType]", eg) "Viewport 1[0]" */
	TMap<FString, FLevelViewportInfo> TransientEditorViews;

	/** List of all actor details panels to update when selection changes */
	TArray< TWeakPtr<class SActorDetails> > AllActorDetailPanels;

	/** Attached sequencer asset editor */
	TWeakPtr<IAssetEditorInstance> SequencerAssetEditor;

	/** Weak pointer to the level editor's Sequencer widget */
	TWeakPtr<SWidget> SequencerWidgetPtr;

	/** Weak pointer to the level editor's most recently created scene outliner */
	TWeakPtr<ISceneOutliner> SceneOutlinerPtr;

	/** The content displayed by the LevelEditorTabIds::WorldBrowserHierarchy tab. */
	TWeakPtr<UE::WorldHierarchy::IWorldHierarchy> WorldHierarchy;

	/** Map containing Weak pointers to all the Outliners in the level editor */
	TMap<FName, TWeakPtr<ISceneOutliner>> SceneOutliners;

	/** Map containing Weak pointers to all the Scene Outliner Tabs in the level editor */
	TMap<FName, TWeakPtr<SDockTab>> SceneOutlinerTabs;

	/** Map containing the display names of all the outliners */
	TMap<FName, FText> SceneOutlinerDisplayNames;

	/** Handle to the registered OnPreviewFeatureLevelChanged delegate. */
	FDelegateHandle PreviewFeatureLevelChangedHandle;

	/** Handle to the registered OnPreviewPlatformChanged delegate. */
	FDelegateHandle PreviewPlatformChangedHandle;

	/** Handle to the registered OnLevelActorDeleted delegate */
	FDelegateHandle LevelActorDeletedHandle;

	/** Handle to the registered OnLevelActorOuterChanged delegate */
	FDelegateHandle LevelActorOuterChangedHandle;

	/** Actor details object filters */
	TSharedPtr<FDetailsViewObjectFilter> ActorDetailsObjectFilter;

	/** Actor details root customization */
	TSharedPtr<IDetailRootObjectCustomization> ActorDetailsRootCustomization;

	/** Actor details SCS editor customization */
	TSharedPtr<FActorDetailsSCSEditorUICustomization> ActorDetailsSCSEditorUICustomization;
		
	/** A delegate which is called any time the LevelEditor's active viewport changes. */
	FOnActiveViewportChanged OnActiveViewportChangedDelegate;

	/** Context row for the Editor Scope Tree system */
	UE::Editor::DataStorage::RowHandle LevelEditorScopeRow = UE::Editor::DataStorage::InvalidRowHandle;

	/** If this flag is raised we will force refresh on next selection update. */
	std::atomic<bool> bNeedsRefresh;

	TMap<FName, TSharedPtr<FLevelEditorModeUILayer>> ModeUILayers;

	/** Pointer to the widget that houses the level editor's mode toolbar */
	TSharedPtr<SBorder> SecondaryModeToolbarWidget;
	
	/** Viewport context menu title, cached each time the element selection set changes */
	FText CachedViewportContextMenuTitle;
};
