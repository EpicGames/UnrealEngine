// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLevelEditorBuildAndSubmit.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "ISourceControlModule.h"
#include "FileHelpers.h"
#include "LevelEditorActions.h"
#include "EditorBuildUtils.h"
#include "Logging/MessageLog.h"
#include "LightingBuildOptions.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SExpandableArea.h"

#define LOCTEXT_NAMESPACE "SLevelEditorBuildAndSubmit"

static const FName NAME_NameColumn(TEXT("Name"));
static const float SIZE_NameColumn = 180.0f;
static const FName NAME_StateColumn(TEXT("State"));
static const float SIZE_StateColumn = 180.0f;

/** Multi-column item used in the additional package list - represents an FPackageItem */
class SPackageItem : public SMultiColumnTableRow<TSharedPtr<SLevelEditorBuildAndSubmit::FPackageItem>>
{
public:
	SLATE_BEGIN_ARGS( SPackageItem )
		: _Item()
		{}

		SLATE_ARGUMENT( TSharedPtr<SLevelEditorBuildAndSubmit::FPackageItem>, Item )		

	SLATE_END_ARGS()
	
	/**
	 * Generates a widget for task graph events list column
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	virtual TSharedRef< SWidget > GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (NAME_NameColumn == ColumnName)
		{
			// Name column contains a check box and the name.
			// The name of the package will probably get truncated by the narrowness of the column so display the full name in a tooltip too.
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 1.f)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SPackageItem::OnCheckStateChanged)
					.IsChecked(Item->Selected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("BuildAndSubmit.NormalFont"))
						.Text(FText::FromString(Item->Name))
						.ToolTipText(FText::FromString(Item->Name))
					]
				];
		}
		else if (NAME_StateColumn == ColumnName)
		{
			// The state column shows the source control status. It should only ever be one of the three states shown below.
			// Uses a smaller font and uses CAPS to make them stand out next to the name column.
			FText StateText;
			if(Item->SourceControlState->IsCheckedOut())
			{
				StateText = LOCTEXT("SourceControlState_CheckedOut", "CHECKED OUT");
			}
			else if(!Item->SourceControlState->IsSourceControlled())
			{
				StateText = LOCTEXT("SourceControlState_NotInDepot", "NOT IN DEPOT");
			}
			else if(Item->SourceControlState->IsAdded())
			{
				StateText = LOCTEXT("SourceControlState_OpenForAdd", "OPEN FOR ADD");
			}
			else
			{
				StateText = LOCTEXT("SourceControlState_Unknown", "UNKNOWN");
			}

			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.f, 0.f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("BuildAndSubmit.SmallFont"))
					.Text(StateText)
				];
		}
		else
		{
			return SNullWidget::NullWidget;
		}
	}

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		Item = InArgs._Item;

		FSuperRowType::Construct( FSuperRowType::FArguments(), InOwnerTableView );
	}

private:
	/** User clicked the check box on this item - set the item struct's selected flag to match */
	void OnCheckStateChanged( const ECheckBoxState NewCheckedState )
	{
		Item->Selected = (NewCheckedState == ECheckBoxState::Checked);
	}

	/** the item that this row represents */
	TSharedPtr<SLevelEditorBuildAndSubmit::FPackageItem> Item;
};

SLevelEditorBuildAndSubmit::~SLevelEditorBuildAndSubmit()
{
	UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);

	ISourceControlModule::Get().GetProvider().UnregisterSourceControlStateChanged_Handle(OnSourceControlStateChangedDelegateHandle);
}

void SLevelEditorBuildAndSubmit::Construct( const FArguments& InArgs, const TSharedRef< class ILevelEditor >& OwningLevelEditor )
{
	bIsExtraPackagesSectionExpanded = false;
	LevelEditor = OwningLevelEditor;

	TSharedRef< SHeaderRow > PackagesListHeaderRow =
		SNew( SHeaderRow )
		+ SHeaderRow::Column(NAME_NameColumn)
		.FillWidth(SIZE_NameColumn)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 0.0f, 3.0f, 0.0f, 0.0f )
			[
				SNew( STextBlock )
				.Text(LOCTEXT("ColumnHeader_Name", "Name"))
			]
		]
		+ SHeaderRow::Column(NAME_StateColumn)
		.FillWidth(SIZE_StateColumn)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 0.0f, 3.0f, 0.0f, 0.0f )
			[
				SNew( STextBlock )
				.Text(LOCTEXT("ColumnHeader_State", "State"))
			]
		];

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f)
		[
			SNew(SExpandableArea)
			.AreaTitle(LOCTEXT("DescriptionSectionTitle", "Submission Description"))
			.Padding(2.f)
			.BodyContent()
			[
				SAssignNew(DescriptionBox, SEditableTextBox)
				.HintText(LOCTEXT("DescriptionDefaultText", "Enter change description here..."))
			]			
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5)
		[
			SNew(SExpandableArea )
			.AreaTitle(LOCTEXT("ExtraPackagesSectionTitle", "Additional Files to Submit"))
			.Padding(2.f)
			.InitiallyCollapsed(true)
			.MaxHeight(300.f)
			.OnAreaExpansionChanged(this, &SLevelEditorBuildAndSubmit::OnShowHideExtraPackagesSection)
			.BodyContent()
			[
				SNew(SListView<TSharedPtr<FPackageItem>>)
				.ListItemsSource(&PackagesList)
				.OnGenerateRow( this, &SLevelEditorBuildAndSubmit::OnGenerateWidgetForPackagesList )
				.SelectionMode( ESelectionMode::None )
				.HeaderRow(PackagesListHeaderRow)
			]			
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f)
		[
			SNew( SExpandableArea )
			.AreaTitle(LOCTEXT("BuildOptionsSectionTitle", "Build Options"))
			.Padding(2.f)
			.BodyContent()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.f)
				[
					SAssignNew(NoSubmitOnMapErrorBox, SCheckBox)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NoSubmitMapErrorsButtonLabel", "Don't Submit in Event of Map Errors"))
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.f)
				[
					SAssignNew(NoSubmitOnSaveErrorBox, SCheckBox)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NoSubmitSaveErrorsButtonLabel", "Don't Submit in Event of Save Errors"))
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.f)
				[
					SAssignNew(ShowPackagesNotInSCBox, SCheckBox)
					.OnCheckStateChanged(this, &SLevelEditorBuildAndSubmit::OnShowPackagesNotInSCBoxChanged)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ShowPackagesButtonLabel", "Show Files not in Revision Control"))
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.f)
				[
					SAssignNew(AddFilesToSCBox, SCheckBox)
					.IsChecked(ECheckBoxState::Checked)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AddFilesButtonLabel", "Add Files to Revision Control if Necessary"))
					]
				]
			]			
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 10.f, 0.f)
			[
				SNew(SButton)
				.OnClicked(this, &SLevelEditorBuildAndSubmit::OnBuildAndCloseClicked)
				.Text(LOCTEXT("BuildAndCloseButtonLabel", "Build and Close"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SLevelEditorBuildAndSubmit::OnBuildClicked)
				.Text(LOCTEXT("BuildButtonLabel", "Build"))
			]
		]
	];

	UpdatePackagesList();

	OnSourceControlStateChangedDelegateHandle = ISourceControlModule::Get().GetProvider().RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateRaw(this, &SLevelEditorBuildAndSubmit::OnSourceControlStateChanged));

	UPackage::PackageDirtyStateChangedEvent.AddRaw(this, &SLevelEditorBuildAndSubmit::OnEditorPackageModified);
}

void SLevelEditorBuildAndSubmit::OnEditorPackageModified(UPackage* Package)
{
	if (bIsExtraPackagesSectionExpanded && Package->IsDirty())
	{
		UpdatePackagesList();
	}
}

void SLevelEditorBuildAndSubmit::OnSourceControlStateChanged()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SLevelEditorBuildAndSubmit::OnSourceControlStateChanged);

	if (bIsExtraPackagesSectionExpanded)
	{
		UpdatePackagesList();
	}
}

TSharedRef< ITableRow > SLevelEditorBuildAndSubmit::OnGenerateWidgetForPackagesList( TSharedPtr<FPackageItem> InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew( SPackageItem, OwnerTable )
		.Item( InItem );
}

void SLevelEditorBuildAndSubmit::UpdatePackagesList()
{
	int32 NumSelected = 0;
	PackagesList.Empty();

	TMap<FString, FSourceControlStatePtr> PackageStates;
	FEditorFileUtils::FindAllSubmittablePackageFiles( PackageStates, false );
	for (TMap<FString, FSourceControlStatePtr>::TConstIterator PackageIter(PackageStates); PackageIter; ++PackageIter)
	{
		FString PackageName = *PackageIter.Key();
		const FSourceControlStatePtr CurPackageSCCState = PackageIter.Value();

		// Only show files in the depot or we have the flag enabled
		if ( CurPackageSCCState.IsValid() && (CurPackageSCCState->IsSourceControlled() || ShowPackagesNotInSCBox->IsChecked()) )
		{
			UPackage* Package = FindPackage(NULL, *PackageName);
			if (Package != NULL)
			{
				bool bSelected = false;
				if (CurPackageSCCState->IsCheckedOut() || CurPackageSCCState->IsAdded())
				{
					if (Package->IsDirty())
					{
						// Checked out, dirty packages are selected by default
						bSelected = true;
					}
				}

				TSharedPtr<FPackageItem> Item(new FPackageItem());
				Item->Name = PackageName;
				Item->SourceControlState = CurPackageSCCState;
				Item->Selected = bSelected;

				// Put pre-selected items at the start of the list
				if (CurPackageSCCState->IsSourceControlled())
				{
					PackagesList.Insert(Item, NumSelected++);
				}
				else
				{
					PackagesList.Add(Item);
				}
			}
		}
	}
}

FReply SLevelEditorBuildAndSubmit::OnBuildAndCloseClicked()
{
	OnBuildClicked();

	// Close the parent dockable tab
	if (ParentDockTab.IsValid())
	{
		ParentDockTab.Pin()->RemoveTabFromParent();
	}

	return FReply::Handled();
}

FReply SLevelEditorBuildAndSubmit::OnBuildClicked()
{
	FLevelEditorActionCallbacks::ConfigureLightingBuildOptions(FLightingBuildOptions());

	// Configure build settings for the automated build
	FEditorBuildUtils::FEditorAutomatedBuildSettings BuildSettings;

	// Set the settings that are based on user checkboxes in the UI
	BuildSettings.BuildErrorBehavior = NoSubmitOnMapErrorBox->IsChecked() ? FEditorBuildUtils::ABB_FailOnError : FEditorBuildUtils::ABB_ProceedOnError;
	BuildSettings.FailedToSaveBehavior = NoSubmitOnSaveErrorBox->IsChecked() ? FEditorBuildUtils::ABB_FailOnError : FEditorBuildUtils::ABB_ProceedOnError;
	BuildSettings.bCheckInPackages = true;
	BuildSettings.bAutoAddNewFiles = AddFilesToSCBox->IsChecked();

	for (int32 i = 0; i < PackagesList.Num(); i++)
	{
		TSharedPtr<FPackageItem> Item = PackagesList[i];

		if(Item->Selected)
		{
			BuildSettings.PackagesToCheckIn.Add(Item->Name);
		}
	}

	BuildSettings.ChangeDescription = DescriptionBox->GetText().ToString();

	// The editor shouldn't be shutdown while using this special editor window
	BuildSettings.bShutdownEditorOnCompletion = false;

	// Prompt the user on what to do if unsaved maps are detected or if a file can't be checked out for some reason
	BuildSettings.NewMapBehavior = FEditorBuildUtils::ABB_PromptOnError;
	BuildSettings.UnableToCheckoutFilesBehavior = FEditorBuildUtils::ABB_PromptOnError;

	// Attempt the automated build process
	FText ErrorMessage;
	FEditorBuildUtils::EditorAutomatedBuildAndSubmit( BuildSettings, ErrorMessage );

	// If the build failed, display any relevant error message to the user

	// Push the errors to the log
	if ( !ErrorMessage.IsEmpty() )
	{
		FMessageLog BuildAndSubmitErrors("BuildAndSubmitErrors");
		BuildAndSubmitErrors.NewPage(LOCTEXT("BuildAndSubmitErrorsNewPage", "Build and Submit"));

		TArray<FString> Errors;
		ErrorMessage.ToString().Replace(LINE_TERMINATOR, TEXT("\n")).ParseIntoArray(Errors, TEXT("\n"), true);

		for (int32 ErrorIdx = 0; ErrorIdx < Errors.Num(); ErrorIdx++)
		{
			BuildAndSubmitErrors.Error(FText::FromString(Errors[ErrorIdx]));
		}

		BuildAndSubmitErrors.Open();
	}

	return FReply::Handled();
}

void SLevelEditorBuildAndSubmit::OnShowHideExtraPackagesSection(bool bIsExpanded)
{
	bIsExtraPackagesSectionExpanded = bIsExpanded;

	if (bIsExtraPackagesSectionExpanded)
	{
		UpdatePackagesList();
	}
}

void SLevelEditorBuildAndSubmit::OnShowPackagesNotInSCBoxChanged(ECheckBoxState InNewState)
{
	if (bIsExtraPackagesSectionExpanded)
	{
		UpdatePackagesList();
	}
}

#undef LOCTEXT_NAMESPACE
