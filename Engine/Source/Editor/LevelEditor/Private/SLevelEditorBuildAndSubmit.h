// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "ILevelEditor.h"
#include "ISourceControlState.h"

class FPackageItem;
class ITableRow;
class SCheckBox;
class SEditableTextBox;
class STableViewBase;

/**
 * Build and Submit tab for the level editor                   
 */
class SLevelEditorBuildAndSubmit : public SCompoundWidget
{
public:
	/** A check box selectable item in the additional package list */
	class FPackageItem
	{
	public:
		FString Name;
		FSourceControlStatePtr SourceControlState;
		bool Selected;
	};

	SLATE_BEGIN_ARGS( SLevelEditorBuildAndSubmit ){}
	SLATE_END_ARGS()

	virtual ~SLevelEditorBuildAndSubmit();

	/**
	 * Construct a SLevelEditorBuildAndSubmit widget based on initial parameters.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< class ILevelEditor >& OwningLevelEditor );

	/**
	 * Sets the parent tab so that the widget can close its host tab.
	 *
	 * @param ParentDockTab		The parent/host dockable tab of this widget.
	 */
	void SetDockableTab(TSharedRef<SDockTab> InParentDockTab) { ParentDockTab = InParentDockTab; }

	void OnEditorPackageModified(class UPackage* Package);

private:
	/** Called by the package file cache callback to inform this widget of source control state changes */
	void OnSourceControlStateChanged();

	/** Called by the OnGenerateRow event of the additional package list - creates the widget representing each package in the list */
	TSharedRef< ITableRow > OnGenerateWidgetForPackagesList( TSharedPtr<FPackageItem> InItem, const TSharedRef<STableViewBase>& OwnerTable );

	/** Called from various places internally, to rebuild the additional package list contents based on the states in the global package file cache */
	void UpdatePackagesList();

	/** Called when the Build and Clsoe button is clicked. Runs the automated build and submit process based in the options set in the widget THEN closes the widget's tab/window */
	FReply OnBuildAndCloseClicked();

	/** Called when the Build button is clicked. Runs the automated build and submit process based in the options set in the widget. */
	FReply OnBuildClicked();

	/**
	 * Called when the additional package list is shown/hidden by the user clicking the expander arrow around the list
	 *
	 * @param bIsExpanded	The expansion state of the package list area (true => visible)
	 */
	void OnShowHideExtraPackagesSection(bool bIsExpanded);

	/**
	 * Called when the check box to show/hide packages not in source control is clicked.
	 * Changes the packages shown in the additional package list.
	 *
	 * @param InNewState	The state of the check box
	 */
	void OnShowPackagesNotInSCBoxChanged(ECheckBoxState InNewState);

	/** Level editor that we're associated with */
	TWeakPtr<class ILevelEditor> LevelEditor;

	/** SDockTab in the LevelEditor that we're associated with */
	TWeakPtr<SDockTab> ParentDockTab;

	/** The package list that acts as the items source for the additional packages list widget */
	TArray<TSharedPtr<FPackageItem>> PackagesList;

	/** The editable text box containing the submission description */
	TSharedPtr<SEditableTextBox> DescriptionBox;

	/** Options check box - Stops the auto-build and submit process submitting files if any map errors occur */
	TSharedPtr<SCheckBox> NoSubmitOnMapErrorBox;

	/** Options check box - Stops the auto-build and submit process submitting files if any save errors occur */
	TSharedPtr<SCheckBox> NoSubmitOnSaveErrorBox;

	/** Options check box - Controls whether packages not in source control are shown in the additional packages list */
	TSharedPtr<SCheckBox> ShowPackagesNotInSCBox;

	/** Options check box - Controls whether new map packages are auto-added to source control during the auto-build and submit process */
	TSharedPtr<SCheckBox> AddFilesToSCBox;

	/** Set by OnShowHideExtraPackagesSection() - keeps track of the visibility or the additional packages list */
	bool bIsExtraPackagesSectionExpanded;

	/** Handle to the registered OnSourceControlStateChanged delegate */
	FDelegateHandle OnSourceControlStateChangedDelegateHandle;
};
