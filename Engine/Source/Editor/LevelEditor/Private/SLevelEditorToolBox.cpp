// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLevelEditorToolBox.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "LevelEditorModesActions.h"
#include "EdMode.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "StatusBarSubsystem.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Misc/Attribute.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SLevelEditorToolBox)

#define LOCTEXT_NAMESPACE "SLevelEditorToolBox"

void ULevelEditorUISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnRegisterLayoutExtensions().AddUObject(this, &ULevelEditorUISubsystem::RegisterLayoutExtensions);
}

void ULevelEditorUISubsystem::Deinitialize()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnRegisterLayoutExtensions().RemoveAll(this);
}

void ULevelEditorUISubsystem::RegisterLayoutExtensions(FLayoutExtender& Extender)
{
	Extender.ExtendLayout(LevelEditorTabIds::PlacementBrowser, ELayoutExtensionPosition::Before, FTabManager::FTab(UAssetEditorUISubsystem::TopLeftTabID, ETabState::ClosedTab));
	Extender.ExtendStack("BottomLeftPanel", ELayoutExtensionPosition::Before, FTabManager::FTab(UAssetEditorUISubsystem::BottomLeftTabID, ETabState::ClosedTab));
	Extender.ExtendStack("VerticalToolbar", ELayoutExtensionPosition::Before, FTabManager::FTab(UAssetEditorUISubsystem::VerticalToolbarID, ETabState::ClosedTab));
	Extender.ExtendLayout(LevelEditorTabIds::LevelEditorSceneOutliner, ELayoutExtensionPosition::Before, FTabManager::FTab(UAssetEditorUISubsystem::TopRightTabID, ETabState::ClosedTab));
	Extender.ExtendLayout(LevelEditorTabIds::LevelEditorSelectionDetails, ELayoutExtensionPosition::Before, FTabManager::FTab(UAssetEditorUISubsystem::BottomRightTabID, ETabState::ClosedTab));
}


FLevelEditorModeUILayer::FLevelEditorModeUILayer(const IToolkitHost* InToolkitHost)
	: FAssetEditorModeUILayer(InToolkitHost)
{
	
}

void FLevelEditorModeUILayer::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	if (!Toolkit->IsAssetEditor())
	{
		FAssetEditorModeUILayer::OnToolkitHostingStarted(Toolkit);
		HostedToolkit = Toolkit;
		Toolkit->SetModeUILayer(SharedThis(this));
		Toolkit->RegisterTabSpawners(ToolkitHost->GetTabManager().ToSharedRef());
		RegisterModeTabSpawners();
		OnToolkitHostReadyForUI.ExecuteIfBound();

		// Set up an owner for the current scope so that we can cleanly clean up the toolbar extension on hosting finish
		FToolMenuOwnerScoped Owner(Toolkit->GetToolkitFName());

		UToolMenu* SecondaryModeToolbar = UToolMenus::Get()->ExtendMenu(GetSecondaryModeToolbarName());

		OnRegisterSecondaryModeToolbarExtension.ExecuteIfBound(SecondaryModeToolbar);
	}

}

void FLevelEditorModeUILayer::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	if (HostedToolkit.IsValid() && HostedToolkit.Pin() == Toolkit)
	{
		FAssetEditorModeUILayer::OnToolkitHostingFinished(Toolkit);

		UToolMenus::Get()->UnregisterOwnerByName(Toolkit->GetToolkitFName());
	}
}








TSharedPtr<FWorkspaceItem> FLevelEditorModeUILayer::GetModeMenuCategory() const
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
	return MenuStructure.GetLevelEditorModesCategory();
}


#undef LOCTEXT_NAMESPACE

