// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "Textures/SlateIcon.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Editor.h"
#include "ILevelEditor.h"
#include "Misc/NotifyHook.h"
#include "StatusBarSubsystem.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "SLevelEditorToolBox.generated.h"

class FExtender;
class SBorder;
class IToolkit;
class SDockTab;
class ILevelEditor;

UCLASS()
class ULevelEditorUISubsystem : public UAssetEditorUISubsystem
{
	GENERATED_BODY()

public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) override;
};


class FLevelEditorModeUILayer : public FAssetEditorModeUILayer
{
public:
	FLevelEditorModeUILayer(const IToolkitHost* InToolkitHost);
	FLevelEditorModeUILayer() {};
	virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;

	virtual TSharedPtr<FWorkspaceItem> GetModeMenuCategory() const override;
};
