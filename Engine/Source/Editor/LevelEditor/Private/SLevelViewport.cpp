// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLevelViewport.h"
#include "Materials/MaterialInterface.h"
#include "Engine/Selection.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Subsystems/EditorAssetSubsystem.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Docking/TabManager.h"
#include "Engine/GameViewportClient.h"
#include "EngineGlobals.h"
#include "ActorFactories/ActorFactory.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "GameFramework/PlayerController.h"
#include "Application/ThrottleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Framework/Docking/LayoutService.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor/Transactor.h"
#include "Exporters/ExportTextContainer.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "GameFramework/WorldSettings.h"
#include "LevelEditorViewport.h"
#include "UnrealEdMisc.h"
#include "UnrealEdGlobals.h"
#include "LevelEditor.h"
#include "LevelViewportActions.h"
#include "LevelEditorActions.h"
#include "SceneView.h"
#include "Slate/SceneViewport.h"
#include "EditorShowFlags.h"
#include "SLevelEditor.h"
#include "AssetSelection.h"
#include "Kismet2/DebuggerCommands.h"
#include "Layers/LayersSubsystem.h"
#include "DragAndDrop/ClassDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/ExportTextDragDropOp.h"
#include "LevelUtils.h"
#include "DragAndDrop/BrushBuilderDragDropOp.h"
#include "ISceneOutlinerColumn.h"
#include "ActorTreeItem.h"
#include "ScopedTransaction.h"
#include "SCaptureRegionWidget.h"
#include "HighresScreenshotUI.h"
#include "ISettingsModule.h"
#include "BufferVisualizationData.h"
#include "NaniteVisualizationData.h"
#include "LumenVisualizationData.h"
#include "SubstrateVisualizationData.h"
#include "GroomVisualizationData.h"
#include "VirtualShadowMapVisualizationData.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SActorPilotViewportToolbar.h"
#include "Engine/LocalPlayer.h"
#include "Slate/SGameLayerManager.h"
#include "Slate/DebugCanvas.h"
#include "FoliageType.h"
#include "IVREditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "BufferVisualizationMenuCommands.h"
#include "NaniteVisualizationMenuCommands.h"
#include "RayTracingDebugVisualizationMenuCommands.h"
#include "LumenVisualizationMenuCommands.h"
#include "MegaLightsVisualizationMenuCommands.h"
#include "SubstrateVisualizationMenuCommands.h"
#include "VirtualShadowMapVisualizationMenuCommands.h"
#include "VirtualTextureVisualizationMenuCommands.h"
#include "EditorLevelUtils.h"
#include "Engine/LevelStreaming.h"
#include "WorldBrowserModule.h"
#include "Bookmarks/IBookmarkTypeTools.h"
#include "ToolMenus.h"
#include "Bookmarks/IBookmarkTypeTools.h"
#include "Editor/EditorPerformanceSettings.h"
#include "UnrealWidget.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "SInViewportDetails.h"
#include "Viewports/InViewportUIDragOperation.h"
#include "SActorEditorContext.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "SWorldPartitionViewportWidget.h"
#include "LevelViewportLayout.h"
#include "EditorViewportTabContent.h"
#include "EditorViewportCommands.h"
#include "Viewports/SDepthBar.h"
#include "FunctionalUIScreenshotTest.h"
#include "GPUSkinCache.h"
#include "GPUSkinCacheVisualizationMenuCommands.h"
#include "SActionableMessageViewportWidget.h"
#include "ShowFlagMenuCommands.h"
#include "SkeletalRenderPublic.h"
#include "ViewportToolBarContext.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "ViewportToolbar/LevelEditorSubmenus.h"
#include "ViewportToolbar/LevelViewportContext.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

static const FName LevelEditorName("LevelEditor");
static FAutoConsoleCommand EnableInViewportMenu(TEXT("Editor.EnableInViewportMenu"), TEXT("Enables the new in-viewport property menu"), FConsoleCommandDelegate::CreateStatic(&SLevelViewport::EnableInViewportMenu));
bool SLevelViewport::bInViewportMenuEnabled = false;

#define LOCTEXT_NAMESPACE "LevelViewport"

namespace SLevelViewportPIEAnimation
{
	float const MouseControlLabelFadeout = 5.0f;
}

namespace UE::SLevelViewport::Internal
{
	bool SaveViewportInfo(UWorld* World, FLevelEditorViewportClient* LevelEditorViewportClient, ULevelEditorViewportSettings* LevelEditorViewportSettings)
	{
		if (!World || !LevelEditorViewportClient || !LevelEditorViewportSettings)
		{
			return false;
		}

		// there could potentially be more than one of the same viewport type.  This effectively takes the last one of a specific type
		World->EditorViews[LevelEditorViewportClient->ViewportType] =
			FLevelViewportInfo(
				LevelEditorViewportClient->GetViewLocation(),
				LevelEditorViewportClient->GetViewRotation(),
				LevelEditorViewportClient->GetOrthoZoom());

		LevelEditorViewportSettings->EditorViews.FindOrAdd(World).LevelViewportsInfo = World->EditorViews;
		LevelEditorViewportSettings->SaveConfig();

		return true;
	}

	bool LoadViewportInfo(UWorld* World, FLevelEditorViewportClient* LevelEditorViewportClient, ULevelEditorViewportSettings* LevelEditorViewportSettings)
	{
		if (!World || !LevelEditorViewportClient || !LevelEditorViewportSettings)
		{
			return false;
		}

		if (FLevelEditorViewporEditorViews* PerUserEditorViews = LevelEditorViewportSettings->EditorViews.Find(World))
		{
			World->EditorViews = PerUserEditorViews->LevelViewportsInfo;
		}

		LevelEditorViewportClient->ResetCamera();

		bool bInitializedOrthoViewport = false;
		for (int32 ViewportType = 0; ViewportType < LVT_MAX; ViewportType++)
		{
			float& CamOrthoZoom = World->EditorViews[ViewportType].CamOrthoZoom;
			if (CamOrthoZoom < MIN_ORTHOZOOM || CamOrthoZoom > MAX_ORTHOZOOM)
			{
				CamOrthoZoom = DEFAULT_ORTHOZOOM;
			}

			if (ViewportType == LVT_Perspective || !bInitializedOrthoViewport)
			{
				LevelEditorViewportClient->SetInitialViewTransform(
					static_cast<ELevelViewportType>(ViewportType),
					World->EditorViews[ViewportType].CamPosition,
					World->EditorViews[ViewportType].CamRotation,
					World->EditorViews[ViewportType].CamOrthoZoom);

				if (ViewportType != LVT_Perspective)
				{
					bInitializedOrthoViewport = true;
				}
			}
		}

		return true;
	}

	// Clears existing selection, then selects the specified actor 
	void SelectActor(AActor* InActor)
	{
		check(InActor);

		const FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(LevelEditorName);
		FEditorModeTools& EditorModeManager = LevelEditorModule.GetFirstLevelEditor()->GetEditorModeManager();
		
		// Deselect any currently selected actors
		EditorModeManager.SelectNone();
		EditorModeManager.GetSelectedActors()->DeselectAll();
		EditorModeManager.GetSelectedObjects()->DeselectAll();
		EditorModeManager.GetSelectedActors()->Select(InActor, true);

		EditorModeManager.ActorSelectionChangeNotify();
	}
}

class FLevelViewportDropContextMenuImpl
{
public:
	/**
	 * Fills in menu options for the actor add/replacement submenu
	 *
	 * @param bReplace		true if we want to add a replace menu instead of add
	 * @param MenuBuilder	The menu to add items to
	 */
	static void FillDropAddReplaceActorMenu( bool bReplace, class FMenuBuilder& MenuBuilder );
};


SLevelViewport::SLevelViewport()
	: HighResScreenshotDialog( nullptr )
	, ViewTransitionType( EViewTransition::None )
	, bViewTransitionAnimPending( false )
	, bPIEHasFocus(false)
	, bPIEContainsFocus(false)
	, UserAllowThrottlingValue(0)
{
}

SLevelViewport::~SLevelViewport()
{
	// Clean up any actor preview viewports
	for (FViewportActorPreview& ActorPreview : ActorPreviews)
	{
		ActorPreview.bIsPinned = false;
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bPreviewInDesktopViewport = !IVREditorModule::Get().IsVREditorModeActive();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	PreviewActors( TArray< AActor* >(), bPreviewInDesktopViewport);

	FLevelViewportCommands::NewStatCommandDelegate.RemoveAll(this);

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>( LevelEditorName );
	LevelEditor.OnRedrawLevelEditingViewports().RemoveAll( this );
	LevelEditor.OnActorSelectionChanged().RemoveAll( this );
	LevelEditor.OnElementSelectionChanged().RemoveAll( this );
	LevelEditor.OnMapChanged().RemoveAll( this );

	if(UObjectInitialized())
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
		GEngine->OnEditorClose().RemoveAll( this );

		GetMutableDefault<ULevelEditorViewportSettings>()->OnSettingChanged().RemoveAll(this);
	}

	// If this viewport has a high res screenshot window attached to it, close it
	if (HighResScreenshotDialog.IsValid())
	{
		HighResScreenshotDialog.Pin()->RequestDestroyWindow();
		HighResScreenshotDialog.Reset();
	}
}

void SLevelViewport::HandleViewportSettingChanged(FName PropertyName)
{
	if ( PropertyName == TEXT("bPreviewSelectedCameras") )
	{
		OnPreviewSelectedCamerasChange();
	}
}

bool SLevelViewport::IsVisible() const
{
	// The viewport is visible if we don't have a parent layout (likely a floating window) or this viewport is visible in the parent layout
	return IsInForegroundTab() && SEditorViewport::IsVisible();
}

bool SLevelViewport::IsInForegroundTab() const
{
	if (ViewportWidget.IsValid() && ParentLayout.IsValid() && !ConfigKey.IsNone())
	{
		return ParentLayout.Pin()->IsLevelViewportVisible(ConfigKey);
	}
	return false;
}

void SLevelViewport::Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InConstructionArguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SLevelViewport::Construct);

	GetMutableDefault<ULevelEditorViewportSettings>()->OnSettingChanged().AddRaw(this, &SLevelViewport::HandleViewportSettingChanged);

	ParentLayout = StaticCastSharedPtr<FLevelViewportLayout>(InConstructionArguments.ParentLayout);
	ParentLevelEditor = StaticCastSharedRef<SLevelEditor>( InArgs._ParentLevelEditor.Pin().ToSharedRef() );
	ConfigKey = InConstructionArguments.ConfigKey;
	LevelViewportClient = InArgs._LevelEditorViewportClient;

	DebuggingBorder = FAppStyle::Get().GetBrush( "LevelViewport.DebugBorder" );
	BlackBackground = FAppStyle::Get().GetBrush( "LevelViewport.BlackBackground" );
	StartingPlayInEditorBorder = FAppStyle::Get().GetBrush( "LevelViewport.StartingPlayInEditorBorder" );
	StartingSimulateBorder = FAppStyle::Get().GetBrush( "LevelViewport.StartingSimulateBorder" );
	ReturningToEditorBorder = FAppStyle::Get().GetBrush( "LevelViewport.ReturningToEditorBorder" );
	NonMaximizedBorder = FAppStyle::Get().GetBrush("LevelViewport.NonMaximizedBorder");

	// Default level viewport client values for settings that could appear in layout config ini
	FLevelEditorViewportInstanceSettings ViewportInstanceSettings;
	ViewportInstanceSettings.ViewportType = InConstructionArguments.ViewportType;
	ViewportInstanceSettings.PerspViewModeIndex = VMI_Lit;
	ViewportInstanceSettings.OrthoViewModeIndex = VMI_Unlit;
	ViewportInstanceSettings.bIsRealtime = InConstructionArguments.bRealtime;
	ConstructLevelEditorViewportClient(ViewportInstanceSettings);

	SEditorViewport::Construct(SEditorViewport::FArguments()
		.ViewportSize(MakeAttributeSP(this, &SLevelViewport::GetSViewportSize))
		);
	TSharedRef<SWidget> EditorViewportWidget = ChildSlot.GetChildAt(0);
	ChildSlot
	[
		SNew(SScaleBox)
		.Stretch(this, &SLevelViewport::OnGetScaleBoxStretch)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.StretchDirection(EStretchDirection::Both)
		[
			EditorViewportWidget
		]
	];

	ActiveViewport = SceneViewport;

	ConstructViewportOverlayContent();


	// If a map has already been loaded, this will test for it and copy the correct camera location out
	OnMapChanged( GWorld, EMapChangeType::LoadMap );

	// Important: We use raw bindings here because we are releasing our binding in our destructor (where a weak pointer would be invalid)
	// It's imperative that our delegate is removed in the destructor for the level editor module to play nicely with reloading.

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>( LevelEditorName );
	LevelEditor.OnRedrawLevelEditingViewports().AddRaw( this, &SLevelViewport::RedrawViewport );

	// Tell the level editor we want to be notified when selection changes
	LevelEditor.OnActorSelectionChanged().AddRaw( this, &SLevelViewport::OnActorSelectionChanged );

	// Tell the level editor we want to be notified when selection changes
	LevelEditor.OnElementSelectionChanged().AddRaw( this, &SLevelViewport::OnElementSelectionChanged );

	// Tell the level editor we want to be notified when selection changes
	LevelEditor.OnMapChanged().AddRaw( this, &SLevelViewport::OnMapChanged );

	GEngine->OnLevelActorDeleted().AddRaw( this, &SLevelViewport::OnLevelActorsRemoved );
	GEngine->OnEditorClose().AddRaw( this, &SLevelViewport::OnEditorClose );

	FEditorDelegates::PostPIEStarted.AddSP(this, &SLevelViewport::TransitionToPIE);
	FEditorDelegates::PrePIEEnded.AddSP(this, &SLevelViewport::TransitionFromPIE);

	bIsInViewportMenuShowing = false;
	bIsInViewportMenuInitialized = false;
}

void SLevelViewport::ConstructViewportOverlayContent()
{
	int32 SlotIndex = 0;

	// Editor debug canvas (swapped for SGameLayerManager during PIE).
	SAssignNew(EditorDebugCanvas, SDebugCanvas).SceneViewport(ActiveViewport.Get());
	ViewportOverlay->AddSlot(SlotIndex++)
	[
		EditorDebugCanvas.ToSharedRef()
	];

	// Add widget to show viewport warnings from the ActionableMessageSubsystem.
	{
		// clang-format off
		ViewportOverlay->AddSlot(SlotIndex)
		.Padding(FMargin(0.f, 8.f, 10.f, 0.f))
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ActionableMessage.Border"))
			[
				SAssignNew(ActionableMessageViewportWidget, SActionableMessageViewportWidget)
				.Visibility_Lambda(
					[this]()
					{
						if (GetToolBarVisibility() != EVisibility::Visible)
						{
							return EVisibility::Collapsed;
						}

						if (!IsActiveLevelViewport())
						{
							return EVisibility::Collapsed;
						}

						return ActionableMessageViewportWidget->GetVisibility();
					}
				)
			]
		];
		// clang-format on
	}

	ViewportOverlay->AddSlot( SlotIndex )
	.HAlign(HAlign_Right)
	.Padding(0.0f, 0.0f, 0.0f, 35.0f)
	[
		SAssignNew( ActorPreviewHorizontalBox, SHorizontalBox )
	];

	auto GetCombinedVisibility = [this]()
	{
		if (GetCurrentScreenPercentageVisibility() == EVisibility::Collapsed &&
			GetCurrentShaderPlatformPreviewTextVisibility() == EVisibility::Collapsed &&
			GetSelectedActorsCurrentLevelTextVisibility() == EVisibility::Collapsed &&
			!IsActorEditorContextVisible())
		{
			return EVisibility::Collapsed;
		}

		return EVisibility::Visible;
	};

	ViewportOverlay->AddSlot( SlotIndex )
	.VAlign( VAlign_Bottom )
	.HAlign( HAlign_Right )
	.Padding( 5.0f )
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("FloatingBorder"))
			[
				SNew(SVerticalBox)
				.Visibility_Lambda(GetCombinedVisibility)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f, 1.0f, 2.0f, 1.0f)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &SLevelViewport::GetCurrentScreenPercentageVisibility)
					// Current screen percentage label
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 1.0f, 2.0f, 1.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ScreenPercentageLabel", "Screen Percentage"))
						.ShadowOffset(FVector2D(1, 1))
					]

					// Current screen percentage
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.0f, 1.0f, 2.0f, 1.0f)
					[
						SNew(STextBlock)
						.Text(this, &SLevelViewport::GetCurrentScreenPercentageText)
						.ShadowOffset(FVector2D(1, 1))
					]
				]
				// add feature level widget
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f, 1.0f, 2.0f, 1.0f)
				[
					BuildShaderPlatformWidget()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f, 1.0f, 2.0f, 1.0f)
				[
					SNew(SVerticalBox)
					.Visibility(this, &SLevelViewport::GetSelectedActorsCurrentLevelTextVisibility)
					// Current level label
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f, 1.0f, 2.0f, 1.0f)
					[
						SNew(STextBlock)
						.Text(this, &SLevelViewport::GetSelectedActorsCurrentLevelText, true)
						.ShadowOffset(FVector2D(1, 1))
					]
					// Current level
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0f, 1.0f, 2.0f, 1.0f)
					[
						SNew(STextBlock)
						.Text(this, &SLevelViewport::GetSelectedActorsCurrentLevelText, false)
						.ShadowOffset(FVector2D(1, 1))
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SActorEditorContext)
					.World(GetWorld())
					.Visibility_Lambda([this]() { return IsActorEditorContextVisible() ? OnGetViewportContentVisibility() : EVisibility::Collapsed; })
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("FloatingBorder"))
			[
				SAssignNew(WorldPartitionViewportWidget, SWorldPartitionViewportWidget)
				.Clickable(true)
				.Visibility_Lambda([this]() 
				{ 
					return (!ActiveViewport.IsValid() || IsPlayInEditorViewportActive()) ? EVisibility::Collapsed : WorldPartitionViewportWidget->GetVisibility(GetWorld());
				})
			]
		]
	];

	// Add highres screenshot region capture editing widget
	ViewportOverlay->AddSlot(SlotIndex)
	.VAlign( VAlign_Fill )
	.HAlign( HAlign_Fill )
	.Padding( 0.f )
	[
		SAssignNew(CaptureRegionWidget, SCaptureRegionWidget)
	];
}

bool SLevelViewport::IsActorEditorContextVisible() const
{
	return GetDefault<ULevelEditorViewportSettings>()->bShowActorEditorContext &&
		GetWorld() &&
		GetWorld()->GetCurrentLevel() &&
		(&GetLevelViewportClient() == GCurrentLevelEditingViewportClient) &&
		ActiveViewport.IsValid() &&
		(ActiveViewport->GetPlayInEditorIsSimulate() || (ActiveViewport->GetClient()->GetWorld() && !ActiveViewport->GetClient()->GetWorld()->IsGameWorld())) &&
		SActorEditorContext::IsVisible(GetWorld());
}

void SLevelViewport::ConstructLevelEditorViewportClient(FLevelEditorViewportInstanceSettings& ViewportInstanceSettings)
{
	if (!LevelViewportClient.IsValid())
	{
		LevelViewportClient = MakeShareable( new FLevelEditorViewportClient(SharedThis(this)) );
	}

	FEngineShowFlags EditorShowFlags(ESFIM_Editor);
	FEngineShowFlags GameShowFlags(ESFIM_Game);
		
	// Use config key if it exists to set up the level viewport client
	if(!ConfigKey.IsNone())
	{
		FString ConfigKeyAsString = ConfigKey.ToString();
		const FLevelEditorViewportInstanceSettings* const ViewportInstanceSettingsPtr = GetDefault<ULevelEditorViewportSettings>()->GetViewportInstanceSettings(ConfigKeyAsString);
		ViewportInstanceSettings = (ViewportInstanceSettingsPtr) ? *ViewportInstanceSettingsPtr : LoadLegacyConfigFromIni(ConfigKeyAsString, ViewportInstanceSettings);

		if(!ViewportInstanceSettings.EditorShowFlagsString.IsEmpty())
		{
			EditorShowFlags.SetFromString(*ViewportInstanceSettings.EditorShowFlagsString);
		}

		if(!ViewportInstanceSettings.GameShowFlagsString.IsEmpty())
		{
			GameShowFlags.SetFromString(*ViewportInstanceSettings.GameShowFlagsString);
		}

		if(!GetBufferVisualizationData().GetMaterial(ViewportInstanceSettings.BufferVisualizationMode))
		{
			ViewportInstanceSettings.BufferVisualizationMode = NAME_None;
		}

	}

	if(ViewportInstanceSettings.ViewportType == LVT_Perspective)
	{
		ApplyViewMode(ViewportInstanceSettings.PerspViewModeIndex, true, EditorShowFlags);
		ApplyViewMode(ViewportInstanceSettings.PerspViewModeIndex, true, GameShowFlags);
	}
	else
	{
		ApplyViewMode(ViewportInstanceSettings.OrthoViewModeIndex, false, EditorShowFlags);
		ApplyViewMode(ViewportInstanceSettings.OrthoViewModeIndex, false, GameShowFlags);
	}

	// Disabling some features for orthographic views. 
	if(ViewportInstanceSettings.ViewportType != LVT_Perspective)
	{
		EditorShowFlags.MotionBlur = 0;
		EditorShowFlags.Fog = 0;
		EditorShowFlags.SetDepthOfField(false);
		GameShowFlags.MotionBlur = 0;
		GameShowFlags.Fog = 0;
		GameShowFlags.SetDepthOfField(false);
	}

	EditorShowFlags.SetSnap(1);
	GameShowFlags.SetSnap(1);

	// Create level viewport client
	LevelViewportClient->ParentLevelEditor = ParentLevelEditor.Pin();
	LevelViewportClient->ViewportType = ViewportInstanceSettings.ViewportType;
	LevelViewportClient->bSetListenerPosition = false;
	LevelViewportClient->EngineShowFlags = EditorShowFlags;
	LevelViewportClient->LastEngineShowFlags = GameShowFlags;
	LevelViewportClient->CurrentBufferVisualizationMode = ViewportInstanceSettings.BufferVisualizationMode;
	LevelViewportClient->CurrentNaniteVisualizationMode = ViewportInstanceSettings.NaniteVisualizationMode;
	LevelViewportClient->CurrentLumenVisualizationMode = ViewportInstanceSettings.LumenVisualizationMode;
	LevelViewportClient->CurrentMegaLightsVisualizationMode = ViewportInstanceSettings.MegaLightsVisualizationMode;
	LevelViewportClient->CurrentSubstrateVisualizationMode = ViewportInstanceSettings.SubstrateVisualizationMode;
	LevelViewportClient->CurrentGroomVisualizationMode = ViewportInstanceSettings.GroomVisualizationMode;
	LevelViewportClient->CurrentVirtualShadowMapVisualizationMode = ViewportInstanceSettings.VirtualShadowMapVisualizationMode;
	LevelViewportClient->CurrentVirtualTextureVisualizationMode = ViewportInstanceSettings.VirtualTextureVisualizationMode;
	LevelViewportClient->CurrentRayTracingDebugVisualizationMode = ViewportInstanceSettings.RayTracingDebugVisualizationMode;
	LevelViewportClient->CurrentGPUSkinCacheVisualizationMode = ViewportInstanceSettings.GPUSkinCacheVisualizationMode;
	LevelViewportClient->ChannelMaskParams.ColorChannelMask = (EColorChannelMask)ViewportInstanceSettings.ColorChannelMask;
	LevelViewportClient->ChannelMaskParams.bInvertAlphaChannelMask = ViewportInstanceSettings.bInvertAlphaChannelMask;
	LevelViewportClient->ChannelMaskParams.bDrawAlphaBlendedCheckerboard = ViewportInstanceSettings.bDrawAlphaBlendedCheckerboard;
	LevelViewportClient->ExposureSettings = ViewportInstanceSettings.ExposureSettings;
	if(ViewportInstanceSettings.ViewportType == LVT_Perspective)
	{
		LevelViewportClient->SetViewLocation( EditorViewportDefs::DefaultPerspectiveViewLocation );
		LevelViewportClient->SetViewRotation( EditorViewportDefs::DefaultPerspectiveViewRotation );
	}
	LevelViewportClient->SetAllowCinematicControl(ViewportInstanceSettings.bAllowCinematicControl);
	LevelViewportClient->SetRealtime(ViewportInstanceSettings.bIsRealtime);
	LevelViewportClient->SetShowStats(ViewportInstanceSettings.bShowOnScreenStats);
	if (ViewportInstanceSettings.bShowFPS_DEPRECATED)
	{
		GetMutableDefault<ULevelEditorViewportSettings>()->bSaveEngineStats = true;
		ViewportInstanceSettings.EnabledStats.AddUnique(TEXT("FPS"));
	}
	if (GetDefault<ULevelEditorViewportSettings>()->bSaveEngineStats)
	{
		GEngine->SetEngineStats(GetWorld(), LevelViewportClient.Get(), ViewportInstanceSettings.EnabledStats, true);
	}
	LevelViewportClient->VisibilityDelegate.BindSP( this, &SLevelViewport::IsVisible );
	LevelViewportClient->ImmersiveDelegate.BindSP( this, &SLevelViewport::IsImmersive );
	LevelViewportClient->bDrawBaseInfo = true;
	LevelViewportClient->bDrawVertices = true;
	LevelViewportClient->ViewFOV = LevelViewportClient->FOVAngle = ViewportInstanceSettings.FOVAngle;
	LevelViewportClient->OverrideFarClipPlane( ViewportInstanceSettings.FarViewPlane );
	
	// Set the selection outline flag based on preferences
	LevelViewportClient->EngineShowFlags.SetSelectionOutline(GetDefault<ULevelEditorViewportSettings>()->bUseSelectionOutline);
	
	// Always composite editor objects after post processing in the editor
	LevelViewportClient->EngineShowFlags.SetCompositeEditorPrimitives(true);

	LevelViewportClient->SetViewModes(ViewportInstanceSettings.PerspViewModeIndex, ViewportInstanceSettings.OrthoViewModeIndex );

	LevelViewportClient->InitializeViewportInteraction();

	bShowEditorToolbar = ViewportInstanceSettings.bShowFullToolbar;

	if (FPlatformMisc::IsRemoteSession())
	{
		// Bind to the change delegate of performance settings and call our handler with a dummy event to set current defaults
		UEditorPerformanceSettings* PerformanceSettings = GetMutableDefault< UEditorPerformanceSettings>();
		PerformanceSettings->OnSettingChanged().AddSP(this, &SLevelViewport::OnPerformanceSettingsChanged);
		FPropertyChangedEvent DummyEvent(nullptr);
		OnPerformanceSettingsChanged(PerformanceSettings, DummyEvent);
	}
}

/** Updates the real-time overrride applied to the viewport */
void SLevelViewport::OnPerformanceSettingsChanged(UObject* Obj, FPropertyChangedEvent& ChangeEvent)
{
	if (FPlatformMisc::IsRemoteSession())
	{
		const FText RDPRealtimeOverrideName = LOCTEXT("RealtimeOverrideMessage_RDP", "Remote Desktop");
		UEditorPerformanceSettings* PerformanceSettings = GetMutableDefault< UEditorPerformanceSettings>();

		if (Obj == PerformanceSettings && ensure(LevelViewportClient))
		{
			// Respond to settings changes by adding or removing the realtime override as appropriate
			if (PerformanceSettings->bDisableRealtimeViewportsInRemoteSessions && !LevelViewportClient->HasRealtimeOverride(RDPRealtimeOverrideName))
			{
				bool bShouldBeRealtime = false;
				LevelViewportClient->AddRealtimeOverride(bShouldBeRealtime, RDPRealtimeOverrideName);
			}
			else if (!PerformanceSettings->bDisableRealtimeViewportsInRemoteSessions && LevelViewportClient->HasRealtimeOverride(RDPRealtimeOverrideName))
			{
				LevelViewportClient->RemoveRealtimeOverride(RDPRealtimeOverrideName, false);
			}
		}
	}
}

void SLevelViewport::OnToggleEditorShortcutsInPIE()
{
	ULevelEditorPlaySettings* const PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
	PlaySettings->EnableEditorShortcutsInPIE = !PlaySettings->EnableEditorShortcutsInPIE;
}

bool SLevelViewport::IsEditorShortcutsEnabledInPIE()
{
	return GetDefault<ULevelEditorPlaySettings>()->EnableEditorShortcutsInPIE;
}

FSceneViewport* SLevelViewport::GetGameSceneViewport() const
{
	return ActiveViewport.Get();
}

void SLevelViewport::ToggleViewportToolbarVisibility()
{
	if (IsPlayInEditorViewportActive())
	{
		if (IsImmersive())
		{
			bShowImmersivePIEToolbar = !bShowImmersivePIEToolbar;
		}
		else
		{
			bShowPIEToolbar = !bShowPIEToolbar;
			if (!bShowPIEToolbar)
			{
				bShowImmersivePIEToolbar = false;
			}
		}
	}
	else
	{
		bShowEditorToolbar = !bShowEditorToolbar;
	}
}

bool SLevelViewport::IsViewportToolbarVisible() const
{
	if (IsPlayInEditorViewportActive())
	{
		return IsImmersive() ? bShowImmersivePIEToolbar : bShowPIEToolbar;
	}
	return bShowEditorToolbar;
}

void SLevelViewport::TransitionToPIE(bool bIsSimulating)
{
	for (FViewportActorPreview& ActorPreview : ActorPreviews)
	{
		if (ActorPreview.LevelViewportClient.IsValid() && !ActorPreview.LevelViewportClient->IsSimulateInEditorViewport())
		{
			ActorPreview.LevelViewportClient->SetIsSimulateInEditorViewport(true);
		}
	}
}

void SLevelViewport::TransitionFromPIE(bool bIsSimulating)
{
	for (FViewportActorPreview& ActorPreview : ActorPreviews)
	{
		if (ActorPreview.LevelViewportClient.IsValid() && ActorPreview.LevelViewportClient->IsSimulateInEditorViewport())
		{
			ActorPreview.LevelViewportClient->SetIsSimulateInEditorViewport(false);
		}
	}
}

EStretch::Type SLevelViewport::OnGetScaleBoxStretch() const
{
	FSceneViewport* GameSceneViewport = GetGameSceneViewport();
	if (GameSceneViewport && GameSceneViewport->HasFixedSize())
	{
		return EStretch::ScaleToFit;
	}
	return EStretch::Fill;
}

FVector2D SLevelViewport::GetSViewportSize() const
{
	FSceneViewport* GameSceneViewport = GetGameSceneViewport();
	if (GameSceneViewport && GameSceneViewport->HasFixedSize())
	{
		return GameSceneViewport->GetSize();
	}

	return SViewport::FArguments::GetDefaultViewportSize();
}

FReply SLevelViewport::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{

	FReply Reply = FReply::Unhandled();

	bool bEnableEditorShortcuts = true;
	if (ULevelEditorPlaySettings* const PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>())
	{
		bEnableEditorShortcuts = PlaySettings->EnableEditorShortcutsInPIE;
	}

	if( HasPlayInEditorViewport() || LevelViewportClient->IsSimulateInEditorViewport() )
	{
		// Only process commands for pie when a play world is active
		bool bHandled = FPlayWorldCommands::GlobalPlayWorldActions->ProcessCommandBindings( InKeyEvent );
		
		if (bEnableEditorShortcuts && HasPlayInEditorViewport())
		{
			bHandled = bHandled || PIECommands->ProcessCommandBindings(InKeyEvent);
		}

		// Always handle commands in pie so they arent bubbled to editor only widgets
		Reply = FReply::Handled();
	}
	
	if( !IsPlayInEditorViewportActive() )
	{
		Reply = SEditorViewport::OnKeyDown(MyGeometry,InKeyEvent);

		// Otherwise, give the in-viewport context menu a chance to handle the keypress.
		if (!Reply.IsEventHandled() && InViewportMenu.IsValid())
		{
			Reply = InViewportMenu->GetGeneratedToolbarMenu()->OnKeyDown(MyGeometry, InKeyEvent);
		}


		// If we are in immersive mode and the event was not handled, we will check to see if the the 
		//  optional parent level editor is set.  If it is, we give it a chance to handle the key event.
		//  This command forwarding is currently only needed when in immersive mode because in that case 
		//  the SLevelEditor is not a direct parent of the viewport.  
		if ( this->IsImmersive() && !Reply.IsEventHandled() )
		{
			TSharedPtr<ILevelEditor> ParentLevelEditorSharedPtr = ParentLevelEditor.Pin();
			if( ParentLevelEditorSharedPtr.IsValid() )
			{
				Reply = ParentLevelEditorSharedPtr->OnKeyDownInViewport( MyGeometry, InKeyEvent );
			}
		}
	}

	return Reply;
}

void SLevelViewport::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	// Prevent OnDragEnter from reentering because it will affect the drop preview placement and management.
	// This may happen currently if an unloaded class is dragged from the class viewer and a slow task is triggered,
	// which re-ticks slate and triggers another mouse move.
	static bool bDragEnterReentranceGuard = false;
	if ( !bDragEnterReentranceGuard )
	{
		bDragEnterReentranceGuard = true;
		// Don't execute the dragdrop op if the current level is locked.
		// This prevents duplicate warning messages firing on DragEnter and Placement.
		ULevel* CurrentLevel = (GetWorld()) ? GetWorld()->GetCurrentLevel() : nullptr;

		if ( CurrentLevel && !FLevelUtils::IsLevelLocked(CurrentLevel) )
		{
			if ( HandleDragObjects(MyGeometry, DragDropEvent) )
			{
				// The operation can be null if HandlePlaceDraggedObjects opens a modal dialog,
				// causing a window focus change that triggers OnDragLeave and resets the drag content.
				if (TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation())
				{
					// Hide the decorator before dropping the object to avoid having a decorator present for the 
					// entire duration of an async asset build if required. 
					// Restore the decorator visibility if the drop fails to preserve previous behavior.
					Operation->SetDecoratorVisibility(false);
					if ( !HandlePlaceDraggedObjects(MyGeometry, DragDropEvent, /*bCreateDropPreview=*/true) )
					{
						Operation->SetDecoratorVisibility(true);
					}
				}
			}
		}
		
		bDragEnterReentranceGuard = false;
	}
}

void SLevelViewport::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	if ( LevelViewportClient->HasDropPreviewActors() )
	{
		LevelViewportClient->DestroyDropPreviewElements();
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid())
	{
		Operation->SetDecoratorVisibility(true);

		if (Operation->IsOfType<FDecoratedDragDropOp>())
		{
			TSharedPtr<FDecoratedDragDropOp> DragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(Operation);
			DragDropOp->ResetToDefaultToolTip();
		}
	}
}

bool SLevelViewport::HandleDragObjects(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bool bValidDrag = false;
	TArray<FAssetData> SelectedAssetDatas;

	TSharedPtr< FDragDropOperation > Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return false;
	}

	if (Operation->IsOfType<FClassDragDropOp>())
	{
		auto ClassOperation = StaticCastSharedPtr<FClassDragDropOp>( Operation );

		bValidDrag = true;

		for (int32 DroppedAssetIdx = 0; DroppedAssetIdx < ClassOperation->ClassesToDrop.Num(); ++DroppedAssetIdx)
		{
			new(SelectedAssetDatas)FAssetData(ClassOperation->ClassesToDrop[DroppedAssetIdx].Get());
		}
	}
	else if (Operation->IsOfType<FExportTextDragDropOp>())
	{
		bValidDrag = true;
	}
	else if (Operation->IsOfType<FBrushBuilderDragDropOp>())
	{
		bValidDrag = true;

		auto BrushOperation = StaticCastSharedPtr<FBrushBuilderDragDropOp>( Operation );

		new(SelectedAssetDatas) FAssetData(BrushOperation->GetBrushBuilder().Get());
	}
	else if (Operation->IsOfType<FInViewportUIDragOperation>())
	{
		bValidDrag = true;
	}
	else if (Operation->IsOfType<FExternalDragOperation>() && StaticCastSharedPtr<FExternalDragOperation>(Operation)->HasText())
	{
		bValidDrag = true;
	}
	else
	{
		GetAssetsFromDrag(DragDropEvent, SelectedAssetDatas);

		if ( SelectedAssetDatas.Num() > 0 )
		{
			bValidDrag = true;
		}
	}

	// Update cached mouse position
	if ( bValidDrag )
	{
		// Grab viewport to offset click position correctly
		FIntPoint ViewportOrigin, ViewportSize;
		LevelViewportClient->GetViewportDimensions(ViewportOrigin, ViewportSize);

		// Save off the local mouse position from the drop point for potential use later (with Drag Drop context menu)
		const FVector2D ToViewportPixelScale = UE::Slate::Viewport::GetViewportLocalToPixelScale(MyGeometry, ViewportSize);
		CachedOnDropLocalMousePos = FVector2f(MyGeometry.AbsoluteToLocal( DragDropEvent.GetScreenSpacePosition() ) * ToViewportPixelScale).IntPoint();
		CachedOnDropLocalMousePos -= ViewportOrigin;
	}

	// Update the currently dragged actor if it exists
	bool bDroppedObjectsVisible = true;
	if (LevelViewportClient->UpdateDropPreviewActors(CachedOnDropLocalMousePos.X, CachedOnDropLocalMousePos.Y, DroppedObjects, bDroppedObjectsVisible))
	{
		// if dragged actors were hidden, show decorator
		Operation->SetDecoratorVisibility(! bDroppedObjectsVisible);
	}

	Operation->SetCursorOverride(TOptional<EMouseCursor::Type>());

	FText HintText;

	// Determine if we can drop the assets
	for ( auto InfoIt = SelectedAssetDatas.CreateConstIterator(); InfoIt; ++InfoIt )
	{
		const FAssetData& AssetData = *InfoIt;

		// Ignore invalid assets
		if ( !AssetData.IsValid() )
		{
			continue;
		}
		
		FDropQuery DropResult = LevelViewportClient->CanDropObjectsAtCoordinates(CachedOnDropLocalMousePos.X, CachedOnDropLocalMousePos.Y, AssetData);
		
		if ( !DropResult.bCanDrop )
		{
			// At least one of the assets can't be dropped.
			Operation->SetCursorOverride(EMouseCursor::SlashedCircle);
			bValidDrag = false;
			HintText = DropResult.HintText;
			break;
		}
		else
		{
			if ( HintText.IsEmpty() )
			{
				HintText = DropResult.HintText;
			}
		}
	}

	if ( Operation->IsOfType<FAssetDragDropOp>() )
	{
		auto AssetOperation = StaticCastSharedPtr<FAssetDragDropOp>(DragDropEvent.GetOperation());
		AssetOperation->SetToolTip(HintText, nullptr);
	}

	return bValidDrag;
}

FReply SLevelViewport::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( HandleDragObjects(MyGeometry, DragDropEvent) )
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SLevelViewport::HandlePlaceDraggedObjects(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bCreateDropPreview)
{
	bool bAllAssetWereLoaded = false;
	bool bValidDrop = false;
	TScriptInterface<IAssetFactoryInterface> AssetFactory = nullptr;

	TSharedPtr< FDragDropOperation > Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return false;
	}

	// Don't handle the placement if we couldn't handle the drag
	if (!HandleDragObjects(MyGeometry, DragDropEvent))
	{
		return false;
	}

	if (Operation->IsOfType<FClassDragDropOp>())
	{
		auto ClassOperation = StaticCastSharedPtr<FClassDragDropOp>( Operation );

		DroppedObjects.Empty();

		// Check if the asset is loaded, used to see if the context menu should be available
		bAllAssetWereLoaded = true;

		for (int32 DroppedAssetIdx = 0; DroppedAssetIdx < ClassOperation->ClassesToDrop.Num(); ++DroppedAssetIdx)
		{
			UObject* Object = ClassOperation->ClassesToDrop[DroppedAssetIdx].Get();

			if(Object)
			{
				DroppedObjects.Add(Object);
			}
			else
			{	
				bAllAssetWereLoaded = false;
			}
		}

		bValidDrop = true;
	}
	else if (Operation->IsOfType<FAssetDragDropOp>())
	{
		bValidDrop = true;
		DroppedObjects.Empty();

		TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>( Operation );

		AssetFactory = DragDropOp->GetAssetFactory();

		bAllAssetWereLoaded = true;
		for (const FAssetData& AssetData : DragDropOp->GetAssets())
		{
			UObject* Asset = AssetData.GetAsset();
			if ( Asset != nullptr )
			{
				DroppedObjects.Add( Asset );
			}
			else
			{
				bAllAssetWereLoaded = false;
			}
		}
	}
	// OLE drops are blocking which causes problem when positioning and maintaining the drop preview
	// Drop preview is disabled when dragging from external sources
	else if ( !bCreateDropPreview && Operation->IsOfType<FExternalDragOperation>() )
	{
		bValidDrop = true;
		DroppedObjects.Empty();

		TArray<FAssetData> DroppedAssetDatas;
		GetAssetsFromDrag(DragDropEvent, DroppedAssetDatas);

		bAllAssetWereLoaded = true;
		for (int32 AssetIdx = 0; AssetIdx < DroppedAssetDatas.Num(); ++AssetIdx)
		{
			const FAssetData& AssetData = DroppedAssetDatas[AssetIdx];

			UObject* Asset = AssetData.GetAsset();
			if (Asset != nullptr)
			{
				DroppedObjects.Add(Asset);
			}
			else
			{
				bAllAssetWereLoaded = false;
			}
		}
	}
	else if ( Operation->IsOfType<FExportTextDragDropOp>() )
	{
		bValidDrop = true;

		TSharedPtr<FExportTextDragDropOp> DragDropOp = StaticCastSharedPtr<FExportTextDragDropOp>( Operation );

		// Check if the asset is loaded, used to see if the context menu should be available
		bAllAssetWereLoaded = true;
		DroppedObjects.Empty();

		// Create a container object to hold the export text and pass it into the actor placement code
		UExportTextContainer* NewContainer = NewObject<UExportTextContainer>();
		NewContainer->ExportText = DragDropOp->ActorExportText;
		DroppedObjects.Add(NewContainer);
	}
	else if ( Operation->IsOfType<FBrushBuilderDragDropOp>() )
	{
		bValidDrop = true;
		DroppedObjects.Empty();

		TSharedPtr<FBrushBuilderDragDropOp> DragDropOp = StaticCastSharedPtr<FBrushBuilderDragDropOp>( Operation );

		if(DragDropOp->GetBrushBuilder().IsValid())
		{
			DroppedObjects.Add(DragDropOp->GetBrushBuilder().Get());
		}
	}

	if ( bValidDrop )
	{
		// Grab the hit proxy, used for the (potential) context menu
		HHitProxy* HitProxy = LevelViewportClient->Viewport->GetHitProxy(CachedOnDropLocalMousePos.X, CachedOnDropLocalMousePos.Y);

		// If Ctrl is down, pop in the context menu
		const bool bShowDropContextMenu = !bCreateDropPreview && DragDropEvent.IsControlDown() && ( !HitProxy || !( HitProxy->IsA( HWidgetAxis::StaticGetType() ) ) );
		bool bDropSuccessful = false;

		// Make sure the drop preview is destroyed
		// Note that this can run gc, better hold onto our objects!
		TArray<TStrongObjectPtr<UObject>> DroppedObjectsLocked;
		Algo::Transform(DroppedObjects, DroppedObjectsLocked, [](UObject* Obj){ return TStrongObjectPtr<UObject>(Obj); }); // Transform to accomodate explicit ctor
		LevelViewportClient->DestroyDropPreviewElements();

		if( !bShowDropContextMenu || !bCreateDropPreview )
		{
			// Otherwise just attempt to drop the object(s)
			FLevelEditorViewportClient::FDropObjectOptions DropOptions;
			DropOptions.FactoryToUse = AssetFactory;
			DropOptions.bOnlyDropOnTarget = false;
			DropOptions.bCreateDropPreview = bCreateDropPreview;
			// Only select actor on drop
			DropOptions.bSelectOutput = !bCreateDropPreview;

			TArray<FTypedElementHandle> Unused;
			bDropSuccessful = LevelViewportClient->DropObjectsAtCoordinates(CachedOnDropLocalMousePos.X, CachedOnDropLocalMousePos.Y, 
				DroppedObjects, Unused, DropOptions);
		}
		else if ( bAllAssetWereLoaded && DroppedObjects.Num() > 0 )
		{
			FWidgetPath WidgetPath = DragDropEvent.GetEventPath() != nullptr ? *DragDropEvent.GetEventPath() : FWidgetPath();

			FSlateApplication::Get().PushMenu(
				SharedThis( this ),
				WidgetPath,
				BuildViewportDragDropContextMenu(),
				DragDropEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu ) );

			bDropSuccessful = true;
		}

		// Give the editor focus (quick Undo/Redo support after a drag drop operation)
		if(ParentLevelEditor.IsValid())
		{
			FGlobalTabmanager::Get()->DrawAttentionToTabManager(ParentLevelEditor.Pin()->GetTabManager().ToSharedRef());
		}

		if(bDropSuccessful)
		{
			SetKeyboardFocusToThisViewport();
		}

		return bDropSuccessful;
	}
	
	return false;
}

void SLevelViewport::GetAssetsFromDrag(const FDragDropEvent& DragDropEvent, TArray<FAssetData>& AssetDataArray)
{
	AssetDataArray = AssetUtil::ExtractAssetDataFromDrag(DragDropEvent);

	// Did we get anything?
	if (AssetDataArray.Num() == 0)
	{
		// Get files from the drag/drop.
		const TSharedPtr<FDragDropOperation>& Operation = DragDropEvent.GetOperation();
		if (Operation.IsValid())
		{
			if (Operation->IsOfType<FExternalDragOperation>())
			{
				TSharedPtr<FExternalDragOperation> DragDropOp = StaticCastSharedPtr<FExternalDragOperation>(Operation);
				if (DragDropOp->HasFiles())
				{
					const TArray<FString>& Files = DragDropOp->GetFiles();
					if (Files.Num() > 0)
					{
						// See if anyone else can get us an asset.
						UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();
						if (EditorAssetSubsystem != nullptr)
						{
							EditorAssetSubsystem->GetOnExtractAssetFromFile().Broadcast(Files, AssetDataArray);
						}
					}
				}
			}
		}
	}
}

FReply SLevelViewport::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	// The drag operation can be null if a modal dialog shown during OnDragOver causes a window focus change, 
	// which triggers OnDragLeave and resets the drag content before OnDrop is called.
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return FReply::Unhandled();
	}

	if (Operation->IsOfType<FInViewportUIDragOperation>())
	{
		FVector2D ScreenSpaceDropLocation = DragDropEvent.GetScreenSpacePosition() - DragDropEvent.GetOperationAs<FInViewportUIDragOperation>()->GetDecoratorOffsetFromCursor();
		DragDropEvent.GetOperationAs<FInViewportUIDragOperation>()->BroadcastDropEvent(MyGeometry.AbsoluteToLocal(ScreenSpaceDropLocation));
	}
	else
	{
		ULevel* CurrentLevel = (GetWorld()) ? GetWorld()->GetCurrentLevel() : nullptr;

		if (CurrentLevel && !FLevelUtils::IsLevelLocked(CurrentLevel))
		{
			return HandlePlaceDraggedObjects(MyGeometry, DragDropEvent, /*bCreateDropPreview=*/false) ? FReply::Handled() : FReply::Unhandled();
		}
		else
		{
			FNotificationInfo Info(LOCTEXT("Error_OperationDisallowedOnLockedLevel", "The requested operation could not be completed because the level is locked."));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}


void SLevelViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SEditorViewport::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );

	const bool bContainsFocus = HasFocusedDescendants();

	// When we have focus we update the 'Allow Throttling' option in slate to be disabled so that interactions in the
	// viewport with Slate widgets that are part of the game, don't throttle.
	if ( GEditor->PlayWorld != nullptr && bPIEContainsFocus != bContainsFocus )
	{
		// We can arrive at this point before creating throttling manager (which registers the cvar), so create it explicitly.
		static const FSlateThrottleManager & ThrottleManager = FSlateThrottleManager::Get();
		static IConsoleVariable* AllowThrottling = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.bAllowThrottling"));
		check(AllowThrottling);

		if ( bContainsFocus )
		{
			UserAllowThrottlingValue = AllowThrottling->GetInt();
			AllowThrottling->Set(0);
		}
		else
		{
			AllowThrottling->Set(UserAllowThrottlingValue);
		}

		bPIEContainsFocus = bContainsFocus;
	}

	// We defer starting animation playback because very often there may be a large hitch after the frame in which
	// the animation was triggered, and we don't want to start animating until after that hitch.  Otherwise, the
	// user could miss part of the animation, or even the whole thing!
	if( bViewTransitionAnimPending )
	{
		ViewTransitionAnim.Play(this->AsShared());
		bViewTransitionAnimPending = false;
	}

	// If we've completed a transition, then start animating back to our regular border.  We
	// do this so that we can avoid a popping artifact after PIE/SIE ends.
	if( !ViewTransitionAnim.IsPlaying() && ViewTransitionType != EViewTransition::None )
	{
		if(ViewTransitionType == EViewTransition::StartingPlayInEditor)
		{
			if (PIEOverlayBorder.IsValid())
			{
				PIEOverlayAnim = FCurveSequence(0.0f, SLevelViewportPIEAnimation::MouseControlLabelFadeout, ECurveEaseFunction::CubicInOut);
				PIEOverlayAnim.Play(this->AsShared());
			}
		}
		ViewTransitionType = EViewTransition::None;
		ViewTransitionAnim = FCurveSequence( 0.0f, 0.25f, ECurveEaseFunction::QuadOut );
		ViewTransitionAnim.PlayReverse(this->AsShared());
	}

	if(IsPlayInEditorViewportActive() && bPIEHasFocus != ActiveViewport->HasMouseCapture())
	{
		bPIEHasFocus = ActiveViewport->HasMouseCapture();
		if (!bPIEHasFocus)
		{
			PIEOverlayAnim = FCurveSequence(0.0f, SLevelViewportPIEAnimation::MouseControlLabelFadeout, ECurveEaseFunction::CubicInOut);
			PIEOverlayAnim.Play(this->AsShared());
		}
	}

	// Update actor preview viewports, if we have any
	UpdateActorPreviewViewports();

#if STATS
	// Check to see if there are any new stat group which need registering with the viewports
	extern CORE_API void CheckForRegisteredStatGroups();
	CheckForRegisteredStatGroups();
#endif

	if (bNeedToUpdatePreviews)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bPreviewInDesktopViewport = !IVREditorModule::Get().IsVREditorModeActive();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (GetDefault<ULevelEditorViewportSettings>()->bPreviewSelectedCameras && GCurrentLevelEditingViewportClient == LevelViewportClient.Get())
		{
			PreviewSelectedCameraActors(bPreviewInDesktopViewport);
		}
		else
		{
			// We're no longer the active viewport client, so remove any existing previewed actors
			PreviewActors(TArray<AActor*>(), bPreviewInDesktopViewport);
		}
		bNeedToUpdatePreviews = false;
	}
}


TSharedRef< SWidget > SLevelViewport::BuildViewportDragDropContextMenu()
{
	// Get all menu extenders for this context menu from the level editor module
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( LevelEditorName );
	TArray<FLevelEditorModule::FLevelViewportMenuExtender_SelectedObjects> MenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportDragDropContextMenuExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute(CommandList.ToSharedRef(), DroppedObjects));
		}
	}
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);

	// Builds a context menu used to perform specific actions on actors selected within the editor
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ViewportContextMenuBuilder( bShouldCloseWindowAfterMenuSelection, CommandList, MenuExtender );
	{
		FLevelViewportDropContextMenuImpl::FillDropAddReplaceActorMenu( false, ViewportContextMenuBuilder );

		// If any actors are in the current editor selection, add submenu for swapping out those actors with an asset from the chosen factory
		if( GEditor->GetSelectedActorCount() > 0 && !AssetSelectionUtils::IsBuilderBrushSelected() )
		{
			FLevelViewportDropContextMenuImpl::FillDropAddReplaceActorMenu( true, ViewportContextMenuBuilder );
		}

		if(DroppedObjects.Num() > 0)
		{
			// Grab the hit proxy, used for determining which object we're potentially targeting
			const HHitProxy* DroppedUponProxy = LevelViewportClient->Viewport->GetHitProxy(CachedOnDropLocalMousePos.X, CachedOnDropLocalMousePos.Y);
			UObject* FirstDroppedObject = DroppedObjects[0];

			// If we're using a material asset, check if the apply material option(s) should be added
			if(DroppedUponProxy && Cast<UMaterialInterface>(FirstDroppedObject) && LevelViewportClient->CanApplyMaterialToHitProxy(DroppedUponProxy))
			{
				ViewportContextMenuBuilder.BeginSection("ApplyMaterial");
				{
					ViewportContextMenuBuilder.AddMenuEntry( FLevelViewportCommands::Get().ApplyMaterialToActor );
				}
				ViewportContextMenuBuilder.EndSection();
			}
		}
	}

	return ViewportContextMenuBuilder.MakeWidget();
}

void SLevelViewport::OnMapChanged( UWorld* World, EMapChangeType MapChangeType )
{
	using namespace UE::SLevelViewport::Internal;

	if( World && ( ( World == GetWorld() ) || ( World->EditorViews[LevelViewportClient->ViewportType].CamUpdated ) ) )
	{
		if( MapChangeType == EMapChangeType::LoadMap )
		{
			ResetNewLevelViewFlags();
			LoadViewportInfo(World, LevelViewportClient.Get(), GetMutableDefault<ULevelEditorViewportSettings>());
		}
		else if( (MapChangeType == EMapChangeType::SaveMap) || (MapChangeType == EMapChangeType::TearDownWorld))
		{
			SaveViewportInfo(World, LevelViewportClient.Get(), GetMutableDefault<ULevelEditorViewportSettings>());
		}
		else if( MapChangeType == EMapChangeType::NewMap )
		{
			ResetNewLevelViewFlags();
			LevelViewportClient->ResetViewForNewMap();
		}
		World->EditorViews[LevelViewportClient->ViewportType].CamUpdated = false;

		World->ShaderPlatformChanged(GEditor->GetActiveShaderPlatform());

		RedrawViewport(true);
	}
}

void SLevelViewport::OnEditorClose()
{
	UE::SLevelViewport::Internal::SaveViewportInfo(GetWorld(), LevelViewportClient.Get(), GetMutableDefault<ULevelEditorViewportSettings>());
}

void SLevelViewport::OnLevelActorsRemoved(AActor* InActor)
{
	// If the piloted actor is being deleted, stop piloting before it becomes invalid.
	// OnLevelActorDeleted fires while the actor is still fully alive, so SetActorLock
	// can safely restore the transform here.
	if (InActor && LevelViewportClient->IsActorLocked(InActor))
	{
		OnActorUnlock();
	}

	// Kill any existing actor previews that have expired
	for( int32 PreviewIndex = 0; PreviewIndex < ActorPreviews.Num(); ++PreviewIndex )
	{
		AActor* ExistingActor = ActorPreviews[PreviewIndex].Actor.Get();
		if ( !ExistingActor || ExistingActor == InActor )
		{
			// decrement index so we don't miss next preview after deleting
			RemoveActorPreview( PreviewIndex-- );
		}
	}
}

void FLevelViewportDropContextMenuImpl::FillDropAddReplaceActorMenu( bool bReplace, FMenuBuilder& MenuBuilder )
{
	// Builds a submenu for the Drag Drop context menu used to replace all actors in the current editor selection with a different asset
	TArray<FAssetData> SelectedAssets;
	AssetSelectionUtils::GetSelectedAssets( SelectedAssets );

	FAssetData TargetAssetData;
	if ( SelectedAssets.Num() > 0 )
	{
		TargetAssetData = SelectedAssets.Top();
	}

	TArray< FActorFactoryAssetProxy::FMenuItem > SelectedAssetMenuOptions;
	FActorFactoryAssetProxy::GenerateActorFactoryMenuItems( TargetAssetData, &SelectedAssetMenuOptions, false );

	if(SelectedAssetMenuOptions.Num() > 0)
	{
		FText AddReplaceTitle = (bReplace)? FText::GetEmpty() : LOCTEXT("DragDropContext_AddAsType", "Add As Type");

		MenuBuilder.BeginSection("AddReplace", AddReplaceTitle);
		{
			for( int32 ItemIndex = 0; ItemIndex < SelectedAssetMenuOptions.Num(); ++ItemIndex )
			{
				const FActorFactoryAssetProxy::FMenuItem& MenuItem = SelectedAssetMenuOptions[ItemIndex];
	
				if ( bReplace )
				{
					FUIAction Action( FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ReplaceActors_Clicked, MenuItem.FactoryToUse, MenuItem.AssetData ) );
	
					FText MenuEntryName = FText::Format( NSLOCTEXT("LevelEditor", "ReplaceActorMenuFormat", "Replace with {0}"), MenuItem.FactoryToUse->GetDisplayName() );
					if ( MenuItem.AssetData.IsValid() )
					{
						MenuEntryName = FText::Format( NSLOCTEXT("LevelEditor", "ReplaceActorUsingAssetMenuFormat", "Replace with {0}: {1}"), 
							MenuItem.FactoryToUse->GetDisplayName(), 
							FText::FromName( MenuItem.AssetData.AssetName ) );
					}
				}
				else
				{
					FUIAction Action( FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::AddActor_Clicked, MenuItem.FactoryToUse, MenuItem.AssetData ) );
				
					FText MenuEntryName = FText::Format( NSLOCTEXT("SLevelViewport", "AddActorMenuFormat", "Add {0}"), MenuItem.FactoryToUse->GetDisplayName() );
					if ( MenuItem.AssetData.IsValid() )
					{
						MenuEntryName = FText::Format( NSLOCTEXT("SLevelViewport", "AddActorUsingAssetMenuFormat", "Add {0}: {1}"), 
							MenuItem.FactoryToUse->GetDisplayName(), 
							FText::FromName( MenuItem.AssetData.AssetName ) );
					}
				}
			}
		}
		MenuBuilder.EndSection();
	}
}

/**
 * Bound event Triggered via FLevelViewportCommands::ApplyMaterialToActor, attempts to apply a material selected in the content browser
 * to an actor being hovered over in the Editor viewport.
 */ 
void SLevelViewport::OnApplyMaterialToViewportTarget()
{
	if(DroppedObjects.Num() > 0)
	{
		// Grab the hit proxy, used for determining which object we're potentially targeting
		const HHitProxy* DroppedUponProxy = LevelViewportClient->Viewport->GetHitProxy(CachedOnDropLocalMousePos.X, CachedOnDropLocalMousePos.Y);
		UObject* FirstDroppedObject = DroppedObjects[0];

		// Ensure we're dropping a material asset and our target is an acceptable receiver
		if(DroppedUponProxy && Cast<UMaterialInterface>(FirstDroppedObject) && LevelViewportClient->CanApplyMaterialToHitProxy(DroppedUponProxy))
		{
			// Drop the object, but ensure we're only affecting the target actor, not whatever may be in the current selection
			TArray< AActor* > TemporaryActors;
			LevelViewportClient->DropObjectsAtCoordinates(CachedOnDropLocalMousePos.X,CachedOnDropLocalMousePos.Y, DroppedObjects, TemporaryActors, true);
		}
	}
}

void SLevelViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	FUICommandList& UICommandListRef = *CommandList;

	BindOptionCommands( UICommandListRef );
	BindViewCommands( UICommandListRef );
	BindDropCommands( UICommandListRef );

	// Re-Map Perspective command so that
	// - Perspective menu entry radial checkbox is not flagged when actors are being piloted (including cameras)
	// - Clicking on the Perspective entry interrupts piloting
	{
		const TWeakPtr<SLevelViewport>& LevelViewportWeak = SharedThis(this).ToWeakPtr();

		UICommandListRef.MapAction(
			FEditorViewportCommands::Get().Perspective,
			FExecuteAction::CreateLambda(
				[LevelViewportWeak]()
				{
					if (const TSharedPtr<::SLevelViewport>& LevelViewportPinned =
							LevelViewportWeak.Pin())
					{
						// If piloting, stop
						LevelViewportPinned->OnActorUnlock();

						if (const TSharedPtr<FEditorViewportClient>& ViewportClient = LevelViewportPinned->GetViewportClient())
						{
							// Set to perspective
							ViewportClient->SetViewportType(LVT_Perspective);
						}
					}
				}
			),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(
				[LevelViewportWeak]()
				{
					if (const TSharedPtr<::SLevelViewport>& LevelViewportPinned =
							LevelViewportWeak.Pin())
					{
						// Don't check action when piloting an actor
						if (LevelViewportPinned->IsAnyActorLocked())
						{
							return false;
						}

						if (const TSharedPtr<FEditorViewportClient>& ViewportClient = LevelViewportPinned->GetViewportClient())
						{
							return ViewportClient->IsPerspective();
						}
					}
					return false;
				}
			)
		);
	}

	if ( ParentLevelEditor.IsValid() )
	{
		UICommandListRef.Append(ParentLevelEditor.Pin()->GetLevelEditorActions().ToSharedRef());
	}

	UICommandListRef.SetCanProduceActionForCommand( FUICommandList::FCanProduceActionForCommand::CreateSP(this, &SLevelViewport::CanProduceActionForCommand) );

	// Exposes the current level viewport command list to subscribers from other systems
	FInputBindingManager::Get().RegisterCommandList(FLevelViewportCommands::Get().GetContextName(), CommandList.ToSharedRef());
	
	PIECommands = MakeShared<FUICommandList>();
	BindPIECommands(*PIECommands);
}
	
void SLevelViewport::BindOptionCommands( FUICommandList& OutCommandList )
{
	const FLevelViewportCommands& ViewportActions = FLevelViewportCommands::Get();

	OutCommandList.MapAction( 
		ViewportActions.AdvancedSettings,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnAdvancedSettings ) );
	OutCommandList.MapAction(
		ViewportActions.PlaySettings,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnPlaySettings ) );

	OutCommandList.MapAction( 
		ViewportActions.ToggleMaximize,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleMaximizeMode ),
		FCanExecuteAction::CreateSP( this, &SLevelViewport::CanToggleMaximizeMode ) );

	
	OutCommandList.MapAction(
		ViewportActions.ToggleGameView,
		FExecuteAction::CreateSP( this, &SLevelViewport::ToggleGameView ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SLevelViewport::IsInGameView ) );

	OutCommandList.MapAction(
		ViewportActions.ToggleImmersive,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleImmersive),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SLevelViewport::IsImmersive ) );

	OutCommandList.MapAction(
		ViewportActions.ToggleSidebarAllTabs,
		FExecuteAction::CreateSP(this, &SLevelViewport::OnToggleSidebarTabs)
	);

	OutCommandList.MapAction(
		ViewportActions.ToggleCinematicPreview,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleAllowCinematicPreview ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SLevelViewport::AllowsCinematicPreview )
		);

	OutCommandList.MapAction(
		ViewportActions.ToggleAllowConstrainedAspectRatioInPreview,
		FExecuteAction::CreateLambda([]() { GEditor->ToggleAllowConstrainedAspectRatioInPreview(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]() { return GEditor->GetAllowConstrainedAspectRatioInPreview(); }),
		FIsActionButtonVisible::CreateLambda([]() { return GEditor->IsPreviewPlatformActive() && (GEditor->PreviewPlatform.GetConstrainedAspectRatio() != 0.0f); })
	);

	OutCommandList.MapAction(
		ViewportActions.ToggleFlipConstrainedAspectRatioInPreview,
		FExecuteAction::CreateLambda([]() { GEditor->ToggleFlipConstrainedAspectRatioInPreview(); }),
		FCanExecuteAction::CreateLambda([]() { return GEditor->GetAllowConstrainedAspectRatioInPreview(); }),
		FIsActionChecked::CreateLambda([]() { return GEditor->GetFlipConstrainedAspectRatioInPreview(); }),
		FIsActionButtonVisible::CreateLambda([]() { return GEditor->IsPreviewPlatformActive() && (GEditor->PreviewPlatform.GetConstrainedAspectRatio() != 0.0f); })
	);

	IAssetRegistry & AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	TArray<FTopLevelAssetPath> ClassNames;
	TSet<FTopLevelAssetPath> DerivedClassNames;
	ClassNames.Add(ACameraActor::StaticClass()->GetClassPathName());
	AssetRegistry.GetDerivedClassNames(ClassNames, TSet<FTopLevelAssetPath>(), DerivedClassNames);

	for (FTopLevelAssetPath ClassPathName : DerivedClassNames)
	{
		FString Name = ClassPathName.ToString();
		// Ignore generated types that cannot be spawned
		if (Name.Contains("SKEL_") || Name.Contains("REINST_"))
		{
			continue;
		}

		UClass* CameraClass = FindObject<UClass>(ClassPathName);
		if (!CameraClass || CameraClass->HasAnyClassFlags(CLASS_Abstract|CLASS_NotPlaceable|CLASS_HideDropDown|CLASS_Hidden))
		{
			continue;
		}

		// Look for existing UI Command info so one isn't created for every viewport
		FName CommandName;
		{
			int32 DotIndex = -1;
			Name.FindLastChar('.', DotIndex);
			CommandName = *Name.Mid(DotIndex + 1);
		}
		TSharedPtr<FUICommandInfo> * FoundCamera = FLevelViewportCommands::Get().CreateCameras.FindByPredicate([CommandName](TSharedPtr<FUICommandInfo> Camera) { return Camera->GetCommandName() == CommandName; });
		
		if (FoundCamera)
		{
			OutCommandList.MapAction(
				*FoundCamera,
				FExecuteAction::CreateSP(this, &SLevelViewport::OnCreateCameraActor, CameraClass)
			);
		}
		else
		{
			// If command info isn't found, create a new one
			TSharedRef<FUICommandInfo> NewCamera = FUICommandInfoDecl(FLevelViewportCommands::Get().AsShared(), CommandName, CameraClass->GetDisplayNameText(), FText::Format(LOCTEXT("SpawnCamerasTooltip", "Spawn Camera here of type {0}"), FText::FromString(Name))).UserInterfaceType(EUserInterfaceActionType::Button).DefaultChord(FInputChord());
			
			OutCommandList.MapAction(
				NewCamera,
				FExecuteAction::CreateSP(this, &SLevelViewport::OnCreateCameraActor, CameraClass)
			);
			
			FLevelViewportCommands::Get().CreateCameras.Add(NewCamera);
		}

	}

	OutCommandList.MapAction(
		ViewportActions.HighResScreenshot,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnTakeHighResScreenshot ),
		FCanExecuteAction()
		);

	OutCommandList.MapAction(
		ViewportActions.ToggleActorPilotCameraView,
		FExecuteAction::CreateSP(this, &SLevelViewport::ToggleActorPilotCameraView),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SLevelViewport::IsLockedCameraViewEnabled )
		);

	// Map each bookmark action
	for( int32 BookmarkIndex = 0; BookmarkIndex < AWorldSettings::NumMappedBookmarks; ++BookmarkIndex )
	{
		OutCommandList.MapAction( 
			ViewportActions.JumpToBookmarkCommands[BookmarkIndex],
			FExecuteAction::CreateSP( this, &SLevelViewport::OnJumpToBookmark, BookmarkIndex ),
			FCanExecuteAction::CreateSP(this, &SLevelViewport::OnHasBookmarkSet, BookmarkIndex)
			);

		OutCommandList.MapAction( 
			ViewportActions.SetBookmarkCommands[BookmarkIndex],
			FExecuteAction::CreateSP( this, &SLevelViewport::OnSetBookmark, BookmarkIndex )
			);

		OutCommandList.MapAction( 
			ViewportActions.ClearBookmarkCommands[BookmarkIndex],
			FExecuteAction::CreateSP( this, &SLevelViewport::OnClearBookmark, BookmarkIndex )
			);
	}

	OutCommandList.MapAction(
		ViewportActions.CompactBookmarks,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnCompactBookmarks )
	);

	OutCommandList.MapAction(
		ViewportActions.ClearAllBookmarks,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnClearAllBookmarks )
		);

	OutCommandList.MapAction(
		ViewportActions.ToggleViewportToolbar,
		FExecuteAction::CreateSP( this, &SLevelViewport::ToggleViewportToolbarVisibility ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SLevelViewport::IsViewportToolbarVisible )
		);

	OutCommandList.MapAction(
		ViewportActions.EnableEditorShortcutsInPIE,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleEditorShortcutsInPIE ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SLevelViewport::IsEditorShortcutsEnabledInPIE )
		);
}

void SLevelViewport::BindViewCommands( FUICommandList& OutCommandList )
{
	const FLevelViewportCommands& ViewportActions = FLevelViewportCommands::Get();

	OutCommandList.MapAction(
		ViewportActions.FindInLevelScriptBlueprint,
		FExecuteAction::CreateSP( this, &SLevelViewport::FindSelectedInLevelScript ),
		FCanExecuteAction::CreateSP( this, &SLevelViewport::CanFindSelectedInLevelScript ) 
		);

	OutCommandList.MapAction(
		ViewportActions.SelectPilotedActor,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnSelectLockedActor ),
		FCanExecuteAction::CreateSP( this, &SLevelViewport::CanExecuteSelectLockedActor )
		);

	OutCommandList.MapAction(
		ViewportActions.EjectActorPilot,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnActorUnlock ),
		FCanExecuteAction::CreateSP( this, &SLevelViewport::CanExecuteActorUnlock )
		);

	OutCommandList.MapAction(
		ViewportActions.PilotSelectedActor,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnActorLockSelected ),
		FCanExecuteAction::CreateSP( this, &SLevelViewport::CanExecuteActorLockSelected )
		);

	if (TSharedPtr<FLevelViewportLayout> LayoutPinned = ParentLayout.Pin())
	{
		if (TSharedPtr<FEditorViewportTabContent> ParentTabContentPinned = LayoutPinned->GetParentTabContent().Pin())
		{
			ParentTabContentPinned->BindViewportLayoutCommands(OutCommandList, ConfigKey);
			
			// Override single panel configuration so that it acts like a one-way "maximize" button
			OutCommandList.MapAction(
				ViewportActions.ViewportConfig_OnePane,
				FExecuteAction::CreateSP(this, &SLevelViewport::MakeMaximized, true, true),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLevelViewport::IsMaximized)
				);
		}
	}

	FBufferVisualizationMenuCommands::Get().BindCommands(OutCommandList, Client);
	if (IsRayTracingAllowed())
	{
		FRayTracingDebugVisualizationMenuCommands::Get().BindCommands(OutCommandList, Client);
	}
	FNaniteVisualizationMenuCommands::Get().BindCommands(OutCommandList, Client);
	FLumenVisualizationMenuCommands::Get().BindCommands(OutCommandList, Client);
	FMegaLightsVisualizationMenuCommands::Get().BindCommands(OutCommandList, Client);
	if (Substrate::IsSubstrateEnabled())
	{
		FSubstrateVisualizationMenuCommands::Get().BindCommands(OutCommandList, Client);
	}
	if (IsGroomEnabled())
	{
		FGroomVisualizationMenuCommands::Get().BindCommands(OutCommandList, Client);
	}
	FVirtualShadowMapVisualizationMenuCommands::Get().BindCommands(OutCommandList, Client);
	if (UseVirtualTexturing(GMaxRHIShaderPlatform))
	{
		FVirtualTextureVisualizationMenuCommands::Get().BindCommands(OutCommandList, Client);
	}
}


void SLevelViewport::BindShowCommands( FUICommandList& OutCommandList )
{
	FLevelViewportCommands& LevelViewportCommands = FLevelViewportCommands::Get();

	OutCommandList.MapAction( 
		LevelViewportCommands.UseDefaultShowFlags,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnUseDefaultShowFlags, false ) );

	SEditorViewport::BindShowCommands( OutCommandList );

	// Show Volumes
	{
		// Map 'Show All' and 'Hide All' commands
		OutCommandList.MapAction(
			LevelViewportCommands.ShowAllVolumes,
			FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleAllVolumeActors, true ) );

		OutCommandList.MapAction(
			LevelViewportCommands.HideAllVolumes,
			FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleAllVolumeActors, false ) );
		

		LevelViewportCommands.RegisterShowVolumeCommands();
		const TArray<FLevelViewportCommands::FShowMenuCommand>& ShowVolumeCommands = LevelViewportCommands.ShowVolumeCommands;
		for (int32 VolumeCommandIndex = 0; VolumeCommandIndex < ShowVolumeCommands.Num(); ++VolumeCommandIndex)
		{
			OutCommandList.MapAction(
				ShowVolumeCommands[ VolumeCommandIndex ].ShowMenuItem,
				FExecuteAction::CreateSP( this, &SLevelViewport::ToggleShowVolumeClass, VolumeCommandIndex ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SLevelViewport::IsVolumeVisible, VolumeCommandIndex ) );
		}
	}

	// Show Layers
	{
		auto CanToggleAllLayers = [this]() { return !UWorld::IsPartitionedWorld(GetWorld()); };
		// Map 'Show All' and 'Hide All' commands
		OutCommandList.MapAction(
			LevelViewportCommands.ShowAllLayers,
			FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleAllLayers, true ),
			FCanExecuteAction::CreateLambda(CanToggleAllLayers));

		OutCommandList.MapAction(
			LevelViewportCommands.HideAllLayers,
			FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleAllLayers, false ),
			FCanExecuteAction::CreateLambda(CanToggleAllLayers));
	}
	
	// Show Sprite Categories
	{
		// Map 'Show All' and 'Hide All' commands
		OutCommandList.MapAction(
			LevelViewportCommands.ShowAllSprites,
			FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleAllSpriteCategories, true ) );

		OutCommandList.MapAction(
			LevelViewportCommands.HideAllSprites,
			FExecuteAction::CreateSP( this, &SLevelViewport::OnToggleAllSpriteCategories, false ) );

		// Bind each show flag to the same delegate.  We use the delegate payload system to figure out what show flag we are dealing with
		LevelViewportCommands.RegisterShowSpriteCommands();
		const TArray<FLevelViewportCommands::FShowMenuCommand>& ShowSpriteCommands = LevelViewportCommands.ShowSpriteCommands;
		for (int32 SpriteCommandIndex = 0; SpriteCommandIndex < ShowSpriteCommands.Num(); ++SpriteCommandIndex)
		{
			OutCommandList.MapAction(
				ShowSpriteCommands[SpriteCommandIndex].ShowMenuItem,
				FExecuteAction::CreateSP(this, &SLevelViewport::ToggleSpriteCategory, SpriteCommandIndex),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLevelViewport::IsSpriteCategoryVisible, SpriteCommandIndex));
		}
	}

	// Show Stat Categories
	{
#if STATS
		// Map 'Hide All' command
		OutCommandList.MapAction(
			LevelViewportCommands.HideAllStats,
			FExecuteAction::CreateSP(this, &SLevelViewport::OnToggleAllStatCommands, false));
#endif

		for (auto StatCatIt = LevelViewportCommands.ShowStatCatCommands.CreateConstIterator(); StatCatIt; ++StatCatIt)
		{
			const TArray< FLevelViewportCommands::FShowMenuCommand >& ShowStatCommands = StatCatIt.Value();
			for (int32 StatIndex = 0; StatIndex < ShowStatCommands.Num(); ++StatIndex)
			{
				const FLevelViewportCommands::FShowMenuCommand& StatCommand = ShowStatCommands[StatIndex];
				BindStatCommand(StatCommand.ShowMenuItem, StatCommand.LabelOverride.ToString());
			}
		}

		// Bind a listener here for any additional stat commands that get registered later.
		FLevelViewportCommands::NewStatCommandDelegate.AddRaw(this, &SLevelViewport::BindStatCommand);
	}
}

void SLevelViewport::BindDropCommands( FUICommandList& OutCommandList )
{
	OutCommandList.MapAction( 
		FLevelViewportCommands::Get().ApplyMaterialToActor,
		FExecuteAction::CreateSP( this, &SLevelViewport::OnApplyMaterialToViewportTarget ) );
}


void SLevelViewport::BindStatCommand(const TSharedPtr<FUICommandInfo> InMenuItem, const FString& InCommandName)
{
	CommandList->MapAction(
		InMenuItem,
		FExecuteAction::CreateSP(this, &SLevelViewport::ToggleStatCommand, InCommandName),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SLevelViewport::IsStatCommandVisible, InCommandName));
}

void SLevelViewport::BindPIECommands(FUICommandList& OutCommandList)
{
	OutCommandList.MapAction(
		FLevelViewportCommands::Get().ToggleViewportToolbar,
		FExecuteAction::CreateSP( this, &SLevelViewport::ToggleViewportToolbarVisibility ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SLevelViewport::IsViewportToolbarVisible )
		);

	OutCommandList.MapAction(FLevelViewportCommands::Get().UseDefaultShowFlags,
		FExecuteAction::CreateSP(this, &SLevelViewport::ResetPIEShowFlags)
	);

	for (const FShowFlagMenuCommands::FShowFlagCommand& FlagCommand : FShowFlagMenuCommands::Get().GetCommands())
	{
		OutCommandList.MapAction(FlagCommand.ShowMenuItem,
			FExecuteAction::CreateSP(this, &SLevelViewport::TogglePIEShowFlag, FlagCommand.FlagIndex),
			FCanExecuteAction::CreateStatic(&FEngineShowFlags::IsForceFlagSet, static_cast<uint32>(FlagCommand.FlagIndex)),
			FIsActionChecked::CreateSP(this, &SLevelViewport::IsPIEShowFlagEnabled, FlagCommand.FlagIndex)
		);
	}
	
	const FEditorViewportCommands& Commands = FEditorViewportCommands::Get();
	
	#define MAP_VIEWMODE_ACTION( ViewModeCommand, ViewModeID ) \
	OutCommandList.MapAction( \
		ViewModeCommand, \
		FExecuteAction::CreateSP( this, &SLevelViewport::SetPIEViewMode, ViewModeID ), \
		FCanExecuteAction(), \
		FIsActionChecked::CreateSP( this, &SLevelViewport::IsPIEViewModeEnabled, ViewModeID ) ) 
		
	// Map each view mode
	MAP_VIEWMODE_ACTION( Commands.WireframeMode, VMI_BrushWireframe );
	MAP_VIEWMODE_ACTION( Commands.UnlitMode, VMI_Unlit );
	MAP_VIEWMODE_ACTION( Commands.LitMode, VMI_Lit );
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MAP_VIEWMODE_ACTION( Commands.LitWireframeMode, VMI_Lit_Wireframe);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (IsRayTracingAllowed())
	{
		MAP_VIEWMODE_ACTION(Commands.PathTracingMode, VMI_PathTracing);
		MAP_VIEWMODE_ACTION(Commands.RayTracingDebugMode, VMI_RayTracingDebug);
	}

	MAP_VIEWMODE_ACTION( Commands.DetailLightingMode, VMI_Lit_DetailLighting );
	MAP_VIEWMODE_ACTION( Commands.LightingOnlyMode, VMI_LightingOnly );
	MAP_VIEWMODE_ACTION( Commands.LightComplexityMode, VMI_LightComplexity );
	MAP_VIEWMODE_ACTION( Commands.ShaderComplexityMode, VMI_ShaderComplexity );
	MAP_VIEWMODE_ACTION( Commands.QuadOverdrawMode, VMI_QuadOverdraw);
	MAP_VIEWMODE_ACTION( Commands.ShaderComplexityWithQuadOverdrawMode, VMI_ShaderComplexityWithQuadOverdraw );
	MAP_VIEWMODE_ACTION( Commands.TexStreamAccPrimitiveDistanceMode, VMI_PrimitiveDistanceAccuracy );
	MAP_VIEWMODE_ACTION( Commands.TexStreamAccMeshUVDensityMode, VMI_MeshUVDensityAccuracy);
	MAP_VIEWMODE_ACTION( Commands.TexStreamAccMaterialTextureScaleMode, VMI_MaterialTextureScaleAccuracy );
	MAP_VIEWMODE_ACTION( Commands.RequiredTextureResolutionMode, VMI_RequiredTextureResolution);
	MAP_VIEWMODE_ACTION( Commands.StationaryLightOverlapMode, VMI_StationaryLightOverlap );

	if (IsStaticLightingAllowed())
	{
		MAP_VIEWMODE_ACTION(Commands.LightmapDensityMode, VMI_LightmapDensity);
	}

	MAP_VIEWMODE_ACTION( Commands.ReflectionOverrideMode, VMI_ReflectionOverride );
	MAP_VIEWMODE_ACTION( Commands.GroupLODColorationMode, VMI_GroupLODColoration);
	MAP_VIEWMODE_ACTION( Commands.LODColorationMode, VMI_LODColoration );
	MAP_VIEWMODE_ACTION( Commands.HLODColorationMode, VMI_HLODColoration);
	MAP_VIEWMODE_ACTION( Commands.GroupStreamingDeficitMode, VMI_StreamingTextureDeficit);
	MAP_VIEWMODE_ACTION( Commands.StreamingTextureDeficitMode, VMI_StreamingTextureDeficit);
	MAP_VIEWMODE_ACTION( Commands.StreamingTextureResidencyMode, VMI_StreamingTextureResidency);
	MAP_VIEWMODE_ACTION( Commands.StreamingMeshLODDeficitMode, VMI_StreamingMeshLODDeficit);
	MAP_VIEWMODE_ACTION( Commands.StreamingMeshLODResidencyMode, VMI_StreamingMeshLODResidency);
	MAP_VIEWMODE_ACTION( Commands.VisualizeBufferMode, VMI_VisualizeBuffer );
	MAP_VIEWMODE_ACTION( Commands.VisualizeNaniteMode, VMI_VisualizeNanite );
	MAP_VIEWMODE_ACTION( Commands.VisualizeLumenMode, VMI_VisualizeLumen );
	MAP_VIEWMODE_ACTION( Commands.VisualizeMegaLightsMode, VMI_VisualizeMegaLights );
	MAP_VIEWMODE_ACTION( Commands.VisualizeSubstrateMode, VMI_VisualizeSubstrate);
	MAP_VIEWMODE_ACTION( Commands.VisualizeGroomMode, VMI_VisualizeGroom);
	MAP_VIEWMODE_ACTION( Commands.VisualizeVirtualShadowMapMode, VMI_VisualizeVirtualShadowMap );
	MAP_VIEWMODE_ACTION( Commands.VisualizeVirtualTextureMode, VMI_VisualizeVirtualTexture);
	MAP_VIEWMODE_ACTION( Commands.CollisionPawn, VMI_CollisionPawn);
	MAP_VIEWMODE_ACTION( Commands.CollisionVisibility, VMI_CollisionVisibility);

	MAP_VIEWMODE_ACTION( Commands.VisualizeLWCComplexity, VMI_LWCComplexity);
	
}

void SLevelViewport::ResetPIEShowFlags()
{
	if (UGameViewportClient* GameClient = GetPlayClient())
	{
		GameClient->EngineShowFlags = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		RefreshPIEViewport();
	}
}

void SLevelViewport::TogglePIEShowFlag(FEngineShowFlags::EShowFlag Flag)
{
	if (UGameViewportClient* GameClient = GetPlayClient())
	{
		GameClient->GetEngineShowFlags()->SetSingleFlag(Flag, !GameClient->EngineShowFlags.GetSingleFlag(Flag));
		RefreshPIEViewport();
	}
}

bool SLevelViewport::IsPIEShowFlagEnabled(FEngineShowFlags::EShowFlag Flag) const
{
	if (UGameViewportClient* GameClient = GetPlayClient())
	{
		return GameClient->GetEngineShowFlags()->GetSingleFlag(Flag);
	}
	return false;
}

void SLevelViewport::SetPIEViewMode(EViewModeIndex ViewMode)
{
	if (UGameViewportClient* GameClient = GetPlayClient())
	{
		GameClient->SetViewMode(ViewMode);
		RefreshPIEViewport();
	}
}

bool SLevelViewport::IsPIEViewModeEnabled(EViewModeIndex ViewMode) const
{
	if (UGameViewportClient* GameClient = GetPlayClient())
	{
		return GameClient->ViewModeIndex == ViewMode;
	}
	return false;
}

void SLevelViewport::RefreshPIEViewport()
{
	if (UGameViewportClient* GameClient = GetPlayClient())
	{
		if (UWorld* World = GameClient->GetWorld())
		{
			FSlateThrottleManager::Get().DisableThrottle(true);
		
			FTimerHandle TimerHandle;
			World->GetTimerManager().SetTimer(TimerHandle, []
			{
				FSlateThrottleManager::Get().DisableThrottle(false);
			}, 0.1f, false);
		}
	}
}

TAutoConsoleVariable<int32> CVarLevelEditorPIEInputOverride(
	TEXT("LevelEditorPIEInputOverride"),
	1,
	TEXT("Enable/disable overriding PIE gampeplay input with level editor PIE shorctuts.")
);

bool SLevelViewport::OnPIEViewportInputOverride(FInputKeyEventArgs& Input)
{
	if (CVarLevelEditorPIEInputOverride.GetValueOnGameThread() == 0)
	{
		return false;
	}

	if (Input.Event != IE_Pressed)
	{
		return false;
	}
	
	FModifierKeysState Modifiers = FSlateApplication::Get().GetModifierKeys();
	FInputChord CheckChord(Input.Key, EModifierKey::FromBools(
		Modifiers.IsControlDown(),
		Modifiers.IsAltDown(),
		Modifiers.IsShiftDown(),
		Modifiers.IsCommandDown()
	));
	
	// This function will completely bypass any game input.
	// The commands that can do this should be limited and carefully considered.
	const TSharedPtr<FUICommandInfo>& Command = FLevelViewportCommands::Get().ToggleViewportToolbar;	
	if (Command->HasActiveChord(CheckChord))
	{
		if (const FUIAction* Action = PIECommands->GetActionForCommand(Command))
		{
			return Action->Execute();
		}
	}
	
	return false;
}

const FSlateBrush* SLevelViewport::OnGetViewportBorderBrush() const
{
	const FSlateBrush* BorderBrush = nullptr;
	if( FSlateApplication::Get().IsNormalExecution() )
	{
		// If a PIE/SIE/Editor transition just completed, then we'll draw a border effect to draw attention to it
		if( ViewTransitionAnim.IsPlaying() )
		{
			switch( ViewTransitionType )
			{
				case EViewTransition::FadingIn:
					BorderBrush = BlackBackground;
					break;

				case EViewTransition::StartingPlayInEditor:
					BorderBrush = StartingPlayInEditorBorder;
					break;

				case EViewTransition::StartingSimulate:
					BorderBrush = StartingSimulateBorder;
					break;

				case EViewTransition::ReturningToEditor:
					BorderBrush = ReturningToEditorBorder;
					break;
			}
		}
		else if(!IsMaximized())
		{
			BorderBrush = NonMaximizedBorder;
		}
	}
	else
	{
		BorderBrush = DebuggingBorder;
	}

	return BorderBrush;
}

EVisibility SLevelViewport::OnGetFocusedViewportIndicatorVisibility() const
{
	EVisibility BaseVisibility = OnGetViewportContentVisibility();
	if (BaseVisibility != EVisibility::Collapsed)
	{
		// Only show the active border if we have a valid client, its the current client being edited and we arent in immersive (in immersive there is only one visible viewport)
		if (LevelViewportClient.IsValid() && LevelViewportClient.Get() == GCurrentLevelEditingViewportClient && !IsImmersive())
		{
			return EVisibility::HitTestInvisible;
		}
		else
		{
			return EVisibility::Collapsed;
		}
	}

	return BaseVisibility;
}

FSlateColor SLevelViewport::OnGetViewportBorderColorAndOpacity() const
{
	FLinearColor ViewportBorderColorAndOpacity = FLinearColor::White;
	if( FSlateApplication::Get().IsNormalExecution() )
	{
		if( ViewTransitionAnim.IsPlaying() )
		{
			ViewportBorderColorAndOpacity = FLinearColor( 1.0f, 1.0f, 1.0f, 1.0f - ViewTransitionAnim.GetLerp() );
		}
	}
	return ViewportBorderColorAndOpacity;
}

EVisibility SLevelViewport::OnGetViewportContentVisibility() const
{
	// Do not show any of the viewports inner slate content (active viewport borders, etc) when we are playing in editor and in immersive mode
	// as they are meaningless in that situation
	EVisibility BaseVisibility = SEditorViewport::OnGetViewportContentVisibility();
	if (BaseVisibility != EVisibility::Visible)
	{
		return BaseVisibility;
	}

	return ( ( IsPlayInEditorViewportActive() && IsImmersive() ) || GEngine->IsStereoscopic3D( ActiveViewport.Get() ) ) ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SLevelViewport::GetToolBarVisibility() const
{
	if (GEngine->IsStereoscopic3D(ActiveViewport.Get()))
	{
		// Never show the toolbar in VR
		return EVisibility::Collapsed;
	}

	return IsViewportToolbarVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SLevelViewport::GetMaximizeToggleVisibility() const
{
	bool bIsMaximizeSupported = false;
	bool bShowMaximizeToggle = false;
	TSharedPtr<FLevelViewportLayout> LayoutPinned = ParentLayout.Pin();
	if (LayoutPinned.IsValid())
	{
		bIsMaximizeSupported = LayoutPinned->IsMaximizeSupported();
		bShowMaximizeToggle = !LayoutPinned->IsTransitioning();
	}

	// Do not show the maximize/minimize toggle when in immersive mode
	return (!bIsMaximizeSupported || IsImmersive()) ? EVisibility::Collapsed : (bShowMaximizeToggle ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility SLevelViewport::GetCloseImmersiveButtonVisibility() const
{
	// Do not show the Immersive toggle button when not in immersive mode
	return IsImmersive() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SLevelViewport::GetTransformToolbarVisibility() const
{ 
	if (IsActiveLevelViewport())
	{
		// Only return visible if we are/were the active viewport. 
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

bool SLevelViewport::IsMaximized() const
{
	if( ParentLayout.IsValid() && !ConfigKey.IsNone())
	{
		return ParentLayout.Pin()->IsViewportMaximized( ConfigKey );
	}

	// Assume the viewport is always maximized if we have no layout for some reason
	return true;
}

bool SLevelViewport::CanMaximize() const
{
	if (TSharedPtr<FLevelViewportLayout> PinnedParentLayout = ParentLayout.Pin())
	{
		return PinnedParentLayout->IsMaximizeSupported();
	}
	return false;
}

TSharedRef<FEditorViewportClient> SLevelViewport::MakeEditorViewportClient() 
{
	return LevelViewportClient.ToSharedRef();
}

FName SLevelViewport::RegisterViewportToolbar() const
{
	const FName ViewportToolbarName = "LevelEditor.ViewportToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
			ViewportToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
		);

		ViewportToolbarMenu->StyleName = "ViewportToolbar";
		ViewportToolbarMenu->bSeparateSections = false;

		// Add the left-aligned part of the viewport toolbar.
		{
			FToolMenuSection& LeftSection = ViewportToolbarMenu->AddSection("Left");

			LeftSection.AddEntry(UE::UnrealEd::CreateTransformsSubmenu());
			UE::LevelEditor::ExtendTransformSubmenu("LevelEditor.ViewportToolbar.Transform");

			LeftSection.AddEntry(UE::UnrealEd::CreateSnappingSubmenu());
			UE::LevelEditor::ExtendSnappingSubmenu("LevelEditor.ViewportToolbar.Snapping");
		}

		// Add the right-aligned part of the viewport toolbar.
		{
			FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			// Add the "Camera" submenu.
			{
				// This matches the name used in SLevelViewportToolBar::GenerateCameraMenu for
				// the old level editor viewport toolbar camera menu.
				const FName ParentSubmenuName = "LevelEditor.LevelViewportToolBar.Camera";

				// Create our parent menu.
				if (!UToolMenus::Get()->IsMenuRegistered(ParentSubmenuName))
				{
					UToolMenus::Get()->RegisterMenu(ParentSubmenuName);
				}

				// Build the menu name our Camera menu will be using so we can extend it and set
				// the old viewport toolbar camera menu as our parent.
				const FName SubmenuName = UToolMenus::JoinMenuPaths(ViewportToolbarName, "Camera");

				// Register our submenu explicitly so we can set the old viewport toolbar's camera menu as our parent.
				UToolMenus::Get()->RegisterMenu(SubmenuName, ParentSubmenuName);

				// Extending using both Level Editor specific and UnrealEd generic entries
				RightSection.AddEntry(UE::LevelEditor::CreateToolbarCameraSubmenu());
				UE::LevelEditor::ExtendCameraSubmenu(SubmenuName);
			}

			// Add the "View Modes" sub menu.
			{
				// Stay backward-compatible with the old viewport toolbar.
				{
					// Create our grandparent menu.
					if (!UToolMenus::Get()->IsMenuRegistered("UnrealEd.ViewportToolbar.View"))
					{
						UToolMenus::Get()->RegisterMenu("UnrealEd.ViewportToolbar.View");
					}

					// Create our parent menu.
					if (!UToolMenus::Get()->IsMenuRegistered("LevelEditor.LevelViewportToolbar.View"))
					{
						UToolMenus::Get()->RegisterMenu(
							"LevelEditor.LevelViewportToolbar.View", "UnrealEd.ViewportToolbar.View"
						);
					}

					// Create our menu.
					UToolMenus::Get()->RegisterMenu(
						"LevelEditor.ViewportToolbar.ViewModes", "LevelEditor.LevelViewportToolbar.View"
					);
				}

				// Add the level editor specific entries.
				UE::LevelEditor::ExtendViewModesSubmenu("LevelEditor.ViewportToolbar.ViewModes");

				// Create and add the submenu entry to make the menu we just created and extended appear in the
				// viewport toolbar itself.
				RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			}

			{
				RightSection.AddEntry(UE::LevelEditor::CreateDynamicChannelsSubmenu());
			}

			// Add the "Show" submenu.
			{
				// Stay backward-compatible with the old viewport toolbar.
				{
					if (!UToolMenus::Get()->IsMenuRegistered("LevelEditor.LevelViewportToolbar.Show"))
					{
						UToolMenus::Get()->RegisterMenu("LevelEditor.LevelViewportToolbar.Show");
					}
					UToolMenus::Get()->RegisterMenu(
						"LevelEditor.ViewportToolbar.Show", "LevelEditor.LevelViewportToolbar.Show"
					);
				}

				RightSection.AddEntry(UE::LevelEditor::CreateShowSubmenu());
			}

			RightSection.AddEntry(UE::LevelEditor::CreatePerformanceAndScalabilitySubmenu());
			RightSection.AddEntry(UE::LevelEditor::CreateSettingsSubmenu());
		}

		// Sections added for compatibility with old toolbar transform menu extensions
		{
			// Existing code might add entries via FToolBarBuilder Extenders,
			// These extenders are based off of toolbar section & extension names.
			// The following sections names are used in STransformViewportToolBar::MakeTransformToolBar(...),
			// This allows legacy extensions to be added directly to the toolbar.

			ViewportToolbarMenu->AddSection("Transform").Alignment = EToolMenuSectionAlign::Last;
			ViewportToolbarMenu->AddSection("LocationGridSnap").Alignment = EToolMenuSectionAlign::Last;
			ViewportToolbarMenu->AddSection("RotationGridSnap").Alignment = EToolMenuSectionAlign::Last;
			ViewportToolbarMenu->AddSection("Layer2DSnap").Alignment = EToolMenuSectionAlign::Last;
			ViewportToolbarMenu->AddSection("ScaleGridSnap").Alignment = EToolMenuSectionAlign::Last;
			ViewportToolbarMenu->AddSection("CameraSpeed").Alignment = EToolMenuSectionAlign::Last;
			
			// TODO: Remove these once the deprecation period is complete
			FToolMenuSection& FallbackSection = ViewportToolbarMenu->AddSection("Fallback");
			FallbackSection.Alignment = EToolMenuSectionAlign::Last;
			
			FallbackSection.AddDynamicEntry("FallbackExtensionPanels", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				ULegacyLevelViewportToolbarContext* Context = Section.FindContext<ULegacyLevelViewportToolbarContext>();
				if (!Context || !Context->LevelViewport.IsValid() || !Context->LevelViewportToolBarWidget.IsValid())
				{
					return;
				}
				
				UViewportToolBarContext* ExtensionContext = NewObject<UViewportToolBarContext>();
				ExtensionContext->Viewport = Context->LevelViewport;
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				ExtensionContext->ViewportToolBar = Context->LevelViewportToolBarWidget;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				
				auto MakeExtensionPanel = [ExtensionContext, &Section](FName ExtensionId)
				{
					FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitWidget(
						ExtensionId,
						SNew(SExtensionPanel)
							.ExtensionPanelID(ExtensionId)
							.ExtensionContext(ExtensionContext),
						FText::GetEmpty(),
						true, false, true
					));
					
					Entry.ToolBarData.ResizeParams.VisibleInOverflow = false;
				};
				
				MakeExtensionPanel("LevelViewportToolBar.LeftExtension");
				MakeExtensionPanel("LevelViewportToolBar.MiddleExtension");
				MakeExtensionPanel("LevelViewportToolBar.RightExtension");
				MakeExtensionPanel("LevelViewportToolBar.RightmostExtension");
			}));
		}
		
		{
			// The viewport sizing entries should _always_ be the last section.
			// Doing otherwise breaks the customized behavior of the overflow menu.
			FToolMenuSection& SizingSection = ViewportToolbarMenu->AddSection("ViewportSizing");
			SizingSection.Alignment = EToolMenuSectionAlign::Last;
			SizingSection.AddEntry(UE::LevelEditor::CreateViewportSizingSubmenu());
		}
	}
	
	return ViewportToolbarName;	
}

TSharedPtr<SWidget> SLevelViewport::BuildViewportToolbar()
{
	FToolMenuContext ViewportToolbarContext;
	{
		ViewportToolbarContext.AppendCommandList(GetCommandList());
		// Note that these extenders can now leak between submenus of the viewport toolbar.
		{
			// Stay backward-compatible with legacy view menu extenders.
			ViewportToolbarContext.AddExtender(UE::LevelEditor::GetViewModesLegacyExtenders());

			FLevelEditorModule& LevelEditorModule =
				FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

			TSharedRef<FUICommandList> CommandListRef = GetCommandList().ToSharedRef();

			// Stay backward-compatible with legacy show menu extenders.
			{
				TSharedPtr<FExtender> Extenders = LevelEditorModule.AssembleExtenders(
					CommandListRef, LevelEditorModule.GetAllLevelViewportShowMenuExtenders()
				);
				ViewportToolbarContext.AddExtender(Extenders);
			}

			// Stay backward-compatible with legacy view menu extenders
			{
				TSharedPtr<FExtender> ViewMenuExtenders = LevelEditorModule.AssembleExtenders(
					CommandListRef, LevelEditorModule.GetAllLevelEditorToolbarViewMenuExtenders()
				);
				ViewportToolbarContext.AddExtender(ViewMenuExtenders);
			}

			// Stay backward-compatible with legacy options menu extenders
			{
				TSharedPtr<FExtender> ViewMenuExtenders = LevelEditorModule.AssembleExtenders(
					CommandListRef, LevelEditorModule.GetAllLevelViewportOptionsMenuExtenders()
				);

				if (ViewMenuExtenders.IsValid())
				{
					ViewportToolbarContext.AddExtender(ViewMenuExtenders);
				}
			}

			// Stay backward-compatible with legacy toolbar menu extenders
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				TSharedPtr<FExtender> OldToolbarExtenders =
					LevelEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				if (OldToolbarExtenders.IsValid())
				{
					ViewportToolbarContext.AddExtender(OldToolbarExtenders);
				}
			}
		}

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject = NewObject<UUnrealEdViewportToolbarContext>();
			ContextObject->Viewport = SharedThis(this);
			ViewportToolbarContext.AddObject(ContextObject);
		}

		// Add the level editor viewport toolbar context.
		{
			ULevelViewportContext* const ContextObject = NewObject<ULevelViewportContext>();
			ContextObject->LevelViewport = SharedThis(this);
			ViewportToolbarContext.AddObject(ContextObject);
		}
		
		// Support legacy SExtensionPanel extensions
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LevelViewportToolBar = SNew(SLevelViewportToolBar);
		// Provide a non-null toolbar widget so that legacy extensions with bad null handling don't crash
		ULegacyLevelViewportToolbarContext* const LegacyContext = NewObject<ULegacyLevelViewportToolbarContext>();
		LegacyContext->LevelViewport = SharedThis(this);
		LegacyContext->LevelViewportToolBarWidget = LevelViewportToolBar;
		ViewportToolbarContext.AddObject(LegacyContext);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		ExtendToolbarContext(ViewportToolbarContext);
	}

	// clang-format off
	return SNew(SWidgetSwitcher)
	.WidgetIndex(this, &SLevelViewport::GetViewportToolbarIndex)
	.Visibility(this, &SLevelViewport::GetToolBarVisibility)
	+ SWidgetSwitcher::Slot()
	[
		UToolMenus::Get()->GenerateWidget(RegisterViewportToolbar(), ViewportToolbarContext)
	]
	+ SWidgetSwitcher::Slot()
	[
		BuildPIEViewportToolbar()
	];
	// clang-format on
}

TSharedRef<SWidget> SLevelViewport::BuildPIEViewportToolbar()
{
	const FName ToolbarName = "LevelEditor.PIEViewportToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		
		ToolbarMenu->StyleName = "ViewportToolbar";
		ToolbarMenu->bSeparateSections = false;
		
		{
			FToolMenuSection& LeftSection = ToolbarMenu->AddSection("Left");
			LeftSection.AddEntry(FToolMenuEntry::InitDynamicEntry(
				"DynamicPIEDetachHint",
				FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
				{
					TSharedPtr<SLevelViewport> Viewport = ULevelViewportContext::GetLevelViewport(Section);
					if (!Viewport)
					{
						return;
					}
					
					Section.AddEntry(FToolMenuEntry::InitWidget(
						"PIEDetachHint",
						SNew(SBox)
						.Padding(FMargin(8.f, 0.f, 0.f, 0.f))
						.Visibility_Lambda([WeakViewport = Viewport.ToWeakPtr()]
						{
							const ULevelEditorPlaySettings* EditorPlayInSettings = GetDefault<ULevelEditorPlaySettings>();
							if (!EditorPlayInSettings->ShowMouseControlLabel)
							{
								return EVisibility::Collapsed;
							}
						
							if (TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin())
							{
								return Viewport->GetActiveViewport()->HasMouseCapture() ? EVisibility::Visible : EVisibility::Collapsed;
							}
							return EVisibility::Collapsed;
						})
						[
							Viewport->BuildMouseCaptureWidget()
						],
						FText::GetEmpty()
					));
				})
			));
		}
		
		{
			FToolMenuSection& RightSection = ToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;
			
			RightSection.AddEntry(UE::LevelEditor::CreatePIEViewModesSubmenu());
			RightSection.AddEntry(UE::LevelEditor::CreatePIEShowSubmenu());
			RightSection.AddEntry(UE::LevelEditor::CreatePIEPerformanceAndScalabilitySubmenu());
			RightSection.AddEntry(UE::LevelEditor::CreatePIESettingsSubmenu());
		}
	}
	
	FToolMenuContext ToolbarContext;
	{
		ToolbarContext.AppendCommandList(PIECommands);
		ToolbarContext.AppendCommandList(GetCommandList());
		
		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject = NewObject<UUnrealEdViewportToolbarContext>();
			ContextObject->Viewport = SharedThis(this);
			ToolbarContext.AddObject(ContextObject);
			
			ContextObject->ExcludedShowMenuFlags.Append({
				FEngineShowFlags::SF_Cameras,
				FEngineShowFlags::SF_CameraFrustums,
				FEngineShowFlags::SF_Grid
			});
			
			ContextObject->IsViewModeSupported.BindLambda([](EViewModeIndex ViewMode)
			{
				switch (ViewMode)
				{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				case VMI_Lit_Wireframe:
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				case VMI_Clay:
				case VMI_Zebra:
				case VMI_FrontBackFace:
				case VMI_RandomColor:
					return false;
				default:
					return true;
				}
			});
			
			ContextObject->DoesViewModeMenuShowSection.BindLambda([](UE::UnrealEd::EHidableViewModeMenuSections InMenuSection)
			{
				switch (InMenuSection)
				{
				case UE::UnrealEd::EHidableViewModeMenuSections::GPUSkinCache:
					return false;
				default:
					return true;
				}
			});
		}

		// Add the level editor viewport toolbar context.
		{
			ULevelViewportContext* const ContextObject = NewObject<ULevelViewportContext>();
			ContextObject->LevelViewport = SharedThis(this);
			ToolbarContext.AddObject(ContextObject);
		}
	}
	
	return UToolMenus::Get()->GenerateWidget(ToolbarName, ToolbarContext);
}

void SLevelViewport::OnUndo()
{
	GUnrealEd->Exec( GetWorld(), TEXT("TRANSACTION UNDO") );
}

void SLevelViewport::OnRedo()
{
	GUnrealEd->Exec(  GetWorld(), TEXT("TRANSACTION REDO") );
}

bool SLevelViewport::CanExecuteUndo() const
{
	return GUnrealEd->Trans->CanUndo() && FSlateApplication::Get().IsNormalExecution();
}

bool SLevelViewport::CanExecuteRedo() const
{
	return GUnrealEd->Trans->CanRedo() && FSlateApplication::Get().IsNormalExecution();
}

int32 SLevelViewport::GetViewportToolbarIndex() const
{
	return IsPlayInEditorViewportActive() ? 1 : 0;
}

void SLevelViewport::OnAdvancedSettings()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "LevelEditor", "Viewport");
}

void SLevelViewport::OnPlaySettings()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "LevelEditor", "PlayIn");
}

void SLevelViewport::OnToggleImmersive()
{
	const bool bWantImmersive = !IsImmersive();
	// We always want to animate in response to user-interactive toggling of maximized state
	constexpr bool bAllowAnimation = true;

	MakeImmersive(bWantImmersive, bAllowAnimation);
}

void SLevelViewport::OnToggleSidebarTabs()
{
	static const TSet<FTabId> Exceptions = { FTabId(TEXT("Sequencer")) };
	ParentLevelEditor.Pin()->GetTabManager()->ToggleSidebarOpenTabs(Exceptions);
}

bool SLevelViewport::IsImmersive() const
{
	if( ParentLayout.IsValid() && !ConfigKey.IsNone())
	{
		return ParentLayout.Pin()->IsViewportImmersive( ConfigKey );
	}

	// Assume the viewport is not immersive if we have no layout for some reason
	return false;
}

void SLevelViewport::OnCreateCameraActor(UClass* InClass)
{		
	// Find the perspective viewport we were using
	FViewport* pViewPort = GEditor->GetActiveViewport();
	FLevelEditorViewportClient* ViewportClient = nullptr;
	for( FLevelEditorViewportClient* LevelViewport : GEditor->GetLevelViewportClients())
	{		
		if( LevelViewport->IsPerspective() && LevelViewport->Viewport == pViewPort )
		{
			ViewportClient = LevelViewport;
			break;
		}
	}

	if( ViewportClient == nullptr )
	{
		// May fail to find viewport if shortcut key was pressed on an ortho viewport, if so early out.
		// This function only works on perspective viewports so new camera can match perspective camera.
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("LevelViewport", "CreateCameraHere", "Create Camera Here"));

	// Set new camera to match viewport
	ACameraActor* pNewCamera = Cast<ACameraActor>(ViewportClient->GetWorld()->SpawnActor(InClass));
	pNewCamera->SetActorLocation( ViewportClient->GetViewLocation(), false );
	pNewCamera->SetActorRotation( ViewportClient->GetViewRotation() );
	pNewCamera->GetCameraComponent()->SetFieldOfView( ViewportClient->ViewFOV );

	// Deselect any currently selected actors, then select newly created camera
	UE::SLevelViewport::Internal::SelectActor(pNewCamera);

	// Send notification about actors that may have changed
	ULevel::LevelDirtiedEvent.Broadcast();

	// Redraw viewports to show new camera
	GEditor->RedrawAllViewports();
}

bool SLevelViewport::IsActiveLevelViewport() const
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorName);
	return LevelEditorModule.GetFirstActiveViewport().Get() == this;
}

bool SLevelViewport::IsPerspectiveViewport() const
{
	bool bIsPerspective = false;
	FViewport* pViewPort = GEditor->GetActiveViewport();
	if( pViewPort && pViewPort->GetClient()->IsOrtho() == false )
	{
		bIsPerspective = true;
	}
	return bIsPerspective;
}

void SLevelViewport::OnTakeHighResScreenshot()
{
	HighResScreenshotDialog = SHighResScreenshotDialog::OpenDialog(ActiveViewport, CaptureRegionWidget);
}

void SLevelViewport::ToggleGameView()
{
	bool bGameViewEnable = !LevelViewportClient->IsInGameView();

	// "Mode Widget" should not automatically be reactivated by selecting an actor after "Game View" is enabled
	LevelViewportClient->bAlwaysShowModeWidgetAfterSelectionChanges = bGameViewEnable ? false : true;

	LevelViewportClient->SetGameView(bGameViewEnable);

	if (!bGameViewEnable)
	{
		// LevelViewportClient->bShowWidget is set to "false" when entering game mode
		// Need to turn it back to "true" when exiting game mode
		LevelViewportClient->ShowWidget(true);
	}
}

bool SLevelViewport::CanToggleGameView() const
{
	return LevelViewportClient->IsPerspective();
}

bool SLevelViewport::IsInGameView() const
{
	return LevelViewportClient->IsInGameView();
}

void SLevelViewport::OnToggleAllVolumeActors( bool bVisible )
{
	// Reinitialize the volume actor visibility flags to the new state.  All volumes should be visible if "Show All" was selected and hidden if it was not selected.
	LevelViewportClient->VolumeActorVisibility.Init( bVisible, LevelViewportClient->VolumeActorVisibility.Num() );

	// Update visibility based on the new state
	// All volume actor types should be taken since the user clicked on show or hide all to get here
	GUnrealEd->UpdateVolumeActorVisibility( nullptr, LevelViewportClient.Get() );
}

/** Called when the user toggles a volume visibility from Volumes sub-menu. **/
void SLevelViewport::ToggleShowVolumeClass( int32 VolumeID )
{
	TArray< UClass* > VolumeClasses;
	UUnrealEdEngine::GetSortedVolumeClasses(&VolumeClasses);

	// Get the corresponding volume class for the clicked menu item.
	UClass *SelectedVolumeClass = VolumeClasses[ VolumeID ];

	LevelViewportClient->VolumeActorVisibility[ VolumeID ] = !LevelViewportClient->VolumeActorVisibility[ VolumeID ];

	// Update the found actors visibility based on the new bitfield
	GUnrealEd->UpdateVolumeActorVisibility( SelectedVolumeClass, LevelViewportClient.Get() );
}

/** Called to determine if vlume class is visible. **/
bool SLevelViewport::IsVolumeVisible( int32 VolumeID ) const
{
	return LevelViewportClient->VolumeActorVisibility[ VolumeID ];
}

/** Called when a user selects show or hide all from the layers visibility menu. **/
void SLevelViewport::OnToggleAllLayers( bool bVisible )
{
	ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	if (bVisible)
	{
		// clear all hidden layers
		LevelViewportClient->ViewHiddenLayers.Empty();
	}
	else
	{
		// hide them all
		TArray<FName> AllLayerNames;
		Layers->AddAllLayerNamesTo(AllLayerNames);
		LevelViewportClient->ViewHiddenLayers = AllLayerNames;
	}

	// update actor visibility for this view
	Layers->UpdatePerViewVisibility(LevelViewportClient.Get());

	LevelViewportClient->Invalidate(); 
}

/** Called when the user toggles a layer from Layers sub-menu. **/
void SLevelViewport::ToggleShowLayer( FName LayerName )
{
	int32 HiddenIndex = LevelViewportClient->ViewHiddenLayers.Find(LayerName);
	if ( HiddenIndex == INDEX_NONE )
	{
		LevelViewportClient->ViewHiddenLayers.Add(LayerName);
	}
	else
	{
		LevelViewportClient->ViewHiddenLayers.RemoveAt(HiddenIndex);
	}

	// update actor visibility for this view
	ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	Layers->UpdatePerViewVisibility(LevelViewportClient.Get(), LayerName);

	LevelViewportClient->Invalidate(); 
}

/** Called to determine if a layer is visible. **/
bool SLevelViewport::IsLayerVisible( FName LayerName ) const
{
	return LevelViewportClient->ViewHiddenLayers.Find(LayerName) == INDEX_NONE;
}

void SLevelViewport::ToggleShowFoliageType(TWeakObjectPtr<UFoliageType> InFoliageType)
{
	UFoliageType* FoliageType = InFoliageType.Get();
	if (FoliageType)
	{
		FoliageType->HiddenEditorViews^= (1ull << LevelViewportClient->ViewIndex);
		// Notify UFoliageType that things have changed
		FoliageType->OnHiddenEditorViewMaskChanged(GetWorld());
	
		// Make sure to redraw viewport when user toggles foliage
		LevelViewportClient->Invalidate();
	}
}

void SLevelViewport::ToggleAllFoliageTypes(bool bVisible)
{
	UWorld* CurrentWorld = GetWorld(); 
	TArray<UFoliageType*> AllFoliageTypes = GEditor->GetFoliageTypesInWorld(CurrentWorld);
	if (AllFoliageTypes.Num())
	{
		const uint64 ViewMask = (1ull << LevelViewportClient->ViewIndex);

		for (UFoliageType* FoliageType  : AllFoliageTypes)
		{
			if (bVisible)
			{
				FoliageType->HiddenEditorViews&= ~ViewMask;
			}
			else
			{
				FoliageType->HiddenEditorViews|= ViewMask;
			}
			
			FoliageType->OnHiddenEditorViewMaskChanged(CurrentWorld);
		}
		
		// Make sure to redraw viewport when user toggles meshes
		LevelViewportClient->Invalidate(); 
	}
}

bool SLevelViewport::IsFoliageTypeVisible(TWeakObjectPtr<UFoliageType> InFoliageType) const
{
	const UFoliageType* FoliageType = InFoliageType.Get();
	if (FoliageType)
	{
		return (FoliageType->HiddenEditorViews & (1ull << LevelViewportClient->ViewIndex)) == 0;
	}
	return false;
}

FViewport* SLevelViewport::GetActiveViewport() 
{ 
	return ActiveViewport.Get();
}

void SLevelViewport::OnFocusViewportToSelection() 
{
	GUnrealEd->Exec( GetWorld(), TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY") );
}

/** Called when the user selects show or hide all from the sprite sub-menu. **/
void SLevelViewport::OnToggleAllSpriteCategories( bool bVisible )
{
	LevelViewportClient->SetAllSpriteCategoryVisibility( bVisible );
	LevelViewportClient->Invalidate();
}

/** Called when the user toggles a category from the sprite sub-menu. **/
void SLevelViewport::ToggleSpriteCategory( int32 CategoryID )
{
	LevelViewportClient->SetSpriteCategoryVisibility( CategoryID, !LevelViewportClient->GetSpriteCategoryVisibility( CategoryID ) );
	LevelViewportClient->Invalidate();
}

/** Called to determine if a category from the sprite sub-menu is visible. **/
bool SLevelViewport::IsSpriteCategoryVisible( int32 CategoryID ) const
{
	return LevelViewportClient->GetSpriteCategoryVisibility( CategoryID );
}

void SLevelViewport::OnToggleAllStatCommands( bool bVisible )
{
	check(bVisible == 0);
	// If it's in the array, it's visible so just toggle it again
	const TArray<FString>* EnabledStats = LevelViewportClient->GetEnabledStats();
	check(EnabledStats);
	while (EnabledStats->Num() > 0)
	{
		const FString& CommandName = EnabledStats->Last();
		ToggleStatCommand(CommandName);
	}
}

void SLevelViewport::OnUseDefaultShowFlags(bool bUseSavedDefaults)
{
	// cache off the current viewmode as it gets trashed when applying FEngineShowFlags()
	const EViewModeIndex CachedViewMode = LevelViewportClient->GetViewMode();

	// Setting show flags to the defaults should not stomp on the current viewmode settings.
	LevelViewportClient->SetGameView(false);

	// Get default save flags
	FEngineShowFlags EditorShowFlags(ESFIM_Editor);
	FEngineShowFlags GameShowFlags(ESFIM_Game);

	if (bUseSavedDefaults && !ConfigKey.IsNone())
	{
		FLevelEditorViewportInstanceSettings ViewportInstanceSettings;
		ViewportInstanceSettings.ViewportType = LevelViewportClient->ViewportType;

		// Get saved defaults if specified
		FString ConfigKeyAsString = ConfigKey.ToString();
		const FLevelEditorViewportInstanceSettings* const ViewportInstanceSettingsPtr = GetDefault<ULevelEditorViewportSettings>()->GetViewportInstanceSettings(ConfigKeyAsString);
		ViewportInstanceSettings = ViewportInstanceSettingsPtr ? *ViewportInstanceSettingsPtr : LoadLegacyConfigFromIni(ConfigKeyAsString, ViewportInstanceSettings);

		if (!ViewportInstanceSettings.EditorShowFlagsString.IsEmpty())
		{
			EditorShowFlags.SetFromString(*ViewportInstanceSettings.EditorShowFlagsString);
		}

		if (!ViewportInstanceSettings.GameShowFlagsString.IsEmpty())
		{
			GameShowFlags.SetFromString(*ViewportInstanceSettings.GameShowFlagsString);
		}
	}

	// this trashes the current viewmode!
	LevelViewportClient->EngineShowFlags = EditorShowFlags;
	// Restore the state of SelectionOutline based on user settings
	LevelViewportClient->EngineShowFlags.SetSelectionOutline(GetDefault<ULevelEditorViewportSettings>()->bUseSelectionOutline);
	LevelViewportClient->LastEngineShowFlags = GameShowFlags;

	// re-apply the cached viewmode, as it was trashed with FEngineShowFlags()
	ApplyViewMode(CachedViewMode, LevelViewportClient->IsPerspective(), LevelViewportClient->EngineShowFlags);
	ApplyViewMode(CachedViewMode, LevelViewportClient->IsPerspective(), LevelViewportClient->LastEngineShowFlags);

	// set volume / layer / sprite visibility defaults
	if (!bUseSavedDefaults)
	{
		LevelViewportClient->InitializeVisibilityFlags();
		GUnrealEd->UpdateVolumeActorVisibility(nullptr, LevelViewportClient.Get());
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		Layers->UpdatePerViewVisibility(LevelViewportClient.Get());
	}

	LevelViewportClient->Invalidate();
}


void SLevelViewport::SetKeyboardFocusToThisViewport()
{
	if( ensure( ViewportWidget.IsValid() ) )
	{
			// Set keyboard focus directly
		FSlateApplication::Get().SetKeyboardFocus( ViewportWidget.ToSharedRef() );
	}
}


void SLevelViewport::SaveConfig(const FString& ConfigName) const
{
	if(GUnrealEd && GetDefault<ULevelEditorViewportSettings>())
	{
		// When we startup the editor we always start it up in IsInGameView()=false mode
		FEngineShowFlags& EditorShowFlagsToSave = LevelViewportClient->IsInGameView() ? LevelViewportClient->LastEngineShowFlags : LevelViewportClient->EngineShowFlags;
		FEngineShowFlags& GameShowFlagsToSave = LevelViewportClient->IsInGameView() ? LevelViewportClient->EngineShowFlags : LevelViewportClient->LastEngineShowFlags;

		FLevelEditorViewportInstanceSettings ViewportInstanceSettings;
		if (const FLevelEditorViewportInstanceSettings* CurrentViewportInstanceSettingsPtr = GetDefault<ULevelEditorViewportSettings>()->GetViewportInstanceSettings(ConfigName))
		{
			ViewportInstanceSettings = *CurrentViewportInstanceSettingsPtr;
		}
		ViewportInstanceSettings.ViewportType = LevelViewportClient->ViewportType;
		ViewportInstanceSettings.PerspViewModeIndex = LevelViewportClient->GetPerspViewMode();
		ViewportInstanceSettings.OrthoViewModeIndex = LevelViewportClient->GetOrthoViewMode();
		ViewportInstanceSettings.EditorShowFlagsString = EditorShowFlagsToSave.ToString();
		ViewportInstanceSettings.GameShowFlagsString = GameShowFlagsToSave.ToString();
		ViewportInstanceSettings.BufferVisualizationMode = LevelViewportClient->CurrentBufferVisualizationMode;
		ViewportInstanceSettings.NaniteVisualizationMode = LevelViewportClient->CurrentNaniteVisualizationMode;
		ViewportInstanceSettings.LumenVisualizationMode = LevelViewportClient->CurrentLumenVisualizationMode;
		ViewportInstanceSettings.MegaLightsVisualizationMode = LevelViewportClient->CurrentMegaLightsVisualizationMode;
		ViewportInstanceSettings.SubstrateVisualizationMode = LevelViewportClient->CurrentSubstrateVisualizationMode;
		ViewportInstanceSettings.GroomVisualizationMode = LevelViewportClient->CurrentGroomVisualizationMode;
		ViewportInstanceSettings.VirtualShadowMapVisualizationMode = LevelViewportClient->CurrentVirtualShadowMapVisualizationMode;
		ViewportInstanceSettings.VirtualTextureVisualizationMode = LevelViewportClient->CurrentVirtualTextureVisualizationMode;
		ViewportInstanceSettings.RayTracingDebugVisualizationMode = LevelViewportClient->CurrentRayTracingDebugVisualizationMode;
		ViewportInstanceSettings.GPUSkinCacheVisualizationMode = LevelViewportClient->CurrentGPUSkinCacheVisualizationMode;
		ViewportInstanceSettings.ColorChannelMask = (int32)LevelViewportClient->ChannelMaskParams.ColorChannelMask;
		ViewportInstanceSettings.bInvertAlphaChannelMask = LevelViewportClient->ChannelMaskParams.bInvertAlphaChannelMask;
		ViewportInstanceSettings.bDrawAlphaBlendedCheckerboard = LevelViewportClient->ChannelMaskParams.bDrawAlphaBlendedCheckerboard;
		ViewportInstanceSettings.ExposureSettings = LevelViewportClient->ExposureSettings;
		ViewportInstanceSettings.FOVAngle = LevelViewportClient->FOVAngle;
		ViewportInstanceSettings.bAllowCinematicControl = LevelViewportClient->AllowsCinematicControl();
	
		LevelViewportClient->SaveRealtimeStateToConfig(ViewportInstanceSettings.bIsRealtime);
	
		ViewportInstanceSettings.bShowOnScreenStats = LevelViewportClient->ShouldShowStats();
		ViewportInstanceSettings.FarViewPlane = LevelViewportClient->GetFarClipPlaneOverride();
		ViewportInstanceSettings.bShowFullToolbar = bShowEditorToolbar;

		if(GetDefault<ULevelEditorViewportSettings>()->bSaveEngineStats)
		{
			const TArray<FString>* EnabledStats = nullptr;

			// If the selected viewport is currently hosting a PIE session, we need to make sure we copy to stats from the active viewport
			// Note: This happens if you close the editor while it's running because SwapStatCommands gets called after the config save when shutting down.
			if(IsPlayInEditorViewportActive())
			{
				EnabledStats = ActiveViewport->GetClient()->GetEnabledStats();
			}
			else
			{
				EnabledStats = LevelViewportClient->GetEnabledStats();
			}

			check(EnabledStats);
			ViewportInstanceSettings.EnabledStats = *EnabledStats;
		}
		GetMutableDefault<ULevelEditorViewportSettings>()->SetViewportInstanceSettings(ConfigName, ViewportInstanceSettings);
	}
	
}


FLevelEditorViewportInstanceSettings SLevelViewport::LoadLegacyConfigFromIni(const FString& InConfigKey, const FLevelEditorViewportInstanceSettings& InDefaultSettings)
{
	FLevelEditorViewportInstanceSettings ViewportInstanceSettings = InDefaultSettings;

	const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

	{
		int32 ViewportTypeAsInt = ViewportInstanceSettings.ViewportType;
		GConfig->GetInt(*IniSection, *(InConfigKey + TEXT(".Type")), ViewportTypeAsInt, GEditorPerProjectIni);
		ViewportInstanceSettings.ViewportType = (ViewportTypeAsInt == -1 || ViewportTypeAsInt == 255) ? LVT_None : static_cast<ELevelViewportType>(ViewportTypeAsInt); // LVT_None used to be -1 or 255

		if(ViewportInstanceSettings.ViewportType == LVT_None)
		{
			ViewportInstanceSettings.ViewportType = LVT_Perspective;
		}
	}
	
	GConfig->GetString(*IniSection, *(InConfigKey + TEXT(".EditorShowFlags")), ViewportInstanceSettings.EditorShowFlagsString, GEditorPerProjectIni);
	GConfig->GetString(*IniSection, *(InConfigKey + TEXT(".GameShowFlags")), ViewportInstanceSettings.GameShowFlagsString, GEditorPerProjectIni);
		
	// A single view mode index has been deprecated in favor of separate perspective and orthographic settings
	EViewModeIndex LegacyViewModeIndex = VMI_Unknown;
	{
		int32 LegacyVMIAsInt = VMI_Unknown;
		GConfig->GetInt(*IniSection, *(InConfigKey+TEXT(".ViewModeIndex")), LegacyVMIAsInt, GEditorPerProjectIni);
		LegacyViewModeIndex = (LegacyVMIAsInt == -1) ? VMI_Unknown : static_cast<EViewModeIndex>(LegacyVMIAsInt); // VMI_Unknown used to be -1
	}

	if(!GConfig->GetInt(*IniSection, *(InConfigKey+TEXT(".PerspViewModeIndex")), (int32&)ViewportInstanceSettings.PerspViewModeIndex, GEditorPerProjectIni))
	{
		if(ViewportInstanceSettings.ViewportType == LVT_Perspective)
		{
			// This viewport may pre-date the ViewModeIndex setting (VMI_Unknown), if so, try to be backward compatible
			ViewportInstanceSettings.PerspViewModeIndex = (LegacyViewModeIndex == VMI_Unknown) ? FindViewMode(LevelViewportClient->EngineShowFlags) : LegacyViewModeIndex;
		}
		else
		{
			// Default to Lit for a perspective viewport
			ViewportInstanceSettings.PerspViewModeIndex = VMI_Lit;
		}
	}

	if(!GConfig->GetInt(*IniSection, *(InConfigKey+TEXT(".OrthoViewModeIndex")), (int32&)ViewportInstanceSettings.OrthoViewModeIndex, GEditorPerProjectIni))
	{
		// Default to Unlit for an orthographic viewport
		ViewportInstanceSettings.OrthoViewModeIndex = (ViewportInstanceSettings.ViewportType != LVT_Perspective && LegacyViewModeIndex != VMI_Unknown) ? LegacyViewModeIndex : VMI_Unlit;
	}

	{
		FString BufferVisualizationModeString;
		if(GConfig->GetString(*IniSection, *(InConfigKey+TEXT(".BufferVisualizationMode")), BufferVisualizationModeString, GEditorPerProjectIni))
		{
			ViewportInstanceSettings.BufferVisualizationMode = *BufferVisualizationModeString;
		}
	}
	
	{
		FString ExposureSettingsString;
		if(GConfig->GetString(*IniSection, *(InConfigKey + TEXT(".ExposureSettings")), ExposureSettingsString, GEditorPerProjectIni))
		{
			ViewportInstanceSettings.ExposureSettings.SetFromString(*ExposureSettingsString);
		}
	}

	GConfig->GetBool(*IniSection, *(InConfigKey + TEXT(".bIsRealtime")), ViewportInstanceSettings.bIsRealtime, GEditorPerProjectIni);
	GConfig->GetBool(*IniSection, *(InConfigKey + TEXT(".bWantStats")), ViewportInstanceSettings.bShowOnScreenStats, GEditorPerProjectIni);
	GConfig->GetBool(*IniSection, *(InConfigKey + TEXT(".bWantFPS")), ViewportInstanceSettings.bShowFPS_DEPRECATED, GEditorPerProjectIni);
	GConfig->GetFloat(*IniSection, *(InConfigKey + TEXT(".FOVAngle")), ViewportInstanceSettings.FOVAngle, GEditorPerProjectIni);
	GConfig->GetBool(*IniSection, *(InConfigKey + TEXT(".bAllowCinematicControl")), ViewportInstanceSettings.bAllowCinematicControl, GEditorPerProjectIni);

	return ViewportInstanceSettings;
}


void SLevelViewport::OnSetBookmark( int32 BookmarkIndex )
{
	IBookmarkTypeTools::Get().CreateOrSetBookmark( BookmarkIndex, LevelViewportClient.Get() );
}

void SLevelViewport::OnJumpToBookmark( int32 BookmarkIndex )
{
	IBookmarkTypeTools::Get().JumpToBookmark( BookmarkIndex, TSharedPtr<struct FBookmarkBaseJumpToSettings>(), LevelViewportClient.Get() );
}

bool SLevelViewport::OnHasBookmarkSet(int32 BookmarkIndex)
{
	return IBookmarkTypeTools::Get().CheckBookmark(BookmarkIndex, LevelViewportClient.Get());
}

void SLevelViewport::OnClearBookmark(int32 BookmarkIndex)
{
	IBookmarkTypeTools::Get().ClearBookmark(BookmarkIndex, LevelViewportClient.Get());
}

void SLevelViewport::OnClearAllBookmarks()
{
	IBookmarkTypeTools::Get().ClearAllBookmarks(LevelViewportClient.Get());
}

void SLevelViewport::OnCompactBookmarks()
{
	IBookmarkTypeTools::Get().CompactBookmarks(LevelViewportClient.Get());
}

void SLevelViewport::OnToggleAllowCinematicPreview()
{
	// Reset the FOV of Viewport for cases where we have been previewing the cinematic with a changing FOV
	LevelViewportClient->ViewFOV = LevelViewportClient->AllowsCinematicControl() ? LevelViewportClient->ViewFOV : LevelViewportClient->FOVAngle;

	LevelViewportClient->SetAllowCinematicControl( !LevelViewportClient->AllowsCinematicControl() );
	LevelViewportClient->Invalidate( false );
}

bool SLevelViewport::AllowsCinematicPreview() const
{
	return LevelViewportClient->AllowsCinematicControl();
}

void SLevelViewport::OnIncrementPositionGridSize()
{
	GEditor->GridSizeIncrement();
	GEditor->RedrawLevelEditingViewports();
}

void SLevelViewport::OnDecrementPositionGridSize()
{
	GEditor->GridSizeDecrement();
	GEditor->RedrawLevelEditingViewports();
}

void SLevelViewport::OnIncrementRotationGridSize()
{
	GEditor->RotGridSizeIncrement();
	GEditor->RedrawLevelEditingViewports();
}
	
void SLevelViewport::OnDecrementRotationGridSize()
{
	GEditor->RotGridSizeDecrement();
	GEditor->RedrawLevelEditingViewports();
}

void SLevelViewport::OnActorLockToggleFromMenu(AActor* Actor)
{
	if (Actor != nullptr)
	{
		const bool bLockNewActor = Actor != LevelViewportClient->GetActiveActorLock().Get();

		// Lock the new actor if it wasn't the same actor that we just unlocked
		if (bLockNewActor)
		{
			// Unlock the previous actor
			OnActorUnlock();

			LockActorInternal(Actor);
		}
	}
}

void SLevelViewport::OnActorLockToggleFromMenu()
{
	OnActorUnlock();
}

bool SLevelViewport::IsActorLocked(const TWeakObjectPtr<AActor> Actor) const
{
	return LevelViewportClient->IsActorLocked(Actor);
}

bool SLevelViewport::IsAnyActorLocked() const
{
	return LevelViewportClient->IsAnyActorLocked();
}

void SLevelViewport::ToggleActorPilotCameraView()
{
	LevelViewportClient->bLockedCameraView = !LevelViewportClient->bLockedCameraView;
}

bool SLevelViewport::IsLockedCameraViewEnabled() const
{
	return LevelViewportClient->bLockedCameraView;
}

void SLevelViewport::SetAllowsCinematicControl(bool bAllow)
{
	LevelViewportClient->SetAllowCinematicControl(bAllow);
}

bool SLevelViewport::GetAllowsCinematicControl() const
{
	return LevelViewportClient->AllowsCinematicControl();
}

void SLevelViewport::FindSelectedInLevelScript()
{
	GUnrealEd->FindSelectedActorsInLevelScript();
}

bool SLevelViewport::CanFindSelectedInLevelScript() const
{
	AActor* Actor = GEditor->GetSelectedActors()->GetTop<AActor>();
	return (Actor != nullptr);
}

void SLevelViewport::OnSelectLockedActor()
{
	if (AActor* LockedActor = LevelViewportClient->GetActiveActorLock().Get())
	{
		// Deselect any currently selected actors, then select the locked/piloted actor
		UE::SLevelViewport::Internal::SelectActor(LockedActor);
	}
}

bool SLevelViewport::CanExecuteSelectLockedActor() const
{
	if (const AActor* LockedActor = LevelViewportClient->GetActiveActorLock().Get())
	{
		return LockedActor->IsSelectable();
	}

	return false;
}

void SLevelViewport::OnActorUnlock()
{
	if (AActor* LockedActor = LevelViewportClient->GetActiveActorLock().Get())
	{
		// Check to see if the locked actor was previously overriding the camera settings
		if (CanGetCameraInformationFromActor(LockedActor))
		{
			// Reset the settings
			LevelViewportClient->ViewFOV = LevelViewportClient->FOVAngle;
		}

		LevelViewportClient->SetActorLock(nullptr);

		// remove roll and pitch from camera when unbinding from actors
		GEditor->RemovePerspectiveViewRotation(true, true, false);

		// Move perspective camera back to pre-piloting transform
		LevelViewportClient->SetViewLocation(CachedPerspectiveCameraTransform.GetLocation());
		LevelViewportClient->SetViewRotation(CachedPerspectiveCameraTransform.GetRotation());
		Invalidate();

		// If we had a camera actor locked, and it was selected, then we should re-show the inset preview
		OnPreviewSelectedCamerasChange();
	}
}

bool SLevelViewport::CanExecuteActorUnlock() const
{
	return IsAnyActorLocked();
}

void SLevelViewport::OnActorLockSelected()
{
	USelection* ActorSelection = GEditor->GetSelectedActors();
	if (1 == ActorSelection->Num())
	{
		AActor* Actor = CastChecked<AActor>(ActorSelection->GetSelectedObject(0));
		LockActorInternal(Actor);
	}
}

bool SLevelViewport::CanExecuteActorLockSelected() const
{
	USelection* ActorSelection = GEditor->GetSelectedActors();
	if (1 == ActorSelection->Num())
	{
		return true;
	}
	return false;
}

bool SLevelViewport::IsSelectedActorLocked() const
{
	USelection* ActorSelection = GEditor->GetSelectedActors();
	if (1 == ActorSelection->Num() && IsAnyActorLocked())
	{
		AActor* Actor = CastChecked<AActor>(ActorSelection->GetSelectedObject(0));
		if (LevelViewportClient->GetActiveActorLock().Get() == Actor)
		{
			return true;
		}
	}
	return false;
}

float SLevelViewport::GetActorLockSceneOutlinerColumnWidth()
{
	return 18.0f;	// 16.0f for the icons and 2.0f padding
}

TSharedRef< ISceneOutlinerColumn > SLevelViewport::CreateActorLockSceneOutlinerColumn( ISceneOutliner& SceneOutliner ) const
{
	/**
	 * A custom column for the SceneOutliner which shows whether an actor is locked to a viewport
	 */
	class FCustomColumn : public ISceneOutlinerColumn
	{
	public:
		/**
		 *	Constructor
		 */
		FCustomColumn( const SLevelViewport* const InViewport )
			: Viewport(InViewport)
		{
		}

		virtual ~FCustomColumn()
		{
		}

		//////////////////////////////////////////////////////////////////////////
		// Begin ISceneOutlinerColumn Implementation

		virtual FName GetColumnID() override
		{
			return FName( "LockedToViewport" );
		}

		virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override
		{
			return SHeaderRow::Column( GetColumnID() )
				.FixedWidth(SLevelViewport::GetActorLockSceneOutlinerColumnWidth())
				[
					SNew( SSpacer )
				];
		}

		virtual const TSharedRef< SWidget > ConstructRowWidget( FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& InRow ) override
		{
			if (FActorTreeItem* ActorItem = TreeItem->CastTo<FActorTreeItem>())
			{
				AActor* Actor = ActorItem->Actor.Get();
				if (!Actor)
				{
					return SNullWidget::NullWidget;
				}

				const bool bLocked = Viewport->IsActorLocked(Actor);

				return SNew(SBox)
					.WidthOverride(SLevelViewport::GetActorLockSceneOutlinerColumnWidth())
					.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f))
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(bLocked ? "PropertyWindow.Locked" : "PropertyWindow.Unlocked"))
					.ColorAndOpacity(bLocked ? FLinearColor::White : FLinearColor(1.0f, 1.0f, 1.0f, 0.5f))
					];
			}
			else
			{
				return SNullWidget::NullWidget;
			}
		}

		// End ISceneOutlinerColumn Implementation
		//////////////////////////////////////////////////////////////////////////

	private:
		const SLevelViewport* Viewport;
	};

	return MakeShareable( new FCustomColumn( this ) );
}

void SLevelViewport::RedrawViewport( bool bInvalidateHitProxies )
{
	if ( bInvalidateHitProxies )
	{
		// Invalidate hit proxies and display pixels.
		LevelViewportClient->Viewport->Invalidate();

		// Force invalidate hit proxy immediately to avoid a crash
		// After level change a mouse click can be processed before the deferred invalidate of hitproxies is performed.
		LevelViewportClient->Viewport->InvalidateHitProxy();
		
		// Also update preview viewports
		for (const FViewportActorPreview& CurActorPreview : ActorPreviews)
		{
			if (CurActorPreview.LevelViewportClient.IsValid())
			{
				CurActorPreview.LevelViewportClient->Viewport->Invalidate();
				CurActorPreview.LevelViewportClient->Viewport->InvalidateHitProxy();
			}			
		}
	}
	else
	{
		// Invalidate only display pixels.
		LevelViewportClient->Viewport->InvalidateDisplay();

		// Also update preview viewports
		for (const FViewportActorPreview& CurActorPreview : ActorPreviews)
		{
			if (CurActorPreview.LevelViewportClient.IsValid())
			{
				CurActorPreview.LevelViewportClient->Viewport->InvalidateDisplay();
			}			
		}
	}
	
	RefreshPIEViewport();
}

bool SLevelViewport::CanToggleMaximizeMode() const
{
	TSharedPtr<FLevelViewportLayout> ParentLayoutPinned = ParentLayout.Pin();
	return (ParentLayoutPinned.IsValid() && ParentLayoutPinned->IsMaximizeSupported() && !ParentLayoutPinned->IsTransitioning());
}

void SLevelViewport::OnToggleMaximizeMode()
{
	TSharedPtr<FLevelViewportLayout> ParentLayoutPinned = ParentLayout.Pin();
	if (ParentLayoutPinned.IsValid() && ParentLayoutPinned->IsMaximizeSupported())
	{
		OnFloatingButtonClicked();

		bool bWantImmersive = IsImmersive();
		bool bWantMaximize = IsMaximized();

		//When in Immersive mode we always want to toggle back to normal editing mode while retaining the previous maximized state
		if( bWantImmersive )
		{
			bWantImmersive = false;
		}
		else
		{
			bWantMaximize = !bWantMaximize;
		}

		// We always want to animate in response to user-interactive toggling of maximized state
		const bool bAllowAnimation = true;


		FName ViewportName = ConfigKey;
		if (!ViewportName.IsNone())
		{
			ParentLayout.Pin()->RequestMaximizeViewport( ViewportName, bWantMaximize, bWantImmersive, bAllowAnimation );
		}
	}
}

FReply SLevelViewport::OnToggleMaximize()
{
	OnToggleMaximizeMode();
	return FReply::Handled();
}

void SLevelViewport::MakeMaximized(bool bWantMaximized, bool bAllowAnimation)
{
	if (TSharedPtr<FLevelViewportLayout> Layout = ParentLayout.Pin())
	{
		const FName ViewportName = ConfigKey;
		if (!ViewportName.IsNone())
		{
			const bool bWantImmersive = IsImmersive();
			Layout->RequestMaximizeViewport( ViewportName, bWantMaximized, bWantImmersive, bAllowAnimation );
		}
	}
}

void SLevelViewport::MakeImmersive( const bool bWantImmersive, const bool bAllowAnimation )
{
	if( ensure( ParentLayout.IsValid() ) ) 
	{
		const bool bWantMaximize = IsMaximized();

		FName ViewportName = ConfigKey;
		if (!ViewportName.IsNone())
		{
			ParentLayout.Pin()->RequestMaximizeViewport( ViewportName, bWantMaximize, bWantImmersive, bAllowAnimation );
		}
	}
}

/**
 * Registers a game viewport with the Slate application so that specific messages can be routed directly to this level viewport if it is an active PIE viewport
 */
void SLevelViewport::RegisterGameViewportIfPIE()
{
	if(ActiveViewport->IsPlayInEditorViewport())
	{
		FSlateApplication::Get().RegisterGameViewport(ViewportWidget.ToSharedRef());
	}
}

bool SLevelViewport::HasPlayInEditorViewport() const
{
	return ActiveViewport->IsPlayInEditorViewport() || ( InactiveViewport.IsValid() && InactiveViewport->IsPlayInEditorViewport() );
}

bool SLevelViewport::IsPlayInEditorViewportActive() const
{
	return ActiveViewport->IsPlayInEditorViewport();
}

UGameViewportClient* SLevelViewport::GetPlayClient() const
{
	return PlayClient.Get();
}

void SLevelViewport::OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	// On the first actor selection after entering Game View, enable the selection show flag
	if (IsVisible() && IsInGameView() && NewSelection.Num() != 0)
	{
		if( LevelViewportClient->bAlwaysShowModeWidgetAfterSelectionChanges )
		{
			LevelViewportClient->EngineShowFlags.SetModeWidgets(true);
		}

		// In game mode, selecting any actor should make LevelViewportClient->bShowWidget be "true"
		LevelViewportClient->ShowWidget(true);
		LevelViewportClient->EngineShowFlags.SetSelection(true);
		LevelViewportClient->EngineShowFlags.SetSelectionOutline(GetDefault<ULevelEditorViewportSettings>()->bUseSelectionOutline);
	}

	bNeedToUpdatePreviews = true;
}

void SLevelViewport::OnElementSelectionChanged(const UTypedElementSelectionSet* SelectionSet, bool bForceRefresh)
{
	// Request preview update. It's possible that the actor currently selected has forbidden default preview.
	// However, we need to show the preview widget when some of its child components selected.
	bNeedToUpdatePreviews = true;
	
	if (!DepthBar)
	{
		return;
	}
	
	DepthBar->ClearDepthIndicators();
	FTypedElementListConstRef ElementList = SelectionSet->GetElementList();
	for (int32 Index = 0; Index < ElementList->Num(); ++Index)
	{
		FTypedElementHandle Handle = ElementList->GetElementHandleAt(Index);
		
		if (TTypedElement<ITypedElementWorldInterface> WorldElement = ElementList->GetElement<ITypedElementWorldInterface>(Handle))
		{
			TAttribute<FBoxSphereBounds> Bounds = TAttribute<FBoxSphereBounds>::CreateLambda([SelectionSet, Handle]
			{
				FBoxSphereBounds Bounds(ForceInit);
				if (TTypedElement<ITypedElementWorldInterface> WorldElement = SelectionSet->GetElementList()->GetElement<ITypedElementWorldInterface>(Handle))
				{
					WorldElement.GetBounds(Bounds);
				}
				
				return Bounds;
			});
				
			TAttribute<FText> DisplayName;
				
			if (TTypedElement<ITypedElementObjectInterface> ObjectElement = ElementList->GetElement<ITypedElementObjectInterface>(Handle))
			{
				if (AActor* Actor = ObjectElement.GetObjectAs<AActor>())
				{
					DisplayName.BindWeakLambda(Actor, [Actor]
					{
						return FText::FromString(Actor->GetActorNameOrLabel());
					});
				}
			}
				
			DepthBar->AddDepthIndicatorSlot()
			.Bounds(Bounds)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("EditorViewport.DepthBar.SelectionHighlight"))
				.ToolTipText(DisplayName)
				.DesiredSizeOverride(FVector2d(8, 4))
			];
		}
	}
}

void SLevelViewport::PreviewSelectedCameraActors(const bool bPreviewInDesktopViewport)
{
	TArray<AActor*> ActorsToPreview;

	for (FSelectionIterator SelectionIt( *GEditor->GetSelectedActors()); SelectionIt; ++SelectionIt)
	{
		AActor* SelectedActor = CastChecked<AActor>( *SelectionIt );

		if (LevelViewportClient->IsLockedToActor(SelectedActor))
		{
			// If this viewport is already locked to the specified camera, then we don't need to do anything
		}
		else if (!FLevelEditorViewportClient::IsDroppingPreviewActor() && CanGetCameraInformationFromActor(SelectedActor))
		{
			ActorsToPreview.Add(SelectedActor);
		}
	}

	PreviewActors(ActorsToPreview, bPreviewInDesktopViewport);
}


class SActorPreview : public SCompoundWidget
{

public:

	~SActorPreview();

	SLATE_BEGIN_ARGS( SActorPreview )
		: _ViewportWidth( 240 ),
		  _ViewportHeight( 180 ),
		  _IsInteractive( false ) {}

		/** Width of the viewport */
		SLATE_ARGUMENT( int32, ViewportWidth )

		/** Height of the viewport */
		SLATE_ARGUMENT( int32, ViewportHeight )

		/** Actor being previewed.*/
		SLATE_ARGUMENT( TWeakObjectPtr< AActor >, PreviewActor )

		/** Parent Viewport this preview is part of.*/
		SLATE_ARGUMENT( TWeakPtr<SLevelViewport>, ParentViewport )

		/** Parent Viewport this preview is part of.*/
		SLATE_ARGUMENT(bool, IsInteractive)

		/** Optional */
		SLATE_DEFAULT_SLOT( FArguments, Content )

	SLATE_END_ARGS()

	/** Called by Slate to construct this widget */
	void Construct( const FArguments& InArgs );


	/** @return	Returns this actor preview's viewport widget */
	const TSharedPtr< SViewport > GetViewportWidget() const
	{
		return ViewportWidget;
	}


	/** SWidget overrides */
	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;

	/** Highlight this preview window by flashing the border. Will replay the curve sequence if it is already in the middle of a highlight. */
	void Highlight();

private:

	/** Called when an actor in the world is selected */
	void OnActorSelected(UObject* InActor);

	/** @return Returns the color and opacity to use for this widget */
	FLinearColor GetColorAndOpacity() const;

	/** @return Returns the border color and opacity to use for this widget (FSlateColor version) */
	FSlateColor GetBorderColorAndOpacity() const;

	/** @return Gets the name of the preview actor.*/
	FText OnReadText() const;

	/** @return Gets the camera settings of the preview actor.*/
	FText OnFilmbackText() const;

	/** @return Gets the Width of the preview viewport.*/
	FOptionalSize OnReadWidth() const;

	/** @return Gets the Height of the preview viewport.*/
	FOptionalSize OnReadHeight() const;

	/** @return Get the Width to wrap the preview actor name at.*/
	float OnReadTextWidth() const;

	/** Called when the pin preview button is clicked */
	FReply OnTogglePinnedButtonClicked();

	/** Swap between the pinned and unpinned icons for VR mode */
	const FSlateBrush* GetVRPinButtonIconBrush() const;

	/** Swap between the pinned and unpinned icons */
	const FSlateBrush* GetPinButtonIconBrush() const;

	/** @return the tooltip to display when hovering over the pin button */
	FText GetPinButtonToolTipText() const;

	/** Called when the detach button is clicked */
	FReply OnToggleDetachButtonClicked();

	/** Swap between the attached and detached icons */
	const FSlateBrush* GetDetachButtonIconBrush() const;

	/** @return the tooltip to display when hovering over the detach button */
	FText GetDetachButtonToolTipText() const;
	
	/** Viewport widget for this actor preview */
	TSharedPtr< SViewport > ViewportWidget;

	/** Actor being previewed.*/
	TWeakObjectPtr< AActor > PreviewActorPtr;

	/** Parent Viewport this preview is part of.*/
	TWeakPtr<SLevelViewport> ParentViewport;

	/** Curve sequence for fading in and out */
	FCurveSequence FadeSequence;

	/** Curve sequence for flashing the border (highlighting) when a pinned preview is re-selected */
	FCurveSequence HighlightSequence;

	/** Padding around the preview actor name */
	static const float PreviewTextPadding;
};

const float SActorPreview::PreviewTextPadding = 3.0f;

SActorPreview::~SActorPreview()
{
	USelection::SelectObjectEvent.RemoveAll(this);
}

void SActorPreview::Construct( const FArguments& InArgs )
{
	const int32 HorizSpacingBetweenViewports = 18;
	const int32 PaddingBeforeBorder = 6;

	USelection::SelectObjectEvent.AddRaw(this, &SActorPreview::OnActorSelected);

	// We don't want the border to be hit testable, since it would just get in the way of other
	// widgets that are added to the viewport overlay.
	this->SetVisibility(EVisibility::SelfHitTestInvisible);

	TSharedPtr<SViewport> PreviewViewport;
	auto GetPreviewContent = [&PreviewViewport](const FArguments& InOpArgs)->TSharedRef<SWidget>
	{
		if (InOpArgs._Content.Widget == SNullWidget::NullWidget)
		{
			return SAssignNew(PreviewViewport, SViewport)
				.RenderDirectlyToWindow(false)
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				.EnableGammaCorrection(false) // Scene rendering handles gamma correction
				.EnableBlending(true);
		}
		else
		{
			return InOpArgs._Content.Widget;
		}
	};

	// We usually don't want actor preview viewports to be interactive at all, but some custom actor previews may want to override this
	EVisibility BorderVisibility = (InArgs._IsInteractive ? EVisibility::SelfHitTestInvisible : EVisibility::HitTestInvisible);
	
	//We draw certain buttons depending on whether we're in editor or VR mode
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EVisibility VRVisibility = IVREditorModule::Get().IsVREditorModeActive() ? EVisibility::Visible : EVisibility::Hidden;
	EVisibility EditorVisibility = IVREditorModule::Get().IsVREditorModeActive() ? EVisibility::Hidden : EVisibility::Visible;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(0.f)

		.Visibility(EVisibility::SelfHitTestInvisible)

		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(FMargin(0, 0, PaddingBeforeBorder, PaddingBeforeBorder))
		[
			SNew( SOverlay )
			+SOverlay::Slot()
			[
				SNew( SBorder )
					.Visibility(BorderVisibility)
					.Padding( 16.0f )
					.BorderImage( FAppStyle::GetBrush( "UniformShadow_Tint" ) )
					.BorderBackgroundColor( this, &SActorPreview::GetBorderColorAndOpacity )
					.ColorAndOpacity( this, &SActorPreview::GetColorAndOpacity )

					[
						SNew( SBox )
							.WidthOverride( this, &SActorPreview::OnReadWidth )
							.HeightOverride(this, &SActorPreview::OnReadHeight )
							[
								SNew( SOverlay )
									+SOverlay::Slot()
									[
										GetPreviewContent(InArgs)
									]
									
									+SOverlay::Slot()
									.Padding(PreviewTextPadding)
									.HAlign(HAlign_Center)
									[
										SNew( STextBlock )
											.Text( this, &SActorPreview::OnReadText )
											.Font( FCoreStyle::GetDefaultFontStyle("Bold", 10) )
											.ShadowOffset( FVector2D::UnitVector )
											.WrapTextAt( this, &SActorPreview::OnReadTextWidth )
									]
									+ SOverlay::Slot()
									.Padding(PreviewTextPadding)
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Bottom)
									[
										SNew(STextBlock)
										.Text(this, &SActorPreview::OnFilmbackText)
										.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
										.ShadowOffset(FVector2D::UnitVector)
									]
							]
					]
				
			]
			+SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Bottom)
			.Padding(24.0f)
			[
				// Create a button to pin/unpin this viewport
				SNew(SButton)
					.ContentPadding(0.f)
					.ForegroundColor(FSlateColor::UseForeground())
					.ButtonStyle(FAppStyle::Get(), "ToggleButton")

					.IsFocusable(false)
					[
						SNew(SImage)
							.Visibility(EVisibility::Visible)
							.Image(this, &SActorPreview::GetPinButtonIconBrush)
					]

					// Bind the button's "on clicked" event to our object's method for this
					.OnClicked(this, &SActorPreview::OnTogglePinnedButtonClicked)
					.Visibility(EditorVisibility)

					// Pass along the block's tool-tip string
					.ToolTipText(this, &SActorPreview::GetPinButtonToolTipText)
			]
			+SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Bottom)
			.Padding( 0.f )
			[
				SNew(SBox)
				.WidthOverride(45.f)
				.HeightOverride(45.f)
				[
					// Create a button to pin/unpin this viewport
					SNew( SButton )
						.ContentPadding(0.f)
						.ForegroundColor( FSlateColor::UseForeground() )
						.ButtonStyle( FAppStyle::Get(), "ToggleButton" )

						.IsFocusable(false)
						[
							SNew( SImage )
								.Visibility( EVisibility::Visible )	
								.Image( this, &SActorPreview::GetVRPinButtonIconBrush )
						]

						// Bind the button's "on clicked" event to our object's method for this
						.OnClicked( this, &SActorPreview::OnTogglePinnedButtonClicked )
						.Visibility( VRVisibility )

						// Pass along the block's tool-tip string
						.ToolTipText( this, &SActorPreview::GetPinButtonToolTipText )
				]
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(0.f)
			[
				SNew(SBox)
				.WidthOverride(45.f)
				.HeightOverride(45.f)
				[
					// Create a button to attach/detach this viewport
					SNew(SButton)
						.ContentPadding(0.f)
						.ForegroundColor(FSlateColor::UseForeground())
						.ButtonStyle(FAppStyle::Get(), "ToggleButton")
						.IsFocusable(false)
						[
							SNew(SImage)
								.Visibility(EVisibility::Visible)
								.Image(this, &SActorPreview::GetDetachButtonIconBrush)
						]

						// Bind the button's "on clicked" event to our object's method for this
						.OnClicked(this, &SActorPreview::OnToggleDetachButtonClicked)
						.Visibility(VRVisibility)

						// Pass along the block's tool-tip string
						.ToolTipText(this, &SActorPreview::GetDetachButtonToolTipText)
				]
			]
		]
	];

	ViewportWidget = PreviewViewport;

	// Setup animation curve for fading in and out.  Note that we add a bit of lead-in time on the fade-in
	// to avoid hysteresis as the user moves the mouse over the view
	{
		/** The amount of time to wait before fading in after the mouse leaves */
		const float TimeBeforeFadingIn = 0.5f;

		/** The amount of time spent actually fading in or out */
		const float FadeTime = 0.25f;

		FadeSequence = FCurveSequence( TimeBeforeFadingIn, FadeTime );

		// Start fading in!
		FadeSequence.Play(this->AsShared(), false, TimeBeforeFadingIn);	// Skip the initial time delay and just fade straight in
	}

	HighlightSequence = FCurveSequence(0.f, 0.5f, ECurveEaseFunction::Linear);

	PreviewActorPtr = InArgs._PreviewActor;
	ParentViewport = InArgs._ParentViewport;
}

FReply SActorPreview::OnTogglePinnedButtonClicked()
{
	TSharedPtr<SLevelViewport> ParentViewportPtr = ParentViewport.Pin();

	if (ParentViewportPtr.IsValid())
	{
		ParentViewportPtr->ToggleActorPreviewIsPinned(PreviewActorPtr);
	}

	return FReply::Handled();
}

const FSlateBrush * SActorPreview::GetVRPinButtonIconBrush() const
{
	const FSlateBrush* IconBrush = nullptr;

	TSharedPtr<SLevelViewport> ParentViewportPtr = ParentViewport.Pin();

	if (ParentViewportPtr.IsValid())
	{
		if (ParentViewportPtr->IsActorPreviewPinned(PreviewActorPtr))
		{
			IconBrush = FAppStyle::GetBrush("VRViewportActorPreview.Pinned");
		}
		else
		{
			IconBrush = FAppStyle::GetBrush("VRViewportActorPreview.Unpinned");
		}

	}

	return IconBrush;
}

const FSlateBrush* SActorPreview::GetPinButtonIconBrush() const
{
	const FSlateBrush* IconBrush = nullptr;

	TSharedPtr<SLevelViewport> ParentViewportPtr = ParentViewport.Pin();

	if (ParentViewportPtr.IsValid())
	{
		if ( ParentViewportPtr->IsActorPreviewPinned(PreviewActorPtr) )
		{
			IconBrush = FAppStyle::GetBrush( "ViewportActorPreview.Pinned" );
		}
		else
		{
			IconBrush = FAppStyle::GetBrush( "ViewportActorPreview.Unpinned" );
		}

	}

	return IconBrush;
}

FText SActorPreview::GetPinButtonToolTipText() const
{
	FText CurrentToolTipText = LOCTEXT("PinPreviewActorTooltip", "Pin Preview");

	TSharedPtr<SLevelViewport> ParentViewportPtr = ParentViewport.Pin();

	if (ParentViewportPtr.IsValid())
	{
		if ( ParentViewportPtr->IsActorPreviewPinned(PreviewActorPtr) )
		{
			CurrentToolTipText = LOCTEXT("UnpinPreviewActorTooltip", "Unpin Preview");
		}
	}
		
	return CurrentToolTipText;
}

FReply SActorPreview::OnToggleDetachButtonClicked()
{
	TSharedPtr<SLevelViewport> ParentViewportPtr = ParentViewport.Pin();

	if (ParentViewportPtr.IsValid())
	{
		ParentViewportPtr->ToggleActorPreviewIsPanelDetached(PreviewActorPtr);
	}

	return FReply::Handled();
}

const FSlateBrush* SActorPreview::GetDetachButtonIconBrush() const
{
	const FSlateBrush* IconBrush = nullptr;

	TSharedPtr<SLevelViewport> ParentViewportPtr = ParentViewport.Pin();

	if (ParentViewportPtr.IsValid())
	{
		if (ParentViewportPtr->IsActorPreviewDetached(PreviewActorPtr))
		{
			IconBrush = FAppStyle::GetBrush("VRViewportActorPreview.Attached");
		}
		else
		{
			IconBrush = FAppStyle::GetBrush("VRViewportActorPreview.Detached");
		}

	}

	return IconBrush;
}

FText SActorPreview::GetDetachButtonToolTipText() const
{
	FText CurrentToolTipText = LOCTEXT("DetachPreviewActorTooltip", "Detach Preview from actor");

	TSharedPtr<SLevelViewport> ParentViewportPtr = ParentViewport.Pin();

	if (ParentViewportPtr.IsValid())
	{
		if (ParentViewportPtr->IsActorPreviewDetached(PreviewActorPtr))
		{
			CurrentToolTipText = LOCTEXT("AttachPreviewActorTooltip", "Attach Preview to actor");
		}
	}

	return CurrentToolTipText;
}


void SActorPreview::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseEnter( MyGeometry, MouseEvent );

	// The viewport could potentially be moved around inside the toolbar when the mouse is captured
	// If that is the case we do not play the fade transition
	if( !FSlateApplication::Get().IsUsingHighPrecisionMouseMovment() )
	{
		if( FadeSequence.IsPlaying() )
		{
			if( FadeSequence.IsForward() )
			{
				// Fade in is already playing so just force the fade out curve to the end so we don't have a "pop" 
				// effect from quickly resetting the alpha
				FadeSequence.JumpToStart();
			}
		}
		else
		{
			FadeSequence.PlayReverse(this->AsShared());
		}
	}
}


void SActorPreview::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseLeave( MouseEvent );

	// The viewport could potentially be moved around inside the toolbar when the mouse is captured
	// If that is the case we do not play the fade transition
	if( !FSlateApplication::Get().IsUsingHighPrecisionMouseMovment() )
	{
		if( FadeSequence.IsPlaying() )
		{
			if( FadeSequence.IsInReverse() )
			{
				FadeSequence.Reverse();
			}
		}
		else
		{
			FadeSequence.Play(this->AsShared());
		}
	}

	// Now is a good time to check if we need to remove any PreviewActors that might have been un-pinned
	TSharedPtr<SLevelViewport> ParentViewportPtr = ParentViewport.Pin();
	if (ParentViewportPtr.IsValid())
	{
		ParentViewportPtr->OnPreviewSelectedCamerasChange();
	}
}


FLinearColor SActorPreview::GetColorAndOpacity() const
{
	FLinearColor Color = FLinearColor::White;
	
	const float HoveredOpacity = 0.4f;
	const float NonHoveredOpacity = 1.0f;

	Color.A = FMath::Lerp( HoveredOpacity, NonHoveredOpacity, FadeSequence.GetLerp() );

	return Color;
}

void SActorPreview::OnActorSelected(UObject* InActor)
{
	if (InActor && InActor == PreviewActorPtr && InActor->IsSelected())
	{
		TSharedPtr<SLevelViewport> ParentViewportPtr = ParentViewport.Pin();
		const bool bIsPreviewPinned = ParentViewportPtr.IsValid() && ParentViewportPtr->IsActorPreviewPinned(PreviewActorPtr);

		if (bIsPreviewPinned)
		{
			Highlight();
		}
	}
}

void SActorPreview::Highlight()
{
	HighlightSequence.JumpToStart();
	HighlightSequence.Play(this->AsShared());
}

FSlateColor SActorPreview::GetBorderColorAndOpacity() const
{
	FLinearColor Color(0.f, 0.f, 0.f, 0.5f);

	if (HighlightSequence.IsPlaying())
	{
		static const FName SelectionColorName("SelectionColor");
		const FLinearColor SelectionColor = FAppStyle::Get().GetSlateColor(SelectionColorName).GetSpecifiedColor().CopyWithNewOpacity(0.5f);
		
		const float Interp = FMath::Sin(HighlightSequence.GetLerp()*6*PI) / 2 + 1;
		Color = FMath::Lerp(SelectionColor, Color, Interp);
	}
	
	return Color;
}

FText SActorPreview::OnReadText() const
{
	if( PreviewActorPtr.IsValid() )
	{
		return FText::FromString(PreviewActorPtr.Get()->GetActorLabel());
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText SActorPreview::OnFilmbackText() const
{
	if (PreviewActorPtr.IsValid())
	{
		UActorComponent* ViewComponent = FLevelEditorViewportClient::FindViewComponentForActor(PreviewActorPtr.Get());
		UCameraComponent* CameraComponent = Cast<UCameraComponent>(ViewComponent);
		if (CameraComponent)
		{
			return CameraComponent->GetFilmbackText();
		}
	}
	return FText::GetEmpty();
}

FOptionalSize SActorPreview::OnReadWidth() const
{
	const float PreviewHeight = OnReadHeight().Get();

	// See if the preview actor wants to constrain the aspect ratio first
	if (AActor* PreviewActor = PreviewActorPtr.Get())
	{
		FMinimalViewInfo CameraInfo;
		if (SLevelViewport::GetCameraInformationFromActor(PreviewActor, /*out*/ CameraInfo))
		{
			if (CameraInfo.bConstrainAspectRatio && (CameraInfo.AspectRatio > 0.0f))
			{
				return PreviewHeight * CameraInfo.AspectRatio;
			}
		}
	}

	// Otherwise try to match the parent viewport's aspect ratio
	if ( ParentViewport.IsValid() )
	{
		return PreviewHeight * ParentViewport.Pin()->GetActiveViewport()->GetDesiredAspectRatio();
	}

	return PreviewHeight * 1.7777f;
}

FOptionalSize SActorPreview::OnReadHeight() const
{
	const float MinimumHeight = 32;
	// Also used as parent height in case valid parent viewport is not set
	const FIntPoint::IntType MaximumHeight = 428;
	// Used to make sure default viewport scale * parent viewport height = roughly same size as original windows
	const float PreviewScalingFactor = 0.06308f;

	FIntPoint::IntType ParentHeight = MaximumHeight;
	if ( ParentViewport.IsValid() )
	{
		ParentHeight = ParentViewport.Pin()->GetActiveViewport()->GetSizeXY().Y;
	}
	return FMath::Clamp( GetDefault<ULevelEditorViewportSettings>()->CameraPreviewSize * static_cast<float>(ParentHeight) * PreviewScalingFactor, MinimumHeight, MaximumHeight );
}

float SActorPreview::OnReadTextWidth() const
{
	return OnReadWidth().Get() - (PreviewTextPadding*2.0f);
}

void SLevelViewport::PreviewActors( const TArray< AActor* >& InActorsToPreview, const bool bPreviewInDesktopViewport /*= true*/)
{
	TArray< AActor* > ActorsToPreview(InActorsToPreview);

	TArray< AActor* > NewActorsToPreview;
	TArray< AActor* > ActorsToStopPreviewing;
	for (TWeakObjectPtr<AActor> Actor : AlwaysPreviewActors)
	{
		AActor *CurActor = Actor.Get();
		if (CurActor != nullptr)
		{
			ActorsToPreview.AddUnique(CurActor);
		}
	}

	// Look for actors that we no longer want to preview
	for( auto ActorPreviewIt = ActorPreviews.CreateConstIterator(); ActorPreviewIt; ++ActorPreviewIt )
	{
		auto ExistingActor = ActorPreviewIt->Actor.Get();
		if( ExistingActor != nullptr )
		{
			auto bShouldKeepActor = false;
			for( auto ActorIt = ActorsToPreview.CreateConstIterator(); ActorIt; ++ActorIt )
			{
				auto CurActor = *ActorIt;
				if( CurActor != nullptr && CurActor == ExistingActor )
				{
					bShouldKeepActor = true;
					break;
				}
			}

			if( !bShouldKeepActor )
			{
				// We were asked to stop previewing this actor
				ActorsToStopPreviewing.AddUnique( ExistingActor );
			}
		}
	}

	// Look for any new actors that we aren't previewing already
	for( auto ActorIt = ActorsToPreview.CreateConstIterator(); ActorIt; ++ActorIt )
	{
		auto CurActor = *ActorIt;

		// Check to see if we're already previewing this actor.  If we are, we'll just skip it
		auto bIsAlreadyPreviewed = false;
		for( auto ExistingPreviewIt = ActorPreviews.CreateConstIterator(); ExistingPreviewIt; ++ExistingPreviewIt )
		{
			// There could be null actors in this list as we haven't actually removed them yet.
			auto ExistingActor = ExistingPreviewIt->Actor.Get();
			if( ExistingActor != nullptr && CurActor == ExistingActor )
			{
				// Already previewing this actor.  Ignore it.
				bIsAlreadyPreviewed = true;
				break;
			}
		}

		if( !bIsAlreadyPreviewed )
		{
			// This is a new actor that we want to preview.  Let's set that up.
			NewActorsToPreview.Add( CurActor );
		}
	}


	// Kill any existing actor previews that we don't want or have expired
	for( int32 PreviewIndex = 0; PreviewIndex < ActorPreviews.Num(); ++PreviewIndex )
	{
		AActor* ExistingActor = ActorPreviews[PreviewIndex].Actor.Get();
		if ( ExistingActor == nullptr )
		{
			// decrement index so we don't miss next preview after deleting
			RemoveActorPreview( PreviewIndex-- , nullptr, bPreviewInDesktopViewport);
		}
		else
		{
			if ( !ActorPreviews[PreviewIndex].bIsPinned )
			{
				for( auto ActorIt = ActorsToStopPreviewing.CreateConstIterator(); ActorIt; ++ActorIt )
				{
					auto CurActor = *ActorIt;
					if( ExistingActor == CurActor )
					{
						// Remove this preview!
						// decrement index so we don't miss next preview after deleting
						RemoveActorPreview( PreviewIndex-- , CurActor, bPreviewInDesktopViewport);
						break;
					}
				}
			}
		}
	}

	// Create previews for any actors that we need to
	if( NewActorsToPreview.Num() > 0 )
	{
		for (AActor* CurActor : NewActorsToPreview)
		{
			TSharedPtr<SWidget> CustomPreviewContent;
			if (UActorComponent* PreviewComp = FLevelEditorViewportClient::FindViewComponentForActor(CurActor))
			{
				CustomPreviewContent = PreviewComp->GetCustomEditorPreviewWidget();
			}
			const bool bNeedsLevelViewport = !CustomPreviewContent.IsValid();

			TSharedPtr<FLevelEditorViewportClient> ActorPreviewLevelViewportClient;
			if (bNeedsLevelViewport)
			{ 
				ActorPreviewLevelViewportClient = MakeShareable(new FLevelEditorViewportClient(SharedThis(this)));

				// NOTE: We don't bother setting ViewLocation, ViewRotation, etc, here.  This is because we'll call
				//       PushControllingActorDataToViewportClient() below which will do this!

				// ParentLevelEditor is used for summoning context menus, which should never happen for these preview
				// viewports, but we'll keep the relationship intact anyway.
				ActorPreviewLevelViewportClient->ParentLevelEditor = ParentLevelEditor.Pin();

				ActorPreviewLevelViewportClient->ViewportType = LVT_Perspective;
				ActorPreviewLevelViewportClient->bSetListenerPosition = false;	// Preview viewports never be a listener

																				// Never draw the axes indicator in these small viewports
				ActorPreviewLevelViewportClient->bDrawAxes = false;

				// Default to "game" show flags for camera previews
				// Still draw selection highlight though
				ActorPreviewLevelViewportClient->EngineShowFlags = FEngineShowFlags(ESFIM_Game);
				ActorPreviewLevelViewportClient->EngineShowFlags.SetSelection(true);
				ActorPreviewLevelViewportClient->LastEngineShowFlags = FEngineShowFlags(ESFIM_Editor);
				if (!bPreviewInDesktopViewport)
				{
					ActorPreviewLevelViewportClient->EngineShowFlags.Tonemapper = false;
				}

				// We don't use view modes for preview viewports
				ActorPreviewLevelViewportClient->SetViewMode(VMI_Unknown);

				// User should never be able to interact with this viewport
				ActorPreviewLevelViewportClient->bDisableInput = true;

				// Never allow cinematics to possess these views
				ActorPreviewLevelViewportClient->SetAllowCinematicControl( false );

				// Our preview viewport is always visible if our owning SLevelViewport is visible, so we hook up
				// to the same IsVisible method
				ActorPreviewLevelViewportClient->VisibilityDelegate.BindSP(this, &SLevelViewport::IsVisible);

				// Push actor transform to view.  From here on out, this will happen automatically in FLevelEditorViewportClient::Tick.
				// The reason we allow the viewport client to update this is to avoid off-by-one-frame issues when dragging actors around.
				ActorPreviewLevelViewportClient->SetActorLock(CurActor);
				ActorPreviewLevelViewportClient->UpdateViewForLockedActor();

				// Preview the play world if the current actor is in the play world
				if (CurActor->GetWorld()->IsGameWorld())
				{
					ActorPreviewLevelViewportClient->SetIsSimulateInEditorViewport(true);
				}
			}

			TSharedPtr< SActorPreview > ActorPreviewWidget;
			if (CustomPreviewContent.IsValid())
			{
				SAssignNew(ActorPreviewWidget, SActorPreview)
					.PreviewActor(CurActor)
					.ParentViewport(SharedThis(this))
					.IsInteractive(true)
					.Content()
					[
						CustomPreviewContent.ToSharedRef()
					];
			}
			else
			{
				SAssignNew(ActorPreviewWidget, SActorPreview)
					.PreviewActor(CurActor)
					.ParentViewport(SharedThis(this));
			}

			TSharedPtr<FSceneViewport> ActorPreviewSceneViewport;
			if (bNeedsLevelViewport)
			{
				TSharedPtr<SViewport> ActorPreviewViewportWidget = ActorPreviewWidget->GetViewportWidget();
				ensure(ActorPreviewViewportWidget.IsValid());
				ActorPreviewSceneViewport = FSceneViewport::Create(ActorPreviewLevelViewportClient, ActorPreviewViewportWidget);
			}

			FViewportActorPreview& NewActorPreview = *new( ActorPreviews ) FViewportActorPreview;
			NewActorPreview.Actor = CurActor;
			NewActorPreview.LevelViewportClient = ActorPreviewLevelViewportClient;
			NewActorPreview.SceneViewport = ActorPreviewSceneViewport;
			NewActorPreview.PreviewWidget = ActorPreviewWidget;
			NewActorPreview.bIsPinned = false;

			// Add our new widget to our viewport's overlay
			// @todo camerapip: Consider using a canvas instead of an overlay widget -- our viewports get SQUASHED when the view shrinks!
			IVREditorModule& VREditorModule = IVREditorModule::Get();
			if (!bPreviewInDesktopViewport)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				VREditorModule.UpdateActorPreview( NewActorPreview.PreviewWidget.ToSharedRef(), ActorPreviews.Num()-1, CurActor);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			else
			{
				ActorPreviewHorizontalBox->AddSlot()
				.AutoWidth()
				[
					ActorPreviewWidget.ToSharedRef()
				];
			}
		}

		// OK, at least one new preview viewport was added, so update settings for all views immediately.
		// This will also be repeated every time the SLevelViewport is ticked, just to make sure that
		// feature such as "real-time" mode stay in sync.
		UpdateActorPreviewViewports();
	}
}

bool SLevelViewport::IsActorAlwaysPreview(TWeakObjectPtr<AActor> Actor) const
{
	return AlwaysPreviewActors.Contains(Actor);
}

void SLevelViewport::SetActorAlwaysPreview(TWeakObjectPtr<AActor> PreviewActor, bool bAlwaysPreview)
{
	if (!IsActorAlwaysPreview(PreviewActor) && bAlwaysPreview)
	{
		AlwaysPreviewActors.Add(PreviewActor);
		bNeedToUpdatePreviews = true;
	}
	else if (IsActorAlwaysPreview(PreviewActor) && !bAlwaysPreview)
	{
		AlwaysPreviewActors.Remove(PreviewActor);
		bNeedToUpdatePreviews = true;
	}
}

void SLevelViewport::ToggleActorPreviewIsPinned(TWeakObjectPtr<AActor> ActorToTogglePinned)
{
	if (ActorToTogglePinned.IsValid())
	{
		AActor* ActorToTogglePinnedPtr = ActorToTogglePinned.Get();

		for (FViewportActorPreview& ActorPreview : ActorPreviews)
		{
			if ( ActorPreview.Actor.IsValid() )
			{
				if ( ActorToTogglePinnedPtr == ActorPreview.Actor.Get() )
				{
					ActorPreview.ToggleIsPinned();
				}
			}
		}
	}
}

void SLevelViewport::ToggleActorPreviewIsPanelDetached(TWeakObjectPtr<AActor> PreviewActor)
{
	if (PreviewActor.IsValid())
	{
		AActor* PreviewActorPtr = PreviewActor.Get();

		for (FViewportActorPreview& ActorPreview : ActorPreviews)
		{
			if (ActorPreview.Actor.IsValid())
			{
				if (PreviewActorPtr == ActorPreview.Actor.Get())
				{
					ActorPreview.ToggleIsPanelDetached();
					IVREditorModule& VREditorModule = IVREditorModule::Get();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					//Disable current actor preview
					VREditorModule.UpdateActorPreview(SNullWidget::NullWidget, ActorPreviews.Num() - 1, PreviewActor.Get(), !ActorPreview.bIsPanelDetached);
					//Enable new current preview (the old one was detached or attached, the new one is the opposite
					VREditorModule.UpdateActorPreview(ActorPreview.PreviewWidget.ToSharedRef(), ActorPreviews.Num() - 1, PreviewActor.Get(), ActorPreview.bIsPanelDetached);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
		}
	}
}



bool SLevelViewport::IsActorPreviewPinned( TWeakObjectPtr<AActor> PreviewActor )
{
	if (PreviewActor.IsValid())
	{
		AActor* PreviewActorPtr = PreviewActor.Get();

		for (FViewportActorPreview& ActorPreview : ActorPreviews)
		{
			if ( ActorPreview.Actor.IsValid() )
			{
				if ( PreviewActorPtr == ActorPreview.Actor.Get() )
				{
					return ActorPreview.bIsPinned;
				}
			}
		}
	}

	return false;
}

bool SLevelViewport::IsActorPreviewDetached(TWeakObjectPtr<AActor> PreviewActor)
{
	if (PreviewActor.IsValid())
	{
		AActor* PreviewActorPtr = PreviewActor.Get();

		for (FViewportActorPreview& ActorPreview : ActorPreviews)
		{
			if (ActorPreview.Actor.IsValid())
			{
				if (PreviewActorPtr == ActorPreview.Actor.Get())
				{
					return ActorPreview.bIsPanelDetached;
				}
			}
		}
	}

	return false;
}

void SLevelViewport::UpdateActorPreviewViewports()
{
	// Remove any previews that are locked to the same actor as the level viewport client's actor lock
	for( int32 PreviewIndex = 0; PreviewIndex < ActorPreviews.Num(); ++PreviewIndex )
	{
		AActor* ExistingActor = ActorPreviews[PreviewIndex].Actor.Get();
		if (ExistingActor && LevelViewportClient->IsActorLocked(ExistingActor))
		{
			RemoveActorPreview( PreviewIndex-- );
		}
	}

	// Look for actors that we no longer want to preview
	for(const FViewportActorPreview& CurActorPreview : ActorPreviews)
	{
		if (CurActorPreview.LevelViewportClient.IsValid())
		{
			CurActorPreview.LevelViewportClient->SetRealtime(LevelViewportClient->IsRealtime());
			CurActorPreview.LevelViewportClient->bDrawBaseInfo = LevelViewportClient->bDrawBaseInfo;
			CurActorPreview.LevelViewportClient->bDrawVertices = LevelViewportClient->bDrawVertices;
			CurActorPreview.LevelViewportClient->EngineShowFlags.SetSelectionOutline(LevelViewportClient->EngineShowFlags.SelectionOutline);
			CurActorPreview.LevelViewportClient->EngineShowFlags.SetCompositeEditorPrimitives(LevelViewportClient->EngineShowFlags.CompositeEditorPrimitives);
		}
	}
}

void SLevelViewport::OnPreviewSelectedCamerasChange()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bPreviewInDesktopViewport = !IVREditorModule::Get().IsVREditorModeActive();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Check to see if previewing selected cameras is enabled and if we're the active level viewport client.
	if (GetDefault<ULevelEditorViewportSettings>()->bPreviewSelectedCameras && GCurrentLevelEditingViewportClient == LevelViewportClient.Get())
	{
		PreviewSelectedCameraActors(bPreviewInDesktopViewport);
	}
	else
	{
		// We're either not the active viewport client or preview selected cameras option is disabled, so remove any existing previewed actors
		PreviewActors(TArray<AActor*>(), bPreviewInDesktopViewport);
	}
}

FText SLevelViewport::GetCurrentScreenPercentageText() const
{
	return FText::FromString(FString::Printf(TEXT("%3d%%"), int32(GetLevelViewportClient().GetPreviewScreenPercentage())));
}

EVisibility SLevelViewport::GetSelectedActorsCurrentLevelTextVisibility() const
{
	EVisibility ContentVisibility = OnGetViewportContentVisibility();
	if (ContentVisibility == EVisibility::Visible)
	{
		ContentVisibility = EVisibility::SelfHitTestInvisible;
	}
	return (&GetLevelViewportClient() == GCurrentLevelEditingViewportClient) 
		&& (GEditor->GetSelectedActorCount() > 0) 
		&& !IsPlayInEditorViewportActive() 
		&& GetWorld() && GetWorld()->GetCurrentLevel() && GetWorld()->GetCurrentLevel()->OwningWorld->GetLevels().Num() > 1
		&& !GetWorld()->IsPartitionedWorld()
		? ContentVisibility : EVisibility::Collapsed;
}

FText SLevelViewport::GetSelectedActorsCurrentLevelText(bool bDrawOnlyLabel) const
{
	// Display the currently selected actor's level
	FText LabelName;
	FText CurrentLevelName;


	if (ActiveViewport.IsValid() && (&GetLevelViewportClient() == GCurrentLevelEditingViewportClient) && GetWorld())
	{
		if (ActiveViewport->GetPlayInEditorIsSimulate() || !ActiveViewport->GetClient()->GetWorld()->IsGameWorld())
		{
			if (bDrawOnlyLabel)
			{
				LabelName = LOCTEXT("SelectedActorsCurrentLevelLabel", "Selected Actor(s) in");
			}
			else
			{	
				ULevel* LevelToMakeCurrent = nullptr;

				// Look to the selected actors for the level to make current.
				// If actors from multiple levels are selected, do nothing.
				for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
				{
					AActor* Actor = static_cast<AActor*>(*It);
					checkSlow(Actor->IsA(AActor::StaticClass()));

					ULevel* ActorLevel = Actor->GetLevel();

					if (!LevelToMakeCurrent)
					{
						// First assignment.
						LevelToMakeCurrent = ActorLevel;
					}
					else if (LevelToMakeCurrent != ActorLevel)
					{
						// Actors from multiple levels are selected -- abort.
						LevelToMakeCurrent = nullptr;
						break;
					}
				}

				FText ActualLevelName = LOCTEXT("MultipleLevelValues", "Multiple Levels");
				if (LevelToMakeCurrent)
				{
					ActualLevelName = FText::FromName(FPackageName::GetShortFName(LevelToMakeCurrent->GetOutermost()->GetFName()));
				}
				if (LevelToMakeCurrent == GetWorld()->PersistentLevel)
				{
					
					FFormatNamedArguments Args;
					Args.Add(TEXT("ActualLevelName"), ActualLevelName);
					CurrentLevelName = FText::Format(LOCTEXT("LevelName", "{0} (Persistent)"), ActualLevelName);
				}
				else
				{
					CurrentLevelName = ActualLevelName;
				}
			}

			if (bDrawOnlyLabel)
			{
				return LabelName;
			}
		}
	}

	return CurrentLevelName;
}

EVisibility SLevelViewport::GetCurrentScreenPercentageVisibility() const
{
	bool Visible = !IsPlayInEditorViewportActive() &&
		GetLevelViewportClient().SupportsPreviewResolutionFraction() &&
		GetLevelViewportClient().GetPreviewScreenPercentage() > 100;

	return Visible ? EVisibility::Visible : EVisibility::Collapsed;
}

void SLevelViewport::OnSetViewportConfiguration(FName ConfigurationName)
{
	TSharedPtr<FLevelViewportLayout> LayoutPinned = ParentLayout.Pin();
	if (LayoutPinned.IsValid())
	{
		TSharedPtr<FEditorViewportTabContent> ViewportTabPinned = LayoutPinned->GetParentTabContent().Pin();
		if (ViewportTabPinned.IsValid())
		{
			ViewportTabPinned->SetViewportConfiguration(ConfigurationName);
			FSlateApplication::Get().DismissAllMenus();
			UToolMenus::Get()->CleanupStaleWidgetsNextTick(true);
		}
	}
}

bool SLevelViewport::IsViewportConfigurationSet(FName ConfigurationName) const
{
	TSharedPtr<FLevelViewportLayout> LayoutPinned = ParentLayout.Pin();
	if (LayoutPinned.IsValid())
	{
		TSharedPtr<FEditorViewportTabContent> ViewportTabPinned = LayoutPinned->GetParentTabContent().Pin();
		if (ViewportTabPinned.IsValid())
		{
			return ViewportTabPinned->IsViewportConfigurationSet(ConfigurationName);
		}
	}
	return false;
}

FName SLevelViewport::GetViewportTypeWithinLayout() const
{
	TSharedPtr<FLevelViewportLayout> LayoutPinned = ParentLayout.Pin();
	if (LayoutPinned.IsValid() && !ConfigKey.IsNone())
	{
		TSharedPtr<ILevelViewportLayoutEntity> Entity = StaticCastSharedPtr<ILevelViewportLayoutEntity>(LayoutPinned->GetViewports().FindRef(ConfigKey));
		if (Entity.IsValid())
		{
			return Entity->GetType();
		}
	}
	return "Default";
}

void SLevelViewport::SetViewportTypeWithinLayout(FName InLayoutType)
{
	TSharedPtr<FLevelViewportLayout> LayoutPinned = ParentLayout.Pin();
	if (LayoutPinned.IsValid() && !ConfigKey.IsNone())
	{
		// Important - RefreshViewportConfiguration does not save config values. We save its state first, to ensure that .TypeWithinLayout (below) doesn't get overwritten
		TSharedPtr<FEditorViewportTabContent> ViewportTabPinned = LayoutPinned->GetParentTabContent().Pin();
		if (!ViewportTabPinned)
		{
			return;
		}

		ViewportTabPinned->SaveConfig();

		const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();
		GConfig->SetString( *IniSection, *( ConfigKey.ToString() + TEXT(".TypeWithinLayout") ), *InLayoutType.ToString(), GEditorPerProjectIni );

		// Force a refresh of the tab content
		ViewportTabPinned->RefreshViewportConfiguration();
		FSlateApplication::Get().DismissAllMenus();
	}
}

void SLevelViewport::ToggleViewportTypeActivationWithinLayout(FName InLayoutType)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GetViewportTypeWithinLayout() != InLayoutType)
	{
		SetViewportTypeWithinLayout(InLayoutType);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool SLevelViewport::IsViewportTypeWithinLayoutEqual(FName InLayoutType)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetViewportTypeWithinLayout() == InLayoutType;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void SLevelViewport::StartPlayInEditorSession(UGameViewportClient* InPlayClient, const bool bInSimulateInEditor)
{
	check( !HasPlayInEditorViewport() );

	check( !InactiveViewport.IsValid() );

	// Ensure our active viewport is for level editing
	check( ActiveViewport->GetClient() == LevelViewportClient.Get() );
	// Save camera settings that may be adversely affected by PIE, so that they may be restored later
	LevelViewportClient->PrepareCameraForPIE();

	PlayClient = InPlayClient;
	
	// Set up binding overrides that can handle input before any player input
	FOverrideInputKeyHandler& PlayInputKeyOverride = PlayClient->OnOverrideInputKey();
	if (PlayInputKeyOverride.IsBound())
	{
		// Carry forward the previous binding and allow it to take precedence
		FOverrideInputKeyHandler OldHandler = PlayInputKeyOverride;
		PlayInputKeyOverride.BindLambda([WeakThis = SharedThis(this).ToWeakPtr(), OldHandler](FInputKeyEventArgs& Args)
		{
			if (OldHandler.IsBound() && OldHandler.Execute(Args))
			{
				return true;
			}
			if (TSharedPtr<SLevelViewport> Viewport = WeakThis.Pin())
			{
				return Viewport->OnPIEViewportInputOverride(Args);
			}
			return false;
		});
	}
	else
	{
		PlayInputKeyOverride.BindSP(this, &SLevelViewport::OnPIEViewportInputOverride);
	}
	
	// Here we will swap the editor viewport client out for the client for the play in editor session
	InactiveViewport = ActiveViewport;
	// Store the content in the viewport widget (editor tool bar etc) so we can show the game UI content if it has any
	InactiveViewportWidgetEditorContent = ViewportWidget->GetContent();

	// Remove keyboard focus to send a focus lost message to the widget to clean up any saved state from the viewport interface thats about to be swapped out
	// Focus will be set when the game viewport is registered
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);

	// Unregister the editor debug canvas from slot 0 before AddAssociation inserts SGameLayerManager there.
	EditorDebugCanvas->SetSceneViewport(nullptr);
	ViewportOverlay->RemoveSlot(0);

	ActiveViewport = FSceneViewport::Create(PlayClient.Get(), ViewportWidget.ToSharedRef());
	ActiveViewport->SetPlayInEditorViewport( true );

	// Whether to start with the game taking mouse control or leaving it shown in the editor
	ActiveViewport->SetPlayInEditorGetsMouseControl(GetDefault<ULevelEditorPlaySettings>()->GameGetsMouseControl);
	ActiveViewport->SetPlayInEditorIsSimulate(bInSimulateInEditor);
	
	ActiveViewport->OnPlayWorldViewportSwapped( *InactiveViewport );
	
	LevelViewportClient->AddRealtimeOverride(false, LOCTEXT("LevelViewport_RealTimeDisableOnPie", "Disable LevelViewport Realtime for PIE"));

	// Register the new viewport widget with Slate for viewport specific message routing.
	FSlateApplication::Get().RegisterGameViewport(ViewportWidget.ToSharedRef() );

	ULevelEditorPlaySettings const* EditorPlayInSettings = GetDefault<ULevelEditorPlaySettings>();
	check(EditorPlayInSettings);

	// Kick off a quick transition effect (border graphics)
	ViewTransitionType = EViewTransition::StartingPlayInEditor;
	ViewTransitionAnim = FCurveSequence( 0.0f, 1.5f, ECurveEaseFunction::CubicOut );
	bViewTransitionAnimPending = true;
	if (EditorPlayInSettings->EnablePIEEnterAndExitSounds)
	{
		GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/GamePreview/StartPlayInEditor_Cue.StartPlayInEditor_Cue"));
	}

	bPIEHasFocus = ActiveViewport->HasMouseCapture();

	GEngine->BroadcastLevelActorListChanged();

	// register for preview platform change
	UEditorEngine* Editor = CastChecked<UEditorEngine>(GEngine);

	PIEPreviewFeatureLevelChangedHandle = Editor->OnPreviewShaderPlatformChanged().AddLambda([InPlayClient](EShaderPlatform NewShaderPlatform)
		{
			InPlayClient->GetWorld()->ShaderPlatformChanged(NewShaderPlatform, true);
		});

}

EVisibility SLevelViewport::GetMouseCaptureLabelVisibility() const
{
	if (GEditor->PlayWorld)
	{
		// Show the label if the local player's PC isn't set to show the cursor
		auto const TargetPlayer = GEngine->GetLocalPlayerFromControllerId(GEditor->PlayWorld, 0);
		if (TargetPlayer && TargetPlayer->PlayerController && !TargetPlayer->PlayerController->ShouldShowMouseCursor())
		{
			return EVisibility::HitTestInvisible;
		}
	}

	return EVisibility::Collapsed;
}

FLinearColor SLevelViewport::GetMouseCaptureLabelColorAndOpacity() const
{
	static const FName DefaultForegroundName("DefaultForeground");

	FSlateColor SlateColor = FAppStyle::GetSlateColor(DefaultForegroundName);
	FLinearColor Col = SlateColor.IsColorSpecified() ? SlateColor.GetSpecifiedColor() : FLinearColor::White; 

	float Alpha = 0.0f;
	
	if(ViewTransitionAnim.IsPlaying() && ViewTransitionType == EViewTransition::StartingPlayInEditor)
	{
		Alpha = ViewTransitionAnim.GetLerp();
	}
	else if(PIEOverlayAnim.IsPlaying())
	{
		Alpha = 1.0f - PIEOverlayAnim.GetLerp();
	}
	
	return Col.CopyWithNewOpacity(Alpha);
}

FText SLevelViewport::GetMouseCaptureLabelText() const
{
	if(ActiveViewport->HasMouseCapture())
	{
		// Default Shift+F1 if a valid chord is not found
		static FInputChord Chord(EKeys::F1, EModifierKey::Shift);
		TSharedPtr<FUICommandInfo> UICommand = FInputBindingManager::Get().FindCommandInContext(TEXT("PlayWorld"), TEXT("GetMouseControl"));
		if (UICommand.IsValid() && UICommand->GetFirstValidChord()->IsValidChord())
		{
			// Just pick the first key bind that is valid for a text suggestion
			Chord = UICommand->GetFirstValidChord().Get();
		}
		FFormatNamedArguments Args;
		Args.Add(TEXT("InputText"), Chord.GetInputText());
		return FText::Format( LOCTEXT("ShowMouseCursorLabel", "{InputText} for Mouse Cursor"), Args );
	}
	else
	{
		return LOCTEXT("GameMouseControlLabel", "Click for Mouse Control");
	}
}

void SLevelViewport::ShowMouseCaptureLabel(ELabelAnchorMode AnchorMode)
{
	static_assert((EVerticalAlignment)((ELabelAnchorMode::LabelAnchorMode_TopLeft / 3) + 1) == EVerticalAlignment::VAlign_Top && (EHorizontalAlignment)((ELabelAnchorMode::LabelAnchorMode_TopLeft % 3) + 1) == EHorizontalAlignment::HAlign_Left, "Alignment from ELabelAnchorMode error.");
	static_assert((EVerticalAlignment)((ELabelAnchorMode::LabelAnchorMode_TopCenter / 3) + 1) == EVerticalAlignment::VAlign_Top && (EHorizontalAlignment)((ELabelAnchorMode::LabelAnchorMode_TopCenter % 3) + 1) == EHorizontalAlignment::HAlign_Center, "Alignment from ELabelAnchorMode error.");
	static_assert((EVerticalAlignment)((ELabelAnchorMode::LabelAnchorMode_TopRight / 3) + 1) == EVerticalAlignment::VAlign_Top && (EHorizontalAlignment)((ELabelAnchorMode::LabelAnchorMode_TopRight % 3) + 1) == EHorizontalAlignment::HAlign_Right, "Alignment from ELabelAnchorMode error.");
	
	static_assert((EVerticalAlignment)((ELabelAnchorMode::LabelAnchorMode_CenterLeft / 3) + 1) == EVerticalAlignment::VAlign_Center && (EHorizontalAlignment)((ELabelAnchorMode::LabelAnchorMode_CenterLeft % 3) + 1) == EHorizontalAlignment::HAlign_Left, "Alignment from ELabelAnchorMode error.");
	static_assert((EVerticalAlignment)((ELabelAnchorMode::LabelAnchorMode_Centered / 3) + 1) == EVerticalAlignment::VAlign_Center && (EHorizontalAlignment)((ELabelAnchorMode::LabelAnchorMode_Centered % 3) + 1) == EHorizontalAlignment::HAlign_Center, "Alignment from ELabelAnchorMode error.");
	static_assert((EVerticalAlignment)((ELabelAnchorMode::LabelAnchorMode_CenterRight / 3) + 1) == EVerticalAlignment::VAlign_Center && (EHorizontalAlignment)((ELabelAnchorMode::LabelAnchorMode_CenterRight % 3) + 1) == EHorizontalAlignment::HAlign_Right, "Alignment from ELabelAnchorMode error.");
	
	static_assert((EVerticalAlignment)((ELabelAnchorMode::LabelAnchorMode_BottomLeft / 3) + 1) == EVerticalAlignment::VAlign_Bottom && (EHorizontalAlignment)((ELabelAnchorMode::LabelAnchorMode_BottomLeft % 3) + 1) == EHorizontalAlignment::HAlign_Left, "Alignment from ELabelAnchorMode error.");
	static_assert((EVerticalAlignment)((ELabelAnchorMode::LabelAnchorMode_BottomCenter / 3) + 1) == EVerticalAlignment::VAlign_Bottom && (EHorizontalAlignment)((ELabelAnchorMode::LabelAnchorMode_BottomCenter % 3) + 1) == EHorizontalAlignment::HAlign_Center, "Alignment from ELabelAnchorMode error.");
	static_assert((EVerticalAlignment)((ELabelAnchorMode::LabelAnchorMode_BottomRight / 3) + 1) == EVerticalAlignment::VAlign_Bottom && (EHorizontalAlignment)((ELabelAnchorMode::LabelAnchorMode_BottomRight % 3) + 1) == EHorizontalAlignment::HAlign_Right, "Alignment from ELabelAnchorMode error.");
	
	EVerticalAlignment VAlign = (EVerticalAlignment)((AnchorMode/3)+1);
	EHorizontalAlignment HAlign = (EHorizontalAlignment)((AnchorMode%3)+1);
	
	{
		ViewportOverlay->AddSlot()
		.HAlign(HAlign)
		.VAlign(VAlign)
		[
			SAssignNew( PIEOverlayBorder, SBorder )
			.BorderImage( FAppStyle::GetBrush("NoBorder") )
			.Visibility(this, &SLevelViewport::GetMouseCaptureLabelVisibility)
			.ColorAndOpacity( this, &SLevelViewport::GetMouseCaptureLabelColorAndOpacity )
			.ForegroundColor( FLinearColor::White )
			.Padding(15.0f)
			[
				SNew( SButton )
				.ButtonStyle( FAppStyle::Get(), "EditorViewportToolBar.MenuButton" )
				.IsFocusable(false)
				.ButtonColorAndOpacity( FSlateColor(FLinearColor::Black) )
				.ForegroundColor( FLinearColor::White )
				[
					BuildMouseCaptureWidget()
				]
			]
		];
	}
}

TSharedRef<SWidget> SLevelViewport::BuildMouseCaptureWidget()
{
	return SNew( SHorizontalBox )
		+ SHorizontalBox::Slot()
		.MaxWidth(32.f)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 2.0f, 2.0f, 2.0f)
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.MaxHeight(16.f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("LevelViewport.CursorIcon"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.Padding(2.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SLevelViewport::GetMouseCaptureLabelText)
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
			.ColorAndOpacity(FLinearColor::White)
		];
}

void SLevelViewport::HideMouseCaptureLabel()
{
	if (PIEOverlayBorder.IsValid())
	{
		ViewportOverlay->RemoveSlot(PIEOverlayBorder.ToSharedRef());
		PIEOverlayBorder.Reset();
	}
}

void SLevelViewport::ResetNewLevelViewFlags()
{
	const bool bUseSavedDefaults = true;
	OnUseDefaultShowFlags(bUseSavedDefaults);
}

void SLevelViewport::EndPlayInEditorSession()
{
	check( HasPlayInEditorViewport() );

	FSlateApplication::Get().UnregisterGameViewport();

	check( InactiveViewport.IsValid() );

	const bool bCheckMissingOverride = false;
	LevelViewportClient->RemoveRealtimeOverride(LOCTEXT("LevelViewport_RealTimeDisableOnPie", "Disable LevelViewport Realtime for PIE"), bCheckMissingOverride);

	PlayClient.Reset();

	if( IsPlayInEditorViewportActive() )
	{
		{
			TSharedPtr<FSceneViewport> GameViewport = ActiveViewport;
			ActiveViewport = InactiveViewport;
			ActiveViewport->OnPlayWorldViewportSwapped( *GameViewport );

			// We should be the only thing holding on to viewports
			check( GameViewport.IsUnique() );
		}

		// Ensure our active viewport is for level editing
		check( ActiveViewport->GetClient() == LevelViewportClient.Get() );

		// If we're going back to VR Editor, refresh the level viewport's render target so the HMD will present frames here
		if( GEngine->IsStereoscopic3D( ActiveViewport.Get() ) )
		{
			ActiveViewport->UpdateViewportRHI( false, ActiveViewport->GetSizeXY().X, ActiveViewport->GetSizeXY().Y, ActiveViewport->GetWindowMode(), PF_Unknown );
		}
		else
		{
			// Restore camera settings that may be adversely affected by PIE
			LevelViewportClient->RestoreCameraFromPIE();
			RedrawViewport(true);

			// Remove camera roll from any PIE camera applied in this viewport. A rolled camera is hard to use for editing
			LevelViewportClient->RemoveCameraRoll();
		}
	}

	if (ViewportOverlay->HasSlotWithZOrder(0))
	{
		ViewportOverlay->RemoveSlot(0);
	}

	// Restore the editor debug canvas now that ActiveViewport is back to the level editor viewport.
	EditorDebugCanvas->SetSceneViewport(ActiveViewport.Get());
	ViewportOverlay->AddSlot(0)
	[
		EditorDebugCanvas.ToSharedRef()
	];

	// Reset the inactive viewport
	InactiveViewport.Reset();

	// Viewport widget should begin drawing the editor viewport
	ViewportWidget->SetViewportInterface( ActiveViewport.ToSharedRef() );
	ViewportWidget->SetContent( InactiveViewportWidgetEditorContent );

	// No longer need to store the content 
	InactiveViewportWidgetEditorContent.Reset();

	HideMouseCaptureLabel();

	// Kick off a quick transition effect (border graphics)
	ViewTransitionType = EViewTransition::ReturningToEditor;
	ViewTransitionAnim = FCurveSequence( 0.0f, 1.5f, ECurveEaseFunction::CubicOut );
	bViewTransitionAnimPending = true;

	if (GetDefault<ULevelEditorPlaySettings>()->EnablePIEEnterAndExitSounds)
	{
		GEditor->PlayEditorSound( TEXT( "/Engine/EditorSounds/GamePreview/EndPlayInEditor_Cue.EndPlayInEditor_Cue" ) );
	}

	GEngine->BroadcastLevelActorListChanged();

	// Remove preview feature level delegate if set
	if (PIEPreviewFeatureLevelChangedHandle.IsValid())
	{
		CastChecked<UEditorEngine>(GEngine)->OnPreviewShaderPlatformChanged().Remove(PIEPreviewFeatureLevelChangedHandle);
		PIEPreviewFeatureLevelChangedHandle.Reset();
	}
}

void SLevelViewport::SwapViewportsForSimulateInEditor()
{
	// Ensure our active viewport was the play in editor viewport
	check( IsPlayInEditorViewportActive() );
	
	// Remove the mouse control label - not relevant for SIE
	HideMouseCaptureLabel();

	// Unregister the game viewport with slate which will release mouse capture and lock
	FSlateApplication::Get().UnregisterGameViewport();

	// Swap between the active and inactive viewport
	TSharedPtr<FSceneViewport> TempViewport = ActiveViewport;
	ActiveViewport = InactiveViewport;
	InactiveViewport = TempViewport;

	ViewportWidget->SetContent( InactiveViewportWidgetEditorContent );

	// Resize the viewport to be the same size the previously active viewport
	// When starting in immersive mode its possible that the viewport has not been resized yet
	ActiveViewport->OnPlayWorldViewportSwapped( *InactiveViewport );

	if (TSharedPtr<IGameLayerManager> GameLayerManager = PlayClient.IsValid() ? PlayClient->GetGameLayerManager() : nullptr)
	{
		GameLayerManager->SetSceneViewport(ActiveViewport.Get());
	}

	ViewportWidget->SetViewportInterface( ActiveViewport.ToSharedRef() );

	// Kick off a quick transition effect (border graphics)
	ViewTransitionType = EViewTransition::StartingSimulate;
	ViewTransitionAnim = FCurveSequence( 0.0f, 1.5f, ECurveEaseFunction::CubicOut );
	bViewTransitionAnimPending = true;
	if (GetDefault<ULevelEditorPlaySettings>()->EnablePIEEnterAndExitSounds)
	{
		GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/GamePreview/PossessPlayer_Cue.PossessPlayer_Cue"));
	}
}


void SLevelViewport::SwapViewportsForPlayInEditor()
{
	// Ensure our inactive viewport was the play in editor viewport
	check( !IsPlayInEditorViewportActive() && HasPlayInEditorViewport() );
	
	// Put the mouse control label up again.
	ULevelEditorPlaySettings const* EditorPlayInSettings = GetDefault<ULevelEditorPlaySettings>();
	check(EditorPlayInSettings);

	// Swap between the active and inactive viewport
	TSharedPtr<FSceneViewport> TempViewport = ActiveViewport;
	ActiveViewport = InactiveViewport;
	InactiveViewport = TempViewport;

	// Resize the viewport to be the same size the previously active viewport
	// When starting in immersive mode its possible that the viewport has not been resized yet
	ActiveViewport->OnPlayWorldViewportSwapped( *InactiveViewport );

	if (TSharedPtr<IGameLayerManager> GameLayerManager = PlayClient.IsValid() ? PlayClient->GetGameLayerManager() : nullptr)
	{
		GameLayerManager->SetSceneViewport(ActiveViewport.Get());
	}

	InactiveViewportWidgetEditorContent = ViewportWidget->GetContent();
	ViewportWidget->SetViewportInterface( ActiveViewport.ToSharedRef() );

	// Register the game viewport with slate which will capture the mouse and lock it to the viewport
	FSlateApplication::Get().RegisterGameViewport( ViewportWidget.ToSharedRef() );

	// Kick off a quick transition effect (border graphics)
	ViewTransitionType = EViewTransition::StartingPlayInEditor;
	ViewTransitionAnim = FCurveSequence( 0.0f, 1.5f, ECurveEaseFunction::CubicOut );
	bViewTransitionAnimPending = true;

	if (EditorPlayInSettings->EnablePIEEnterAndExitSounds)
	{
		GEditor->PlayEditorSound( TEXT( "/Engine/EditorSounds/GamePreview/EjectFromPlayer_Cue.EjectFromPlayer_Cue" ) );
	}
}


void SLevelViewport::OnSimulateSessionStarted()
{
	// Kick off a quick transition effect (border graphics)
	ViewTransitionType = EViewTransition::StartingSimulate;
	ViewTransitionAnim = FCurveSequence( 0.0f, 1.5f, ECurveEaseFunction::CubicOut );
	bViewTransitionAnimPending = true;
	if (GetDefault<ULevelEditorPlaySettings>()->EnablePIEEnterAndExitSounds)
	{
		GEditor->PlayEditorSound( TEXT( "/Engine/EditorSounds/GamePreview/StartSimulate_Cue.StartSimulate_Cue" ) );
	}
	 
	// Make sure the viewport's hit proxies are invalidated.  If not done, clicking in the viewport could select an editor world actor
	ActiveViewport->InvalidateHitProxy();
}


void SLevelViewport::OnSimulateSessionFinished()
{
	// Kick off a quick transition effect (border graphics)
	ViewTransitionType = EViewTransition::ReturningToEditor;
	ViewTransitionAnim = FCurveSequence( 0.0f, 1.5f, ECurveEaseFunction::CubicOut );
	bViewTransitionAnimPending = true;
	if (GetDefault<ULevelEditorPlaySettings>()->EnablePIEEnterAndExitSounds)
	{
		GEditor->PlayEditorSound( TEXT( "/Engine/EditorSounds/GamePreview/EndSimulate_Cue.EndSimulate_Cue" ) );
	}

	// Make sure the viewport's hit proxies are invalidated.  If not done, clicking in the viewport could select a pie world actor
	ActiveViewport->InvalidateHitProxy();
}

FText SLevelViewport::GetLockedIconToolTip() const
{
	FText ToolTipText;
	if (IsAnyActorLocked())
	{
		ToolTipText = FText::Format( LOCTEXT("ActorLockedIcon_ToolTip", "Viewport Locked to {0}"), FText::FromString( LevelViewportClient->GetActiveActorLock().Get()->GetActorLabel() ) );
	}

	return ToolTipText;
}

UWorld* SLevelViewport::GetWorld() const
{
	return ParentLevelEditor.IsValid() ? ParentLevelEditor.Pin()->GetWorld() : nullptr;
}

FReply SLevelViewport::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	// SEditorViewport redirects all clicks to focus the viewport widget.
	// This means clicking on the toolbar in PIE captures the mouse cursor for the game.
	return FReply::Unhandled();
}

void SLevelViewport::ToggleInViewportContextMenu()
{
	USelection* ActorSelection = GEditor->GetSelectedActors();
	if (ActorSelection->Num())
	{
		if (!bIsInViewportMenuShowing)
		{
			// Set up the correct menu location first
			if (!bIsInViewportMenuInitialized)
			{
				FVector2D NewViewportContextMenuLocation = GetDefault<ULevelEditorViewportSettings>()->LastInViewportMenuLocation;
				if (!NewViewportContextMenuLocation.IsZero())
				{
					UpdateInViewportMenuLocation(NewViewportContextMenuLocation);
				}
				else
				{
					AActor* SelectedActor = ActorSelection->GetTop<AActor>();
					FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
						GetActiveViewport(),
						LevelViewportClient->GetScene(),
						LevelViewportClient->EngineShowFlags)
						.SetRealtimeUpdate(IsRealtime()));
					// SceneView is deleted with the ViewFamily
					FSceneView* SceneView = LevelViewportClient->CalcSceneView(&ViewFamily);
					const float InvDpiScale = 1.0f / LevelViewportClient->GetDPIScale();
					FVector2D ScreenPos;
					SceneView->WorldToPixel(SelectedActor->GetTransform().GetLocation(), ScreenPos);
					ScreenPos *= InvDpiScale;
					const float EdgeFactor = 0.85f;
					const float MinX = static_cast<float>(SceneView->UnscaledViewRect.Width()) * InvDpiScale * (1.f - EdgeFactor);
					const float MinY = static_cast<float>(SceneView->UnscaledViewRect.Height()) * InvDpiScale * (1.f - EdgeFactor);
					const float MaxX = static_cast<float>(SceneView->UnscaledViewRect.Width()) * InvDpiScale * EdgeFactor;
					const float MaxY = (static_cast<float>(SceneView->UnscaledViewRect.Height()) * InvDpiScale * EdgeFactor);
					const bool bOutside = ScreenPos.X < MinX || ScreenPos.X > MaxX || ScreenPos.Y < MinY || ScreenPos.Y > MaxY;
					if (bOutside)
					{
						ScreenPos.X = (static_cast<float>(SceneView->UnscaledViewRect.Width()) * InvDpiScale) / 2.0f;
						ScreenPos.Y = (static_cast<float>(SceneView->UnscaledViewRect.Height()) * InvDpiScale) / 2.0f;
					}
					UpdateInViewportMenuLocation(ScreenPos);
				}
				bIsInViewportMenuInitialized = true;
			}

			bIsInViewportMenuShowing = true;
			InViewportMenu = SNew(SInViewportDetails)
				.InOwningViewport(SharedThis(this))
				.InOwningLevelEditor(ParentLevelEditor.Pin());
			InViewportMenuWrapper = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Left)
				.Padding(TAttribute<FMargin>(this, &SLevelViewport::GetContextMenuPadding))
				[
					InViewportMenu.ToSharedRef()
				];

			// Immediately update it (otherwise it will appear empty)
			{
				TArray<UObject*> SelectedActors;
				for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
				{
					AActor* Actor = static_cast<AActor*>(*It);
					checkSlow(Actor->IsA(AActor::StaticClass()));

					if (IsValidChecked(Actor))
					{
						SelectedActors.Add(Actor);
					}
				}

				const bool bForceRefresh = true;
				InViewportMenu->SetObjects(SelectedActors, bForceRefresh);
			}
			AddOverlayWidget(InViewportMenuWrapper.ToSharedRef());
		}
		else
		{
			HideInViewportContextMenu();
		}
	
	}
}

void SLevelViewport::HideInViewportContextMenu()
{
	if (InViewportMenuWrapper.IsValid())
	{
		RemoveOverlayWidget(InViewportMenuWrapper.ToSharedRef());
	}
	bIsInViewportMenuShowing = false;
	InViewportMenu.Reset();
}

bool SLevelViewport::CanToggleInViewportContextMenu()
{
	return SLevelViewport::bInViewportMenuEnabled;
}

void SLevelViewport::EnableInViewportMenu()
{
	SLevelViewport::bInViewportMenuEnabled = !SLevelViewport::bInViewportMenuEnabled;
}

FMargin SLevelViewport::GetContextMenuPadding() const
{
	return FMargin(InViewportContextMenuLocation.X, InViewportContextMenuLocation.Y, 0, 0);
}

void SLevelViewport::RemoveActorPreview( int32 PreviewIndex, AActor* Actor, const bool bRemoveFromDesktopViewport /*=true */ )
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	if (!bRemoveFromDesktopViewport)
	{
		if (ActorPreviews.IsValidIndex(PreviewIndex))
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			VREditorModule.UpdateActorPreview(SNullWidget::NullWidget, PreviewIndex, Actor, ActorPreviews[PreviewIndex].bIsPanelDetached);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
	else
	{
		// Remove widget from viewport overlay
		ActorPreviewHorizontalBox->RemoveSlot(ActorPreviews[PreviewIndex].PreviewWidget.ToSharedRef());
	}
	if (ActorPreviews.IsValidIndex(PreviewIndex))
	{
		// Remove from our list of actor previews.  This will destroy our level viewport client and viewport widget.
		ActorPreviews.RemoveAt(PreviewIndex);
	}
}

void SLevelViewport::AddOverlayWidget(TSharedRef<SWidget> OverlaidWidget, int32 ZOrder)
{
	ViewportOverlay->AddSlot(ZOrder)
	[
		OverlaidWidget
	];
}

void SLevelViewport::RemoveOverlayWidget(TSharedRef<SWidget> OverlaidWidget)
{
	ViewportOverlay->RemoveSlot(OverlaidWidget);
}

bool SLevelViewport::CanProduceActionForCommand(const TSharedRef<const FUICommandInfo>& Command) const
{
	return IsActiveLevelViewport();
}

void SLevelViewport::LockActorInternal(AActor* NewActorToLock)
{
	if (NewActorToLock != nullptr)
	{
		LevelViewportClient->SetActorLock(NewActorToLock);
		if (LevelViewportClient->IsPerspective() && LevelViewportClient->GetActiveActorLock().IsValid())
		{
			// Store perspective camera transform before piloting
			CachedPerspectiveCameraTransform = LevelViewportClient->GetViewTransform();
			LevelViewportClient->MoveCameraToLockedActor();
		}
	}

	// Make sure the inset preview is closed if we are locking a camera that was already part of the selection set and thus being previewed.
	OnPreviewSelectedCamerasChange();
}

bool SLevelViewport::GetCameraInformationFromActor(AActor* Actor, FMinimalViewInfo& out_CameraInfo)
{
	//
	//@TODO: CAMERA: Support richer camera interactions in SIE; this may shake out naturally if everything uses camera components though

	bool bFoundCamInfo = false;
	if (UActorComponent* ViewComponent = FLevelEditorViewportClient::FindViewComponentForActor(Actor))
	{
		bFoundCamInfo = ViewComponent->GetEditorPreviewInfo(/*DeltaTime =*/0.0f, out_CameraInfo);
		ensure(bFoundCamInfo);
	}
	return bFoundCamInfo;
}

bool SLevelViewport::CanGetCameraInformationFromActor(AActor* Actor)
{
	FMinimalViewInfo CameraInfo;

	return GetCameraInformationFromActor(Actor, /*out*/ CameraInfo);
}

void SLevelViewport::OnFloatingButtonClicked()
{
	// if one of the viewports floating buttons has been clicked, update the global viewport ptr
	LevelViewportClient->SetLastKeyViewport();
}

void SLevelViewport::RemoveAllPreviews(const bool bRemoveFromDesktopViewport /**= true*/)
{
	// Clean up any actor preview viewports
	for (FViewportActorPreview& ActorPreview : ActorPreviews)
	{
		ActorPreview.bIsPinned = false;
	}
	PreviewActors(TArray< AActor* >(), bRemoveFromDesktopViewport);
}

#undef LOCTEXT_NAMESPACE
