// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSurfaceProperties.h"
#include "Widgets/Text/STextBlock.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Lightmass/LightmassPrimitiveSettingsObject.h"
#include "UnrealEdGlobals.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "SurfaceIterators.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Engine/Polys.h"

#define LOCTEXT_NAMESPACE "SSurfaceProperties"


void SSurfaceProperties::Construct( const FArguments& InArgs )
{
	bUseNegativePanningU = false;
	bUseNegativePanningV = false;
	bUseNegativeRotation = false;

	CachedScalingValueU = 1.0f;
	CachedScalingValueV = 1.0f;

	// Initialize scale fields according to the scale of the first selected surface
	if (TSelectedSurfaceIterator<> It(GetWorld()); It)
	{
		FBspSurf* Surf = *It;
		UModel* Model = It.GetModel();

		const FVector3f TextureU(Model->Vectors[Surf->vTextureU]);
		const FVector3f TextureV(Model->Vectors[Surf->vTextureV]);

		const float TextureUSize = TextureU.Size();
		const float TextureVSize = TextureV.Size();

		if (!FMath::IsNearlyZero(TextureUSize))
		{
			CachedScalingValueU = 1.0f / TextureUSize;
		}

		if (!FMath::IsNearlyZero(TextureVSize))
		{
			CachedScalingValueV = 1.0f / TextureVSize;
		}
	}

	bPreserveScaleRatio = false;
	bUseRelativeScaling = false;

	GConfig->GetBool(TEXT("SelectionDetails"), TEXT("PreserveScaleRatio"), bPreserveScaleRatio, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SelectionDetails"), TEXT("UseRelativeScaling"), bUseRelativeScaling, GEditorPerProjectIni);

	static const float ScalingValues[] = { 1.0f / 16, 1.0f / 8, 1.0f / 4, 1.0f / 2, 1, 2, 4, 8, 16 };
	for(int Idx = 0; Idx < UE_ARRAY_COUNT(ScalingValues); Idx++)
	{
		ScalingFactors.Add(MakeShareable(new FString(FString::Printf(TEXT("%f"),ScalingValues[Idx]))));
	}

	static const FSlateBrush* BorderStyle =  FAppStyle::GetBrush( "DetailsView.GroupSection" );
	static const FLinearColor BorderColor = FLinearColor(.2f,.2f,.2f,.2f);

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f,0.f,0.f,5.f)
		[
			SNew(SBorder)
			.BorderBackgroundColor( BorderColor )
			.BorderImage( BorderStyle )
			.Padding(10.f)
			.AddMetaData<FTagMetaData>(TEXT("DetailsView.TexturePan"))
			[
				ConstructTexturePan()
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f,0.f,0.f,5.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(3)
			.Padding(0.f,0.f,5.f,0.f)
			[
				SNew(SBorder)
				.BorderBackgroundColor( BorderColor )
				.BorderImage( BorderStyle )
				.Padding(10.f)
				.AddMetaData<FTagMetaData>(TEXT("DetailsView.TextureRotate"))
				[
					ConstructTextureRotate()
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(2.f)
			[
				SNew(SBorder)
				.BorderBackgroundColor( BorderColor )
				.BorderImage( BorderStyle )
				.Padding(10.f)
				.AddMetaData<FTagMetaData>(TEXT("DetailsView.TextureFlip"))
				[
					ConstructTextureFlip()
				]
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f,0.f,0.f,5.f)
		[
			SNew(SBorder)
			.BorderBackgroundColor( BorderColor )
			.BorderImage( BorderStyle )
			.Padding(10.f)
			.AddMetaData<FTagMetaData>(TEXT("DetailsView.TextureScale"))
			[
				ConstructTextureScale()
			]
		]
			
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f,0.f,0.f,5.f)
		[
			SNew(SBorder)
			.BorderBackgroundColor( BorderColor )
			.BorderImage( BorderStyle )
			.Padding(10.f)
			.AddMetaData<FTagMetaData>(TEXT("DetailsView.ConstructLighting"))
			[
				ConstructLighting()
			]
		]
	];
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SSurfaceProperties::ConstructTexturePan()
{
	TSharedRef<SVerticalBox> ParentBox = SNew(SVerticalBox);
	
	ParentBox->AddSlot()
	.AutoHeight()
	.Padding(0.f,0.f,0.f,5.f)
	[
		SNew(STextBlock)
		.Text( LOCTEXT("Pan", "Pan:") )
	];

	TSharedPtr<SHorizontalBox> HorizontalBox;

	ParentBox->AddSlot()
	.AutoHeight()
	[
		SAssignNew(HorizontalBox,SHorizontalBox)
	];

	TSharedPtr<SVerticalBox> VerticalBox;
	HorizontalBox->AddSlot()
	.AutoWidth()
	[
		SAssignNew(VerticalBox,SVerticalBox)
	];

	static const TextureCoordChannel Channels[] = {UChannel,VChannel};
	for (int i = 0; i < 2; i++)
	{
		VerticalBox->AddSlot()
		.FillHeight(1)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		.Padding(5.f)
		[
			SNew( SCheckBox )
			.IsChecked( this, &SSurfaceProperties::IsUsingNegativePanning, Channels[i] )
			.OnCheckStateChanged( this, &SSurfaceProperties::OnTogglePanningDirection, Channels[i])
			.Style( FAppStyle::Get(), "TransparentCheckBox" )
			.ToolTipText( LOCTEXT("InvertPanningDirection", "Toggle panning direction.") )
			[
				SNew( SImage )
				.Image( this, &SSurfaceProperties::GetTogglePanDirectionImage, Channels[i] )
				.ColorAndOpacity( FSlateColor::UseForeground() )
			]
		];
	}
	
	TSharedPtr<SUniformGridPanel> GridBox;	
	HorizontalBox->AddSlot()
	.FillWidth(1.f)
	[
		SAssignNew(GridBox,SUniformGridPanel)
	];

	const static FText ButtonFields[] = {FText::FromString("1/256"), FText::FromString("1/64"), FText::FromString("1/16"), FText::FromString("1/4")};
	const static int32 ButtonIncriments[] = {1,4,16,64};
	for (int32 i = 0; i < UE_ARRAY_COUNT(ButtonFields); i++)
	{
		GridBox->AddSlot(i, 0)
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(ButtonFields[i])
			.ToolTipText(LOCTEXT("PanUTooltip", "Pans U texture coordinate"))
			.OnClicked(this,&SSurfaceProperties::OnPanTexture,ButtonIncriments[i],SSurfaceProperties::UChannel)
		];
	
		GridBox->AddSlot(i, 1)
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(ButtonFields[i])
			.ToolTipText(LOCTEXT("PanVTooltip", "Pans V texture coordinate"))
			.OnClicked(this,&SSurfaceProperties::OnPanTexture,ButtonIncriments[i],SSurfaceProperties::VChannel)
		];
	}
	
	// Create the last two custom buttons on the end (there will always be two)
	for (int i = 0; i < 2; i++)
	{
		TSharedPtr<SComboButton> ComboButton;
		TSharedPtr<SWidget> NumberBox;
		GridBox->AddSlot(UE_ARRAY_COUNT(ButtonFields),i)
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			SAssignNew(ComboButton,SComboButton)
			.VAlign(VAlign_Fill)
			.ButtonContent()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PanToolCustomPan", "---"))
					.ToolTipText(LOCTEXT("PanToolCustomPanToolTip", "Set Custom pan amount"))
				]
			] 
			.MenuContent()
			[
				SNew(SBorder)
				.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
				[
					SAssignNew(NumberBox, SNumericEntryBox<int32>)
					.OnValueCommitted( this, &SSurfaceProperties::OnCustomPanValueCommitted,(TextureCoordChannel)i)
				]
			]
		];

		ComboButton->SetMenuContentWidgetToFocus(NumberBox);

		CustomPanButtoms.Add(TWeakPtr<SComboButton>(ComboButton));
	}

	return ParentBox;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SSurfaceProperties::ConstructTextureRotate()
{
	
	TSharedRef<SVerticalBox> Parent = SNew(SVerticalBox);
	
	Parent->AddSlot()
	.AutoHeight()
	.Padding(0.f,0.f,5.f,5.f)
	[
		SNew (STextBlock)
		.Text(LOCTEXT("RotateTitle", "Rotate:"))
	];

	TSharedPtr<SHorizontalBox> RotateBox;
	Parent->AddSlot()
	.AutoHeight()
	[
		SAssignNew(RotateBox,SHorizontalBox)
	];

	RotateBox->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.Padding(5.f)
	[
		SNew( SCheckBox )
		.IsChecked( this, &SSurfaceProperties::IsUsingNegativeRotation )
		.OnCheckStateChanged( this, &SSurfaceProperties::OnToggleRotationDirection )
		.Style( FAppStyle::Get(), "TransparentCheckBox" )
		.ToolTipText( LOCTEXT("InvertRotation", "Toggle Rotation direction.") )
		[
			SNew( SImage )
			.Image( this, &SSurfaceProperties::GetToggleRotationDirectionImage )
			.ColorAndOpacity( FSlateColor::UseForeground() )
		]
	];

	// Rotation button fields:
	const static FText ButtonFields[] = { LOCTEXT("Rotate45Degrees", "45"), LOCTEXT("Rotate90Degrees", "90"), LOCTEXT("RotateCustom", "---") };
	const static int32 RotationValues[] = {45, 90, -1};
	const static RotationAction RotationActions[] = {Rotate,Rotate,RotateCustom};
	
	// create rotation controls
	for (int Idx = 0; Idx < UE_ARRAY_COUNT(RotationValues); Idx++)
	{
		if (RotationActions[Idx] == RotateCustom)
		{
			TSharedPtr<SComboButton> CurrentButton;
			TSharedPtr<SWidget> NumberBox;
			RotateBox->AddSlot()
			.FillWidth(1.f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(CurrentButton,SComboButton)
				.VAlign(VAlign_Center)
				.ButtonContent()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(ButtonFields[Idx])
						.ToolTipText(LOCTEXT("RotateToolTip_Custom", "Sets a custom rotate amount"))
					]
				] 
				.MenuContent()
				[
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
					[
						SAssignNew(NumberBox, SNumericEntryBox<int32>)
						.OnValueCommitted( this, &SSurfaceProperties::OnCustomRotateValueCommitted)
					]
				]
			];

			CurrentButton->SetMenuContentWidgetToFocus(NumberBox);
			CustomRotationButton = TWeakPtr<SComboButton>(CurrentButton);
		}
		else
		{
			RotateBox->AddSlot()
			.FillWidth(1.f)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[	
				SNew(SButton)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Text(ButtonFields[Idx])
				.ToolTipText(LOCTEXT("RotateToolTip", "Rotate texture"))
				.OnClicked(this,&SSurfaceProperties::OnRotateTexture,RotationValues[Idx],RotationActions[Idx])
			];
		}
	}

	return Parent;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> SSurfaceProperties::ConstructTextureFlip()
{
	TSharedPtr<SHorizontalBox> FlipBox;

	TSharedRef<SVerticalBox> Parent = SNew(SVerticalBox);
	Parent->AddSlot()
	.AutoHeight()
	.Padding(0.f,0.f,5.f,5.f)
	[
		SNew (STextBlock)
		.Text(LOCTEXT("FlipTitle", "Flip:"))
	];

	Parent->AddSlot()
	.AutoHeight()
	[
		SAssignNew(FlipBox,SHorizontalBox)
	];

	// Flip button fields:
	const static FText ButtonFields[] = {LOCTEXT("RotateFlipU", "Flip U"), LOCTEXT("RotateFlipV", "Flip V")};
	const static TextureCoordChannel TextureCoordinateChannels[] = {UChannel,VChannel};

	// create flip controls
	for (int Idx = 0; Idx < UE_ARRAY_COUNT(ButtonFields); Idx++)
	{
		FlipBox->AddSlot()
		.FillWidth(1)
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[	
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ContentPadding(FMargin(0.f, 5.f))
			.Text(ButtonFields[Idx])
			.ToolTipText(LOCTEXT("FlipToolTip", "Flip texture"))
			.OnClicked(this,&SSurfaceProperties::OnFlipTexture,TextureCoordinateChannels[Idx])
		];
	}

	return Parent;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SSurfaceProperties::ConstructTextureScale()
{
	TSharedRef<SVerticalBox> Parent =  SNew(SVerticalBox);
	
	Parent->AddSlot()
	.AutoHeight()
	.Padding(0.f,0.f,0.f,5.f)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SHyperlink)
					.Text(this, &SSurfaceProperties::GetScalingLabel)
					.ToolTipText(LOCTEXT("ScalingRelativeToggle", "Toggle between Absolute and Relative scaling"))
					.OnNavigate( this, &SSurfaceProperties::OnScaleLabelClicked )
					.TextStyle(FAppStyle::Get(), "DetailsView.HyperlinkStyle")
			]
	];

	TSharedPtr<SHorizontalBox> Controls;
	Parent->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	[
		SAssignNew(Controls,SHorizontalBox)
	];

	TSharedPtr<SComboButton> NewComboButton;
	TSharedPtr<SListView<TSharedPtr<FString>>> NewListView;

	FString ControlLabels[] = {"U", "V"};
	TextureCoordChannel Channels[] = {UChannel,VChannel};
	
	for(int idx = 0; idx < UE_ARRAY_COUNT(ControlLabels); idx++)
	{
		Controls->AddSlot()
		.HAlign(HAlign_Fill)
		.FillWidth(1.f)
		[
			SAssignNew(NewComboButton, SComboButton)
			.ContentPadding(0.f)
			.HAlign(HAlign_Fill)
			.ButtonContent()
			[
				SNew(SNumericEntryBox<float>)
				.OnValueCommitted( this,&SSurfaceProperties::OnScaleValueCommitted,Channels[idx])
				.Value( this, &SSurfaceProperties::OnGetScalingValue,Channels[idx] )
				.LabelVAlign(VAlign_Center)
				.Label()
				[
					SNew(STextBlock)
					.Text(FText::FromString(ControlLabels[idx]))
				]
			]
			.MenuContent()
			[
				SAssignNew(NewListView, SListView<TSharedPtr<FString>>)
				.ListItemsSource(&ScalingFactors)
				.OnGenerateRow(this, &SSurfaceProperties::OnGenerateScaleTableRow)
				.OnSelectionChanged(this, &SSurfaceProperties::OnScaleSelectionChanged,Channels[idx])
			]
		];

		ScalingComboButton.Add(NewComboButton);
		ScalingListViews.Add(NewListView);
	}

	Controls->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)	
	.HAlign(HAlign_Left)
	[
		SNew( SCheckBox )
		.IsChecked( this, &SSurfaceProperties::IsPreserveScaleRatioChecked )
		.OnCheckStateChanged( this, &SSurfaceProperties::OnPreserveScaleRatioToggled )
		.Style( FAppStyle::Get(), "TransparentCheckBox" )
		.ToolTipText( LOCTEXT("PreserveScaleSurfaceToolTip", "When locked changes to either scaling value will be applied to the other.") )
		[
			SNew( SImage )
			.Image( this, &SSurfaceProperties::GetPreserveScaleRatioImage )
			.ColorAndOpacity( FSlateColor::UseForeground() )
		]
	];

	Controls->AddSlot()
	.FillWidth(1.f);

	Controls->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	[
		SNew(SButton)
		.Text(LOCTEXT("ApplyScaling", "Apply"))
		.ToolTipText(LOCTEXT("ApplyScalingToolTip", "Apply scaling to selected surfaces"))
		.OnClicked(this, &SSurfaceProperties::OnApplyScaling)
	];

	return Parent;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> SSurfaceProperties::ConstructLighting()
{
	LevelLightmassSettingsObjects.Empty();
	SelectedLightmassSettingsObjects.Empty();
	
	if ( GetWorld() )
	{
		for ( int32 LevelIndex = 0 ; LevelIndex < GetWorld()->GetNumLevels() ; ++LevelIndex )
		{
			const ULevel* Level = GetWorld()->GetLevel(LevelIndex);
			const UModel* Model = Level->Model;

			TLightmassSettingsObjectArray ObjArray;
			for(int32 SurfaceIndex = 0; SurfaceIndex < Model->Surfs.Num();SurfaceIndex++)
			{
				const FBspSurf&	Surf =  Model->Surfs[SurfaceIndex];

				if(Surf.PolyFlags & PF_Selected)
				{
					FLightmassPrimitiveSettings TempSettings = Model->LightmassSettings[Surf.iLightmassIndex];
					int32 FoundIndex = INDEX_NONE;
					for (int32 CheckIndex = 0; CheckIndex < ObjArray.Num(); CheckIndex++)
					{
						if (ObjArray[CheckIndex]->LightmassSettings == TempSettings)
						{
							FoundIndex = CheckIndex;
							break;
						}
					}
					if (FoundIndex == INDEX_NONE)
					{
						ULightmassPrimitiveSettingsObject* LightmassSettingsObject = NewObject<ULightmassPrimitiveSettingsObject>();
						LightmassSettingsObject->LightmassSettings = TempSettings;
						ObjArray.Add(LightmassSettingsObject);
						SelectedLightmassSettingsObjects.Add(LightmassSettingsObject);
					}
				}
			}
			LevelLightmassSettingsObjects.Add(ObjArray);
		}
	}

	// setup UI
	TSharedRef<SVerticalBox> Parent = SNew(SVerticalBox);

	Parent->AddSlot()
	.AutoHeight()
	.Padding(0.f,0.f,0.f,5.f)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("LightingTitle", "Lighting:"))
	];

	Parent->AddSlot()
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f,0.f,10.f,5.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LightingLightMapResolution", "Lightmap Resolution:"))
		]

		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.FillWidth(1.f)
		.Padding(0.f,0.f,0.f,5.f)
		[
			SNew(SNumericEntryBox<float>)
			.OnValueCommitted(this, &SSurfaceProperties::OnLightmapResolutionCommitted)
			.UndeterminedString( LOCTEXT("MultipleValues", "Multiple Values") )
			.Value(this, &SSurfaceProperties::GetLightmapResolutionValue)
		]

		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.FillWidth(1.f)
	];

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.bAllowSearch = false;
	Args.NotifyHook = this;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyView = PropertyModule.CreateDetailView(Args);
	PropertyView->SetObjects( ObjectPtrDecay(SelectedLightmassSettingsObjects) );

	Parent->AddSlot()
	.AutoHeight()
	[
		PropertyView.ToSharedRef()
	];

	return Parent;
}

FReply SSurfaceProperties::OnPanTexture(int32 PanAmount, TextureCoordChannel Channel)
{
	int32 PanV = 0;
	int32 PanU = 0;
	bool InvertPanDirection = false;
	if (Channel == UChannel)
	{
		PanU = PanAmount;
		InvertPanDirection = bUseNegativePanningU;
	}
	else
	{
		PanV = PanAmount;
		InvertPanDirection = bUseNegativePanningV;
	}

	const float Mod = InvertPanDirection? -1.f : 1.f;
	GUnrealEd->Exec( GetWorld(), *FString::Printf( TEXT("POLY TEXPAN U=%f V=%f"), static_cast<float>(PanU) * Mod, static_cast<float>(PanV) * Mod ) );

	return FReply::Handled();
}

FReply SSurfaceProperties::OnRotateTexture(int32 RotationAmount, RotationAction Action)
{
	const float Mod =  bUseNegativeRotation ? -1.f : 1.f;

	const float RotateRadians = static_cast<float>(RotationAmount) / 180.0f * PI;

	const float UU = cosf(RotateRadians);
	const float VV = UU;
	const float UV = -sinf(RotateRadians) * Mod;
	const float VU = sinf(RotateRadians) * Mod;
	GUnrealEd->Exec( GetWorld(), *FString::Printf( TEXT("POLY TEXMULT UU=%f VV=%f UV=%f VU=%f"), UU, VV, UV, VU ) );

	return FReply::Handled();
}

FReply SSurfaceProperties::OnFlipTexture(TextureCoordChannel Channel)
{
	if (Channel == UChannel)
	{
		GUnrealEd->Exec( GetWorld(), TEXT("POLY TEXMULT UU=-1 VV=1") );
	}
	else
	{
		GUnrealEd->Exec( GetWorld(), TEXT("POLY TEXMULT UU=1 VV=-1") );
	}

	return FReply::Handled();
}

void SSurfaceProperties::OnScaleTexture(float InScaleU, float InScaleV, bool InRelative)
{
	if( InScaleU == 0.f )
	{
		InScaleU = 1.f;
	}
	if( InScaleV == 0.f )
	{
		InScaleV = 1.f;
	}

	InScaleU = 1.0f / InScaleU;
	InScaleV = 1.0f / InScaleV;

	GUnrealEd->Exec( GetWorld(), *FString::Printf( TEXT("POLY TEXSCALE %s UU=%f VV=%f"), InRelative?TEXT("RELATIVE"):TEXT(""), InScaleU, InScaleV ) );
}

void SSurfaceProperties::AddReferencedObjects(FReferenceCollector& Collector)
{
	// we need to serialize all the UObjects in case of a GC
	for( int32 Index = 0; Index < LevelLightmassSettingsObjects.Num(); Index++ )
	{
		TLightmassSettingsObjectArray& SettingsArray = LevelLightmassSettingsObjects[ Index ];
		for( int32 SettingIndex = 0; SettingIndex < SettingsArray.Num(); SettingIndex++ )
		{
			Collector.AddReferencedObject( SettingsArray[ SettingIndex ] );
		}
	}
	for( int32 Index = 0; Index < SelectedLightmassSettingsObjects.Num(); Index++ )
	{
		Collector.AddReferencedObject( SelectedLightmassSettingsObjects[ Index ] );
	}
}

FString SSurfaceProperties::GetReferencerName() const
{
	return TEXT("SSurfaceProperties");
}

TOptional<float> SSurfaceProperties::GetLightmapResolutionValue() const
{
	float LightMapScale = 0.0f;
	int32 SelectedSurfaceCount = 0;
	bool bMultipleValues = false;
	if ( GetWorld() )
	{
		for ( int32 LevelIndex = 0 ; LevelIndex < GetWorld()->GetNumLevels() ; ++LevelIndex )
		{
			const ULevel* Level = GetWorld()->GetLevel(LevelIndex);
			const UModel* Model = Level->Model;

			TLightmassSettingsObjectArray ObjArray;
			for(int32 SurfaceIndex = 0; SurfaceIndex < Model->Surfs.Num();SurfaceIndex++)
			{
				const FBspSurf&	Surf =  Model->Surfs[SurfaceIndex];

				if(Surf.PolyFlags & PF_Selected)
				{
					if(SelectedSurfaceCount == 0)
					{
						LightMapScale = Surf.LightMapScale;
					}
					else if(SelectedSurfaceCount > 0 && LightMapScale != Surf.LightMapScale)
					{
						bMultipleValues = true;
					}

					SelectedSurfaceCount++;
				}
			}
		}
	}

	if(bMultipleValues)
	{
		// Return an unset value so it displays the undetermined indicator
		return TOptional<float>();
	}
	else
	{
		return LightMapScale;
	}
}

void SSurfaceProperties::OnLightmapResolutionCommitted(float NewValue, ETextCommit::Type CommitInfo)
{
	const float LightMapScale = FMath::Clamp<float>(NewValue, 0.1f, 65536.0f);

	bool bSurfacesDirty = false;
	for ( int32 LevelIndex = 0 ; LevelIndex < GetWorld()->GetNumLevels(); ++LevelIndex )
	{
		ULevel* Level = GetWorld()->GetLevel(LevelIndex);
		UModel* Model = Level->Model;
		for(int32 SurfaceIndex = 0;SurfaceIndex < Model->Surfs.Num();SurfaceIndex++)
		{
			FBspSurf&	Surf = Model->Surfs[SurfaceIndex];
			if ( (Surf.PolyFlags&PF_Selected) != 0 && Surf.Actor != NULL )
			{
				Surf.Actor->Brush->Polys->Element[Surf.iBrushPoly].LightMapScale = LightMapScale;
				Surf.LightMapScale = LightMapScale;
				bSurfacesDirty = true;
			}
		}
	}

	if ( bSurfacesDirty )
	{
		GetWorld()->MarkPackageDirty();
		ULevel::LevelDirtiedEvent.Broadcast();
	}
}

void SSurfaceProperties::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged )
{
	// update any selected lightmass settings with the new information
	if (SelectedLightmassSettingsObjects.Num() > 0)
	{
		SetLightmassSettingsForSelectedSurfaces(Cast<ULightmassPrimitiveSettingsObject>(SelectedLightmassSettingsObjects[0])->LightmassSettings);
	}
}

void SSurfaceProperties::SetLightmassSettingsForSelectedSurfaces(FLightmassPrimitiveSettings& InSettings)
{
	bool bSawLightmassSettingsChange = false;
	for (int32 LevelIndex = 0 ; LevelIndex < GetWorld()->GetNumLevels() ; ++LevelIndex)
	{
		ULevel* Level = GetWorld()->GetLevel(LevelIndex);
		UModel* Model = Level->Model;
		for (int32 SurfaceIndex = 0 ; SurfaceIndex < Model->Surfs.Num() ; ++SurfaceIndex)
		{
			FBspSurf& Surf = Model->Surfs[SurfaceIndex];
			if (((Surf.PolyFlags&PF_Selected) != 0) && (Surf.Actor != NULL))
			{
				int32 LookupIndex = FMath::Clamp<int32>(Surf.iLightmassIndex, 0, Model->LightmassSettings.Num());
				FLightmassPrimitiveSettings& Settings = Model->LightmassSettings[LookupIndex];
				if (!(Settings == InSettings))
				{
					// See if we can find the one of interest...
					int32 FoundLightmassIndex = INDEX_NONE;
					if (Model->LightmassSettings.Find(InSettings, FoundLightmassIndex) == false)
					{
						FoundLightmassIndex = Model->LightmassSettings.Add(InSettings);
					}
					Surf.iLightmassIndex = FoundLightmassIndex;
					bSawLightmassSettingsChange = true;
					Surf.Actor->Brush->Polys->Element[Surf.iBrushPoly].LightmassSettings = InSettings;
				}
			}
		}

		// Clean out unused Lightmass settings from the model...
		if (bSawLightmassSettingsChange)
		{
			TArray<bool> UsedIndices;
			UsedIndices.Empty(Model->LightmassSettings.Num());
			UsedIndices.AddZeroed(Model->LightmassSettings.Num());
			for (int32 SurfaceIndex = 0 ; SurfaceIndex < Model->Surfs.Num() ; ++SurfaceIndex)
			{
				FBspSurf& Surf = Model->Surfs[SurfaceIndex];
				if (Surf.Actor != NULL)
				{
					if ((Surf.iLightmassIndex >= 0) && (Surf.iLightmassIndex < Model->LightmassSettings.Num()))
					{
						UsedIndices[Surf.iLightmassIndex] = true;
					}
				}
			}

			for (int32 UsedIndex = UsedIndices.Num() - 1; UsedIndex >= 0; UsedIndex--)
			{
				if (UsedIndices[UsedIndex] == false)
				{
					Model->LightmassSettings.RemoveAt(UsedIndex);
					for (int32 SurfaceIndex = 0 ; SurfaceIndex < Model->Surfs.Num() ; ++SurfaceIndex)
					{
						FBspSurf& Surf = Model->Surfs[SurfaceIndex];
						if (Surf.Actor != NULL)
						{
							check (Surf.iLightmassIndex != UsedIndex);
							if (Surf.iLightmassIndex > UsedIndex)
							{
								Surf.iLightmassIndex--;
								check(Surf.iLightmassIndex >= 0);
							}
						}
					}
				}
			}
		}
	}
	if (bSawLightmassSettingsChange)
	{
		GetWorld()->MarkPackageDirty();
		ULevel::LevelDirtiedEvent.Broadcast();
	}
}

void SSurfaceProperties::OnCustomPanValueCommitted( int32 NewValue, ETextCommit::Type CommitInfo, TextureCoordChannel Channel )
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		OnPanTexture( NewValue, Channel); 
	}
	
	CustomPanButtoms[Channel].Pin()->SetIsOpen(false);
}

void SSurfaceProperties::OnScaleLabelClicked( )
{
	bUseRelativeScaling = !bUseRelativeScaling;
	GConfig->SetBool(TEXT("SurfaceSelection"), TEXT("UseRelativeScaling"), bUseRelativeScaling, GEditorPerProjectIni);
}

FText SSurfaceProperties::GetScalingLabel() const
{
	return (bUseRelativeScaling) ? LOCTEXT("ScaleRelativeTitle", "Scale Relative:") : LOCTEXT("ScaleTitle", "Scale:");
}

TSharedRef< ITableRow > SSurfaceProperties::OnGenerateScaleTableRow( TSharedPtr<FString> Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
	[
		SNew(STextBlock) .Text(FText::FromString(*Item.Get()))
	];
}

void SSurfaceProperties::OnScaleSelectionChanged(TSharedPtr<FString> ProposedSelection, ESelectInfo::Type SelectInfo, TextureCoordChannel Channel)
{
	if (ProposedSelection.IsValid())
	{
		float Scaling = FCString::Atof(*(*ProposedSelection.Get()));
		OnScaleValueCommitted(Scaling, ETextCommit::OnEnter, Channel);

		ScalingListViews[Channel].Pin()->ClearSelection();
		ScalingComboButton[Channel].Pin()->SetIsOpen(false);
	}
}

void  SSurfaceProperties::OnScaleValueCommitted(float Value, ETextCommit::Type CommitInfo,TextureCoordChannel Channel)
{
	if (bPreserveScaleRatio)
	{
		CachedScalingValueU = CachedScalingValueV = Value;
	}
	else if (Channel == UChannel)
	{
		CachedScalingValueU = Value;
	}
	else
	{
		CachedScalingValueV = Value;
	}
}

TOptional<float> SSurfaceProperties::OnGetScalingValue(TextureCoordChannel Channel) const
{
	return (Channel == UChannel) ? CachedScalingValueU : CachedScalingValueV;
}

FReply SSurfaceProperties::OnApplyScaling()
{
	OnScaleTexture(CachedScalingValueU,CachedScalingValueV,bUseRelativeScaling);
	return FReply::Handled();
}

const FSlateBrush* SSurfaceProperties::GetPreserveScaleRatioImage() const
{
	return bPreserveScaleRatio ? FAppStyle::GetBrush( TEXT("Icons.Lock") ) : FAppStyle::GetBrush( TEXT("Icons.Unlock") ) ;
}

ECheckBoxState SSurfaceProperties::IsPreserveScaleRatioChecked() const
{
	return bPreserveScaleRatio ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SSurfaceProperties::OnPreserveScaleRatioToggled( ECheckBoxState NewState )
{
	bPreserveScaleRatio = (NewState == ECheckBoxState::Checked) ? true : false;
	CachedScalingValueV = CachedScalingValueU;
	GConfig->SetBool(TEXT("SurfaceSelection"), TEXT("PreserveScaleRatio"), bPreserveScaleRatio, GEditorPerProjectIni);
}

void SSurfaceProperties::OnCustomRotateValueCommitted(int32 NewValue, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		OnRotateTexture(NewValue,Rotate);
	}
	
	CustomRotationButton.Pin()->SetIsOpen(false);
}

const FSlateBrush* SSurfaceProperties::GetTogglePanDirectionImage( TextureCoordChannel Channel ) const
{
	if (Channel == UChannel)
	{
		return bUseNegativePanningU ? FAppStyle::GetBrush( TEXT("SurfaceDetails.PanUNegative") ) : FAppStyle::GetBrush( TEXT("SurfaceDetails.PanUPositive") ) ;
	}

	return bUseNegativePanningV ? FAppStyle::GetBrush( TEXT("SurfaceDetails.PanVNegative") ) : FAppStyle::GetBrush( TEXT("SurfaceDetails.PanVPositive") ) ;
}

ECheckBoxState SSurfaceProperties::IsUsingNegativePanning( TextureCoordChannel Channel ) const
{
	if (Channel == UChannel)
	{
		return bUseNegativePanningU ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return bUseNegativePanningV ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SSurfaceProperties::OnTogglePanningDirection( ECheckBoxState NewState, TextureCoordChannel Channel )
{
	bool CheckBoxState = (NewState == ECheckBoxState::Checked) ? true : false;
	(Channel == UChannel) ? bUseNegativePanningU = CheckBoxState : bUseNegativePanningV = CheckBoxState;
}

const FSlateBrush* SSurfaceProperties::GetToggleRotationDirectionImage() const
{
	return bUseNegativeRotation ? FAppStyle::GetBrush( TEXT("SurfaceDetails.ClockwiseRotation") ) : FAppStyle::GetBrush( TEXT("SurfaceDetails.AntiClockwiseRotation") ) ;
}

ECheckBoxState SSurfaceProperties::IsUsingNegativeRotation() const
{
	return bUseNegativeRotation ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SSurfaceProperties::OnToggleRotationDirection( ECheckBoxState NewState )
{
	bUseNegativeRotation = (NewState == ECheckBoxState::Checked) ? true : false;
}

#undef LOCTEXT_NAMESPACE
