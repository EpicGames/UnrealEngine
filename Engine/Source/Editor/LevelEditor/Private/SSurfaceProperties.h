// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "UObject/GCObject.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Engine/World.h"
#include "Misc/NotifyHook.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Engine/Channel.h"

class IDetailsView;
class SComboButton;
class ULightmassPrimitiveSettingsObject;

/** 
 * Surface Properties
 * This class creates a widget which allows the user to edit any BSP
 * surfaces selected. Functionality includes allowing Panning, Rotating
 * and scaling of the texture coordinates. As well as editing the lighting
 * settings of the surface. 
 */
class SSurfaceProperties : public SCompoundWidget, public FNotifyHook, public FGCObject
{
public:
	SLATE_BEGIN_ARGS( SSurfaceProperties ){}
	SLATE_END_ARGS()

	/** Creates the widgets UI */
	void Construct( const FArguments& InArgs );

	UWorld* GetWorld() const { return GWorld;}		// Fallback to GWorld
private:
	/** Texture Coordinate channels to perform operations on */
	enum TextureCoordChannel
	{
		UChannel = 0,
		VChannel = 1
	};

	/** Actions available to rotation controls */
	enum RotationAction
	{
		Rotate,
		RotateCustom
	};
	
	// FNotifyHook Interface
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) override;
	
	// FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	/** 
	 * Creates Pan texture controls
	 * @returns Shared Reference to a widget containing constructed UI elements
	 */
	TSharedRef<SWidget> ConstructTexturePan();
	
	/** 
	 * Creates Rotate texture controls
	 * @returns Shared Reference to a widget containing constructed UI elements
	 */
	TSharedRef<SWidget> ConstructTextureRotate();

	/** 
	 * Creates Flip texture controls
	 * @returns Shared Reference to a widget containing constructed UI elements
	 */
	TSharedRef<SWidget> ConstructTextureFlip();
	
	/** 
	 * Creates Scale texture controls
	 * @returns Shared Reference to a widget containing constructed UI elements
	 */
	TSharedRef<SWidget> ConstructTextureScale();
	
	/** 
	 * Creates Lighting controls
	 * @returns Shared Reference to a widget containing constructed UI elements
	 */
	TSharedRef<SWidget> ConstructLighting();

	/** 
	 * Pans the texture on the selected surfaces
	 * @param PanAmount			- The amount to pan the texture coordinates.
	 * @param Channel			- The Texture coordinate channel to perform the operation on.
	 * 
	 * @return Always returns FReply::Handled()
	 */
	FReply OnPanTexture(int32 PanAmount, TextureCoordChannel Channel);

	/** 
	 * Rotates the texture on the selected surfaces
	 * @param RotationAmount	- The amount in Degrees to rotate the texture. 
	 * @param Action			- Dictates whether to Rotate by the passed in amount or Flip the texture
	 * 
	 * @return Always returns FReply::Handled()
	 */
	FReply OnRotateTexture(int32 RotationAmount, RotationAction Action);

	/** 
	 * Flips the texture on the selected surfaces
	 * @param Channel			- which texture coordinate Channel to flip the texture around.
	 * 
	 * @return Always returns FReply::Handled()
	 */
	FReply OnFlipTexture(TextureCoordChannel Channel);

	/** 
	 * Scales the texture on the selected surfaces
	 * @param InScaleU			- Value to scale the U texture coordinate by.
	 * @param InScaleV			- Value to scale the V texture coordinate by.
	 * @param InRelative		- Whether to scale the coordinates relatively or absolutely.
	 */
	void OnScaleTexture( float InScaleU, float InScaleV, bool InRelative );
	
	/** 
	 * Updates the selected surfaces light mass settings.
	 * @param InSettings		- Updated settings to be applied to surfaces
	 */
	void SetLightmassSettingsForSelectedSurfaces(FLightmassPrimitiveSettings& InSettings);

	/** 
	 * Called by the lightmap resolution control to access what value to display.
	 * @return Value to display
	 */
	TOptional<float> GetLightmapResolutionValue() const;

	/** 
	 * Called when the lightmap resolution field has been updated
	 * @param NewValue			- New Lightmap resolution value
	 */
	void OnLightmapResolutionCommitted(float NewValue, ETextCommit::Type CommitInfo);

	/** 
	 * Called when a new value is chosen from the scale Combobuttons drop down menu.
	 * @param ProposedSelection	- Value chosen from the menu, to be applied as the new scale value
	 * @param Channel			- Texture coordinate channel to operate on
	 * @param SelectInfo		- Provides context on how the selection changed
	 */
	void OnScaleSelectionChanged( TSharedPtr<FString> ProposedSelection,  ESelectInfo::Type SelectInfo, TextureCoordChannel Channel );

	/** 
	 * Called when a new Scale value is typed into the editable text field on a scale Combobutton.
	 * @param Channel			- Texture coordinate channel to operate on
	 * @param Value				- New value to be set as the scaling factor
	 */
	void OnScaleValueCommitted(float Value, ETextCommit::Type CommitInfo, TextureCoordChannel Channel);
	
	/** 
	 * Called by the scaling combobuttons to access what value to display.
	 * @param Channel			- Texture coordinate channel to operate on
	 * @returns Value to display (CachedScalingValueU or CachedScalingValueV)
	 */
	TOptional<float> OnGetScalingValue(TextureCoordChannel Channel) const;

	/** 
	 * Called by the Apply button in the scaling field, results in 
	 * executing a scaling operation using the values from the scaling factors. 
	 */
	FReply OnApplyScaling();

	/** 
	 * Called by the Preserve Scaling Ratio control to access which image to display
	 * @returns A FSlateBursh pointer to the correct image to display
	 */
	const FSlateBrush* GetPreserveScaleRatioImage() const;

	/** 
	 * Called by the Preserve Scaling Ratio control when clicked
	 * @param NewState			- The new state which the control has entered into
	 */
	void OnPreserveScaleRatioToggled( ECheckBoxState NewState );

	/** 
	 * called by the Preserve scaling ratio control to determine whether or not the control is toggled
	 * @returns The appropriate Checked state for whether the control is toggled
	 */
	ECheckBoxState IsPreserveScaleRatioChecked() const;

	/** 
	 * Constructs the internal Table rows of the Scale combobuttons menu. 
	 * @param Item				- Table entry to generate row for 
	 * @param OwnerTable		- Table entry parent table
	 *
	 * @return Shared reference to a newly generated TableRow 
	 */
	TSharedRef< ITableRow > OnGenerateScaleTableRow( TSharedPtr<FString> Item, const TSharedRef< STableViewBase >& OwnerTable );

	/** 
	 * Toggles Scale mode from Relative to Absolute, called when Scale label clicked
	 * @param Payload			- Required by delegate format, ignored in this case.
	 */
	void OnScaleLabelClicked( );

	/** 
	 * Returns the text for the scaling label, determined by the value of bUseRelativeScaling
	 */
	FText GetScalingLabel() const;

	/** 
	 * Called with a value has been committed by one of the custom pan combobuttons
	 * @param NewValue			- The amount to pan texture coordinate by. 
	 * @param Channel			- Texture coordinate channel to apply the panning on.
	 */
	void OnCustomPanValueCommitted( int32 NewValue, ETextCommit::Type CommitInfo, TextureCoordChannel Channel );

	/** 
	 * Called when a custom rotate value has been set via the rotations combobutton.
	 * @param NewValue			- Rotation value to be applied. 
	 */
	void OnCustomRotateValueCommitted(int32 NewValue, ETextCommit::Type CommitInfo );
	
	/** 
	 * Returns the image to display on the Toggle Pan Direction control, for the specified texture channel
	 * @param Channel			- Texture Coordinate Channel this control operates on.
	 */
	const FSlateBrush* GetTogglePanDirectionImage( TextureCoordChannel Channel ) const;

	/** 
	 * Returns the correct checkbox state for the specified texture channels panning state.
	 * @param Channel			- Texture Coordinate Channel this control operates on.
	 */
	ECheckBoxState IsUsingNegativePanning( TextureCoordChannel Channel ) const;

	/** 
	 * Toggles the specified texture channels panning state.
	 * @param NewState			- State of the toggled control. 
	 * @param Channel			- Texture Coordinate Channel this control operates on.
	 */
	void OnTogglePanningDirection( ECheckBoxState NewState, TextureCoordChannel Channel );

	/** 
	 * Returns the image to display on the Toggle Rotation Direction control.
	 */
	const FSlateBrush* GetToggleRotationDirectionImage() const;

	/** 
	 * Returns the correct checkbox state for the current rotation directions state.
	 */
	ECheckBoxState IsUsingNegativeRotation() const;

	/** 
	 * Toggles the Rotation directions state.
	 * @param NewValue			- Rotation value to be applied. 
	 */
	void OnToggleRotationDirection( ECheckBoxState NewState );

	/** typedef to make it easier to handle the lightmass objects */
	typedef TArray<TObjectPtr<ULightmassPrimitiveSettingsObject>> TLightmassSettingsObjectArray;

	/** Stores the lightmass settings objects, regardless of their selection status. Must be serialized. */
	TArray<TLightmassSettingsObjectArray> LevelLightmassSettingsObjects;

	/** Stores the selected lightmass settings objects, must be serialized. Used to create property tree. */
	TArray<TObjectPtr<UObject>> SelectedLightmassSettingsObjects;

	/** Holds reference to the lightmass property tree view */
	TSharedPtr<class IDetailsView> PropertyView;

	/** Stores a list of all of the scaling factor options for the scaling menu values */
	TArray<TSharedPtr<FString>> ScalingFactors;

	/** Stores a reference to the two scaling buttons */
	TArray<TWeakPtr<SComboButton>> ScalingComboButton;
	
	/** Stores a reference to the two scaling buttons List views*/
	TArray<TWeakPtr<SListView<TSharedPtr<FString>>>> ScalingListViews;

	/** Stores a pointer to the combobutton of the custom rotation control */
	TWeakPtr<SComboButton> CustomRotationButton;

	/** Stores pointers to the combobuttons of the custom panning controls */
	TArray<TWeakPtr<SComboButton>> CustomPanButtoms;

	/** Cached scaling values */
	float CachedScalingValueU;
	float CachedScalingValueV;

	/** If TRUE then any change to one scaling value will be reflected in the other */
	bool bPreserveScaleRatio;

	/** If TRUE Scaling will be done relative rather than absolute. */
	bool bUseRelativeScaling;

	/** If TRUE the panning operation the U texture coordinate will be inverted */
	bool bUseNegativePanningU;

	/** If TRUE the panning operation the V texture coordinate will be inverted */
	bool bUseNegativePanningV;

	/** If TRUE the Rotation operation will be inverted */
	bool bUseNegativeRotation;

	/** Cached value of Shifts current state, this should be updated before any buttons OnClicked is called */
	bool bShiftIsDown;
};
