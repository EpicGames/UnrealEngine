// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutliner/SceneOutlinerWidgetFactory.h"

#include "Columns/SceneOutlinerColumns.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerModule.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerWidget"

void USceneOutlinerWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage;

	DataStorageUi.RegisterWidgetPurpose(IUiProvider::FPurposeInfo(
		"LevelEditor", "SceneOutliner", "ActorOutliner",
		IUiProvider::EPurposeType::UniqueByName,
		LOCTEXT("ActorOutlinerPurpose", "Widget used to generate the base ISceneOutliner widget.")));
}

void USceneOutlinerWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorageUi.RegisterWidgetFactory<FSceneOutlinerWidgetConstructor>(DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo(
		"LevelEditor", "SceneOutliner", "ActorOutliner").GeneratePurposeID()));
}

FSceneOutlinerWidgetConstructor::FSceneOutlinerWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FSceneOutlinerWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	TWeakPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();

	FSceneOutlinerInitializationOptions InitOptions;
	if (TSharedPtr<ILevelEditor> LevelEditorPinned = LevelEditor.Pin())
	{
		// Fallback to the main SceneOutliner Tab Id
		FName TabIdentifier = LevelEditorTabIds::LevelEditorSceneOutliner;

		const FMetaDataEntryView OutlinerIdMeta = Arguments.FindGeneric("OutlinerIdentifier");
		if(const FString* const* IdMetaData = OutlinerIdMeta.TryGetExact<const FString*>())
		{
			TabIdentifier = FName(**IdMetaData);
		}
		LevelEditorPinned->GetSceneOutlinerInitializationOptions(TabIdentifier, InitOptions);
	}

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	TSharedRef<ISceneOutliner> Outliner = SceneOutlinerModule.CreateActorBrowser(InitOptions);
	DataStorage->AddColumn(WidgetRow, FSceneOutlinerColumn { .Outliner = Outliner });
	
	return Outliner;
}

#undef LOCTEXT_NAMESPACE