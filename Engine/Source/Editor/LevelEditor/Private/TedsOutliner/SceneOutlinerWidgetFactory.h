// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "SceneOutlinerWidgetFactory.generated.h"

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

UCLASS()
class USceneOutlinerWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~USceneOutlinerWidgetFactory() override = default;
	
	void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FSceneOutlinerWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FSceneOutlinerWidgetConstructor();
	~FSceneOutlinerWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

#undef UE_API
