// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelViewportClickSelection.h"
#include "BaseBehaviors/DoubleClickBehavior.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "EditorViewportSelectability.h"
#include "EngineUtils.h"
#include "HModel.h"
#include "SceneView.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "UnrealEdGlobals.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportSelectionUtilities.h"

namespace UE::Editor::LevelViewportInteractions
{

// See FLevelEditorViewportClient::ProcessClick for reference.
// This might look simplified, since ViewportSelectionUtilities::ClickElement is likely handling some cases which are never hit in the original code
void ProcessClick(FEditorViewportClient* InEditorViewportClient, const FViewportClick& InViewportClick, HHitProxy* InHitProxy)
{
	if (!InEditorViewportClient)
	{
		return;
	}

	if (!InEditorViewportClient->IsLevelEditorClient())
	{
		return;
	}

	FEditorModeTools* ModeTools = InEditorViewportClient->GetModeTools();
	if (!ModeTools)
	{
		return;
	}

	UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>();

	if (ModeTools->HandleClick(InEditorViewportClient, InHitProxy, InViewportClick))
	{
		return;
	}

	const FTypedElementHandle HitElement = InHitProxy ? InHitProxy->GetElementHandle() : FTypedElementHandle();

	if (InHitProxy == nullptr)
	{
		UE::Editor::ViewportSelectionUtilities::ClickBackdrop(InEditorViewportClient,InViewportClick);
	}
	else if (GUnrealEd->ComponentVisManager.HandleClick(InEditorViewportClient, InHitProxy, InViewportClick))
	{
		// Component vis manager handled the click
	}
	else if (HitElement && UE::Editor::ViewportSelectionUtilities::ClickElement(InEditorViewportClient, HitElement, InViewportClick))
	{
		// Element handled the click
	}
	else if (InHitProxy->IsA(HBSPBrushVert::StaticGetType()) && static_cast<HBSPBrushVert*>(InHitProxy)->Brush.IsValid())
	{
		FVector Vertex = FVector(*static_cast<HBSPBrushVert*>(InHitProxy)->Vertex);
		UE::Editor::ViewportSelectionUtilities::ClickBrushVertex(InEditorViewportClient, static_cast<HBSPBrushVert*>(InHitProxy)->Brush.Get(),&Vertex,InViewportClick);
	}
	else if (InHitProxy->IsA(HStaticMeshVert::StaticGetType()))
	{
		UE::Editor::ViewportSelectionUtilities::ClickStaticMeshVertex(InEditorViewportClient,
			static_cast<HStaticMeshVert*>(InHitProxy)->Actor,
			static_cast<HStaticMeshVert*>(InHitProxy)->Vertex,InViewportClick);
	}
	else if (BrushSubsystem && BrushSubsystem->ProcessClickOnBrushGeometry(InEditorViewportClient, InHitProxy, InViewportClick))
	{
		// Handled by the brush subsystem
	}
	else if (InHitProxy->IsA(HModel::StaticGetType()))
	{
		HModel* ModelHit = static_cast<HModel*>(InHitProxy);

		// Compute the viewport's current view family.
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues( InEditorViewportClient->Viewport, InEditorViewportClient->GetScene(), InEditorViewportClient->EngineShowFlags ));
		FSceneView* SceneView = InEditorViewportClient->CalcSceneView( &ViewFamily );

		uint32 SurfaceIndex = INDEX_NONE;
		if(ModelHit->ResolveSurface(SceneView,InViewportClick.GetClickPos().X,InViewportClick.GetClickPos().Y,SurfaceIndex))
		{
			UE::Editor::ViewportSelectionUtilities::ClickSurface(InEditorViewportClient, ModelHit->GetModel(),SurfaceIndex,InViewportClick);
		}
	}
	else if (InHitProxy->IsA(HLevelSocketProxy::StaticGetType()))
	{
		UE::Editor::ViewportSelectionUtilities::ClickLevelSocket(InEditorViewportClient, InHitProxy, InViewportClick);
	}
	else if (InViewportClick.GetKey() == EKeys::LeftMouseButton
			 && !InViewportClick.IsControlDown() 
			 && !InViewportClick.IsShiftDown()
		     && !InViewportClick.IsAltDown())
	{
		UE::Editor::ViewportSelectionUtilities::ClickBackdrop(InEditorViewportClient, InViewportClick);
	}
}
}

ULevelViewportClickSelection::ULevelViewportClickSelection()
{
	InteractionName = TEXT("Level Viewport Click Selection");
    Groups = { UE::Editor::ViewportInteractions::ViewportClick };
}

void ULevelViewportClickSelection::BuildBehaviors()
{
	Super::BuildBehaviors();

	using namespace UE::Editor::ViewportInteractions;
	
	if (UViewportClickBehavior* ClickBehavior = ClickBehaviorWeak.Get())
	{
		ClickBehavior->SetBindings({
			EKeys::LeftMouseButton,
			FButtonBinding(EKeys::LeftControl).Required(false),
			FButtonBinding(EKeys::LeftCommand).Required(false),
			FButtonBinding(EKeys::LeftAlt).Required(false),
			FButtonBinding(EKeys::LeftShift).Required(false)
		});
	}
	
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(CLICK_PRIORITY);
	RegisterInputBehavior(HoverBehavior);
}

void ULevelViewportClickSelection::OnClickDown(const FInputDeviceRay& InClickPos)
{
	if (!OnUpdateHover(InClickPos))
	{
		SetMouseCursorOverride(EMouseCursor::Default);
	}
}

FInputRayHit ULevelViewportClickSelection::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	if (GetHitProxy(PressPos))
	{
		return FInputRayHit(0.0);
	}
	return FInputRayHit();
}

void ULevelViewportClickSelection::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	OnUpdateHover(DevicePos);
}

bool ULevelViewportClickSelection::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (HHitProxy* HitProxy = GetHitProxy(DevicePos))
	{
		const EMouseCursor::Type Cursor = FEditorViewportSelectability::PassesAdditionalObjectSelectablePredicates(HitProxy)
			? HitProxy->GetMouseCursor() : EMouseCursor::Default;
		SetMouseCursorOverride(Cursor);
		if (FEditorViewportClient* Client = GetEditorViewportClient())
		{
			Client->CheckHoveredHitProxy(HitProxy);
		}
		return true;
	}

	return false;
}

void ULevelViewportClickSelection::OnEndHover()
{
	ClearMouseCursorOverride();
	if (FEditorViewportClient* Client = GetEditorViewportClient())
	{
		Client->CheckHoveredHitProxy(nullptr);
	}
}

void ULevelViewportClickSelection::ProcessClick_Internal(const FViewportClick& InViewportClick, HHitProxy* InHitProxy, FSceneView& View)
{
	UE::Editor::LevelViewportInteractions::ProcessClick(GetEditorViewportClient(), InViewportClick, InHitProxy);
}

ULevelViewportClickContextMenu::ULevelViewportClickContextMenu()
{
	InteractionName = TEXT("Level Viewport Context Menu");
	Groups = { UE::Editor::ViewportInteractions::ViewportClick };
}

void ULevelViewportClickContextMenu::BuildBehaviors()
{
	Super::BuildBehaviors();
	
	using namespace UE::Editor::ViewportInteractions;
	
	if (UViewportClickBehavior* ClickBehavior = ClickBehaviorWeak.Get())
	{
		ClickBehavior->SetBindings({
			EKeys::RightMouseButton,
			FButtonBinding(EKeys::LeftControl).Required(false),
			FButtonBinding(EKeys::LeftCommand).Required(false),
			FButtonBinding(EKeys::LeftAlt).Required(false),
			FButtonBinding(EKeys::LeftShift).Required(false)
		});
	}
}

ULevelViewportSetPivot::ULevelViewportSetPivot()
{
	InteractionName = TEXT("Level Viewport Set Pivot");
	Groups = { UE::Editor::ViewportInteractions::ViewportClick };
}

void ULevelViewportSetPivot::BuildBehaviors()
{
	Super::BuildBehaviors();

	if (UViewportClickBehavior* ClickBehavior = ClickBehaviorWeak.Get())
	{
		ClickBehavior->SetBindings({
			EKeys::MiddleMouseButton,
			EKeys::LeftAlt
		});
	}
}

void ULevelViewportSetPivot::ProcessClick_Internal(const FViewportClick& InViewportClick, HHitProxy* InHitProxy, FSceneView& View)
{
	if (GEditor)
	{
		// This only sets the pivot for the _next_ transform interaction.
		// The position of the pivot will be reset as soon as the next transform interaction completes.
		// This is consistent with legacy behavior
		GEditor->SetPivot(GEditor->ClickLocation, true, false, true);
	}
}

TOptional<FInputCapturePriority> ULevelViewportDoubleClickBehavior::GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage Stage, const FInputDeviceState& InputState) const
{
	if (InputState.Mouse.Left.bDoubleClicked)
	{
		return FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY - 1);
	}
	return TOptional<FInputCapturePriority>();
}

ULevelViewportDoubleClick::ULevelViewportDoubleClick()
{
	InteractionName = TEXT("Level Viewport Double Click");
    Groups = { UE::Editor::ViewportInteractions::ViewportClick };
}

void ULevelViewportDoubleClick::BuildBehaviors()
{
	ULevelViewportDoubleClickBehavior* ClickBehavior = NewObject<ULevelViewportDoubleClickBehavior>();
	ClickBehavior->Initialize(this);
	
	ClickBehavior->SetAllowsCaptureStealing(true);
	RegisterInputBehavior(ClickBehavior);

	ClickBehaviorWeak = ClickBehavior;
}
