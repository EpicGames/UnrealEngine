// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ViewportInteractions/ViewportClickInteraction.h"
#include "LevelViewportClickSelection.generated.h"

#define UE_API LEVELEDITOR_API

/**
 * Left-mouse click in level editor viewport, for elements selection
 */
UCLASS(MinimalAPI)
class ULevelViewportClickSelection : public UViewportClickInteraction, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API ULevelViewportClickSelection();

	UE_API virtual void OnClickDown(const FInputDeviceRay& InClickPos) override;

	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

protected:
	UE_API virtual void BuildBehaviors() override;
	UE_API virtual void ProcessClick_Internal(const FViewportClick& InViewportClick, HHitProxy* InHitProxy, FSceneView& View) override;
};

/**
 * Right-mouse click in level editor viewport, for selecting + summoning context menu
 */
UCLASS(MinimalAPI)
class ULevelViewportClickContextMenu : public ULevelViewportClickSelection
{
	GENERATED_BODY()

public:
	UE_API ULevelViewportClickContextMenu();
	
protected:
	UE_API virtual void BuildBehaviors() override;
};

/**
 * Set the pivot based on click, for various meta-interactions
 */
UCLASS(MinimalAPI)
class ULevelViewportSetPivot : public ULevelViewportClickSelection
{
	GENERATED_BODY()

public:
	UE_API ULevelViewportSetPivot();
	
protected:
	UE_API virtual void BuildBehaviors() override;
	
	UE_API virtual void ProcessClick_Internal(const FViewportClick& InViewportClick, HHitProxy* InHitProxy, FSceneView& View) override;
};

UCLASS(MinimalAPI)
class ULevelViewportDoubleClickBehavior : public UViewportClickBehavior
{
	GENERATED_BODY()

protected:
	UE_API virtual TOptional<FInputCapturePriority> GetCapturePriority(UE::Editor::ViewportInteractions::EInputStage Stage, const FInputDeviceState& InputState) const;
};

/**
 * Left-mouse double click
 */
UCLASS(MinimalAPI)
class ULevelViewportDoubleClick : public ULevelViewportClickSelection
{
	GENERATED_BODY()

public:
	UE_API ULevelViewportDoubleClick();

protected:
	UE_API virtual void BuildBehaviors() override; 
};

#undef UE_API
