// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportToolbar/LevelEditorSubmenus.h"

#include "AudioDevice.h"
#include "Bookmarks/BookmarkUI.h"
#include "Bookmarks/IBookmarkTypeTools.h"
#include "Camera/CameraActor.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorViewportCommands.h"
#include "Engine/SceneCapture.h"
#include "Engine/GameViewportClient.h"
#include "EngineUtils.h"
#include "FoliageType.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "GameFramework/WorldSettings.h"
#include "GroomVisualizationData.h"
#include "ISettingsModule.h"
#include "Layers/LayersSubsystem.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "LevelViewportActions.h"
#include "SLevelViewport.h"
#include "Selection.h"
#include "SortHelper.h"
#include "Stats/StatsData.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ViewportToolbar/LevelViewportContext.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "VirtualTextureVisualizationMenuCommands.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SVolumeControl.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "VirtualShadowMapVisualizationData.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "RayTracingDebugVisualizationMenuCommands.h"
#include "MegaLightsVisualizationData.h"
#include "MegaLightsVisualizationMenuCommands.h"
#include "RenderUtils.h"

#define LOCTEXT_NAMESPACE "LevelEditorViewportToolbar"

namespace UE::LevelEditor::Private
{

bool IsLandscapeLODSettingChecked(FLevelEditorViewportClient& ViewportClient, int8 Value)
{
	return ViewportClient.LandscapeLODOverride == Value;
}

void OnLandscapeLODChanged(FLevelEditorViewportClient& ViewportClient, int8 NewValue)
{
	ViewportClient.LandscapeLODOverride = NewValue;
	ViewportClient.Invalidate();
}

TMap<FName, TArray<UFoliageType*>> GroupFoliageByOuter(const TArray<UFoliageType*> FoliageList)
{
	TMap<FName, TArray<UFoliageType*>> Result;

	for (UFoliageType* FoliageType : FoliageList)
	{
		if (FoliageType->IsAsset())
		{
			Result.FindOrAdd(NAME_None).Add(FoliageType);
		}
		else
		{
			FName LevelName = FoliageType->GetOutermost()->GetFName();
			Result.FindOrAdd(LevelName).Add(FoliageType);
		}
	}

	Result.KeySort(
		[](const FName& A, const FName& B)
		{
			return (A.LexicalLess(B) && B != NAME_None);
		}
	);
	return Result;
}

void PopulateMenuWithCommands(UToolMenu* Menu, TArray<FLevelViewportCommands::FShowMenuCommand> MenuCommands, int32 EntryOffset)
{
	FToolMenuSection& Section = Menu->AddSection("Section");

	// Generate entries for the standard show flags
	// Assumption: the first 'n' entries types like 'Show All' and 'Hide All' buttons, so insert a separator after them
	for (int32 EntryIndex = 0; EntryIndex < MenuCommands.Num(); ++EntryIndex)
	{
		FName EntryName = NAME_None;

		if (MenuCommands[EntryIndex].ShowMenuItem)
		{
			EntryName = MenuCommands[EntryIndex].ShowMenuItem->GetCommandName();
			ensure(Section.FindEntry(EntryName) == nullptr);
		}

		Section.AddMenuEntry(EntryName, MenuCommands[EntryIndex].ShowMenuItem, MenuCommands[EntryIndex].LabelOverride);

		if (EntryIndex == EntryOffset - 1)
		{
			Section.AddSeparator(NAME_None);
		}
	}
}

void PopulateShowLayersSubmenu(UToolMenu* InMenu, TWeakPtr<::SLevelViewport> InViewport)
{
	{
		FToolMenuSection& Section = InMenu->AddSection("LevelViewportLayers");
		Section.AddMenuEntry(FLevelViewportCommands::Get().ShowAllLayers, LOCTEXT("ShowAllLabel", "Show All"));
		Section.AddMenuEntry(FLevelViewportCommands::Get().HideAllLayers, LOCTEXT("HideAllLabel", "Hide All"));
	}

	if (TSharedPtr<::SLevelViewport> ViewportPinned = InViewport.Pin())
	{
		FToolMenuSection& Section = InMenu->AddSection("LevelViewportLayers2");
		// Get all the layers and create an entry for each of them
		TArray<FName> AllLayerNames;
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		Layers->AddAllLayerNamesTo(AllLayerNames);

		for (int32 LayerIndex = 0; LayerIndex < AllLayerNames.Num(); ++LayerIndex)
		{
			const FName LayerName = AllLayerNames[LayerIndex];
			// const FString LayerNameString = LayerName;

			FUIAction Action(
				FExecuteAction::CreateSP(ViewportPinned.ToSharedRef(), &::SLevelViewport::ToggleShowLayer, LayerName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(ViewportPinned.ToSharedRef(), &::SLevelViewport::IsLayerVisible, LayerName)
			);

			Section.AddMenuEntry(
				NAME_None, FText::FromName(LayerName), FText::GetEmpty(), FSlateIcon(), Action, EUserInterfaceActionType::ToggleButton
			);
		}
	}
}

void SetLevelViewportFOV(const TSharedRef<::SLevelViewport>& InLevelViewport, float InValue)
{
	bool bUpdateStoredFOV = true;

	if (InLevelViewport->GetLevelViewportClient().GetActiveActorLock().IsValid())
	{
		if (ACameraActor* CameraActor =
				Cast<ACameraActor>(InLevelViewport->GetLevelViewportClient().GetActiveActorLock().Get()))
		{
			CameraActor->GetCameraComponent()->FieldOfView = InValue;
			bUpdateStoredFOV = false;
		}
	}

	if (bUpdateStoredFOV)
	{
		InLevelViewport->GetLevelViewportClient().FOVAngle = InValue;
	}

	InLevelViewport->GetLevelViewportClient().ViewFOV = InValue;
	InLevelViewport->GetLevelViewportClient().Invalidate();
}

void SetFarViewPlaneValue(const TSharedRef<::SLevelViewport>& InLevelViewport, float InValue)
{
	FLevelEditorViewportClient& ViewportClient = InLevelViewport->GetLevelViewportClient();
	return ViewportClient.OverrideFarClipPlane(InValue);
}

float GetLevelViewportFOV(const TSharedRef<::SLevelViewport>& InLevelViewport)
{
	FLevelEditorViewportClient& ViewportClient = InLevelViewport->GetLevelViewportClient();
	return ViewportClient.ViewFOV;
}

float GetFarViewPlaneValue(const TSharedRef<::SLevelViewport>& InLevelViewport)
{
	FLevelEditorViewportClient& ViewportClient = InLevelViewport->GetLevelViewportClient();
	return ViewportClient.GetFarClipPlaneOverride();
}

bool AddJumpToBookmarkMenu(UToolMenu* InMenu, const FLevelEditorViewportClient* ViewportClient)
{
	FToolMenuSection& Section =
		InMenu->FindOrAddSection("JumpToBookmark", LOCTEXT("JumpToBookmarksSectionName", "Jump to Bookmark"));

	// Add a menu entry for each bookmark

	const int32 NumberOfBookmarks = static_cast<int32>(IBookmarkTypeTools::Get().GetMaxNumberOfBookmarks(ViewportClient));
	const int32 NumberOfMappedBookmarks = FMath::Min<int32>(AWorldSettings::NumMappedBookmarks, NumberOfBookmarks);

	bool bFoundAnyBookmarks = false;

	for (int32 BookmarkIndex = 0; BookmarkIndex < NumberOfMappedBookmarks; ++BookmarkIndex)
	{
		if (IBookmarkTypeTools::Get().CheckBookmark(BookmarkIndex, ViewportClient))
		{
			bFoundAnyBookmarks = true;
			Section.AddMenuEntry(
				NAME_None,
				FLevelViewportCommands::Get().JumpToBookmarkCommands[BookmarkIndex],
				FBookmarkUI::GetPlainLabel(BookmarkIndex),
				FBookmarkUI::GetJumpToTooltip(BookmarkIndex),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.SubMenu.Bookmarks")
			);
		}
	}

	return bFoundAnyBookmarks;
}

void AddClearBookmarkMenu(UToolMenu* InMenu, const TWeakPtr<::SLevelViewport>& InViewport)
{
	FToolMenuSection& Section = InMenu->AddSection("Section");

	// Add a menu entry for each bookmark
	// FEditorModeTools& Tools = GLevelEditorModeTools();
	if (TSharedPtr<::SLevelViewport> LevelViewportPinned = InViewport.Pin())
	{
		FLevelEditorViewportClient& ViewportClient = LevelViewportPinned->GetLevelViewportClient();

		const int32 NumberOfBookmarks =
			static_cast<int32>(IBookmarkTypeTools::Get().GetMaxNumberOfBookmarks(&ViewportClient));
		const int32 NumberOfMappedBookmarks = FMath::Min<int32>(AWorldSettings::NumMappedBookmarks, NumberOfBookmarks);

		for (int32 BookmarkIndex = 0; BookmarkIndex < NumberOfMappedBookmarks; ++BookmarkIndex)
		{
			if (IBookmarkTypeTools::Get().CheckBookmark(BookmarkIndex, &ViewportClient))
			{
				Section.AddMenuEntry(
					NAME_None,
					FLevelViewportCommands::Get().ClearBookmarkCommands[BookmarkIndex],
					FBookmarkUI::GetPlainLabel(BookmarkIndex)
				);
			}
		}

		for (int32 BookmarkIndex = NumberOfMappedBookmarks; BookmarkIndex < NumberOfBookmarks; ++BookmarkIndex)
		{
			if (IBookmarkTypeTools::Get().CheckBookmark(BookmarkIndex, &ViewportClient))
			{
				FUIAction Action;
				Action.ExecuteAction.BindSP(
					LevelViewportPinned.ToSharedRef(), &::SLevelViewport::OnClearBookmark, BookmarkIndex
				);

				Section.AddMenuEntry(
					NAME_None,
					FBookmarkUI::GetPlainLabel(BookmarkIndex),
					FBookmarkUI::GetClearTooltip(BookmarkIndex),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Clean"),
					Action
				);
			}
		}
	}
}

void GeneratePlacedCameraMenuEntries(
	FToolMenuSection& InSection, TArray<AActor*> InLookThroughActors, const TSharedPtr<::SLevelViewport>& InLevelViewport
)
{
	// Sort the cameras to make the ordering predictable for users.
	InLookThroughActors.StableSort(
		[](const AActor& Left, const AActor& Right)
		{
			// Do "natural sorting" via SceneOutliner::FNumericStringWrapper to make more sense to humans (also matches
			// the Scene Outliner). This sorts "Camera2" before "Camera10" which a normal lexicographical sort wouldn't.
			SceneOutliner::FNumericStringWrapper LeftWrapper(FString(Left.GetActorLabel()));
			SceneOutliner::FNumericStringWrapper RightWrapper(FString(Right.GetActorLabel()));

			return LeftWrapper < RightWrapper;
		}
	);

	for (AActor* LookThroughActor : InLookThroughActors)
	{
		// Needed for the delegate hookup to work below
		AActor* GenericActor = LookThroughActor;

		FText ActorDisplayName = FText::FromString(LookThroughActor->GetActorLabel());
		FUIAction LookThroughCameraAction(
			FExecuteAction::CreateSP(InLevelViewport.ToSharedRef(), &::SLevelViewport::OnActorLockToggleFromMenu, GenericActor),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(
				InLevelViewport.ToSharedRef(), &::SLevelViewport::IsActorLocked, MakeWeakObjectPtr(GenericActor)
			)
		);

		FSlateIcon ActorIcon;

		if (LookThroughActor->IsA<ACameraActor>() || LookThroughActor->IsA<ASceneCapture>())
		{
			ActorIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CameraComponent");
		}
		else
		{
			ActorIcon = FSlateIconFinder::FindIconForClass(LookThroughActor->GetClass());
		}

		InSection.AddMenuEntry(
			NAME_None,
			ActorDisplayName,
			FText::Format(LOCTEXT("LookThroughCameraActor_ToolTip", "Look through and pilot {0}"), ActorDisplayName),
			ActorIcon,
			LookThroughCameraAction,
			EUserInterfaceActionType::RadioButton
		);
	}
}

FToolMenuEntry CreateEjectActorPilotEntry()
{
	return FToolMenuEntry::InitDynamicEntry(
		"EjectActorPilotDynamicSection",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InnerSection) -> void
			{
				TSharedPtr<::SLevelViewport> LevelViewport = ULevelViewportContext::GetLevelViewport(InnerSection);
				if (!LevelViewport)
				{
					return;
				}

				FToolUIAction EjectActorPilotAction;

				EjectActorPilotAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
					[LevelViewportWeak = LevelViewport.ToWeakPtr()](const FToolMenuContext& Context) -> void
					{
						if (TSharedPtr<::SLevelViewport> LevelViewport = LevelViewportWeak.Pin())
						{
							LevelViewport->OnActorLockToggleFromMenu();
						}
					}
				);

				EjectActorPilotAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda(
					[LevelViewportWeak = LevelViewport.ToWeakPtr()](const FToolMenuContext& Context)
					{
						if (TSharedPtr<::SLevelViewport> EditorViewport = LevelViewportWeak.Pin())
						{
							return EditorViewport->IsAnyActorLocked();
						}
						return false;
					}
				);

				// We use this entry to gather its Name, Tooltip and Icon. See comment below as to why we cannot directly use this entry.
				FToolMenuEntry SourceEjectPilotEntry =
					FToolMenuEntry::InitMenuEntry(FLevelViewportCommands::Get().EjectActorPilot);

				// We want to use SetShowInToolbarTopLevel to show the Eject entry in the Top Level only when piloting is active.
				// Currently, this will not work with Commands, e.g. AddMenuEntry(FLevelViewportCommands::Get().EjectActorPilot).
				// So, we create the entry using FToolMenuEntry::InitMenuEntry, and we create our own Action to handle it.
				FToolMenuEntry EjectPilotActor = FToolMenuEntry::InitMenuEntry(
					"EjectActorPilot",
					LOCTEXT("EjectActorPilotLabel", "Stop Piloting Actor"),
					LOCTEXT(
						"EjectActorPilotTooltip", "Stop piloting an actor with the current viewport. Unlocks the viewport's position and orientation from the actor the viewport is currently piloting."
					),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelViewport.EjectActorPilot"),
					EjectActorPilotAction,
					EUserInterfaceActionType::Button
				);

				const TAttribute<bool> bShownInTopLevel = TAttribute<bool>::CreateLambda(
					[LevelViewportWeak = LevelViewport.ToWeakPtr()]() -> bool
					{
						if (TSharedPtr<::SLevelViewport> LevelViewport = LevelViewportWeak.Pin())
						{
							return LevelViewport->IsAnyActorLocked();
						}

						return true;
					}
				);

				EjectPilotActor.SetShowInToolbarTopLevel(bShownInTopLevel);

				InnerSection.AddEntry(EjectPilotActor);
			}
		)
	);
}

FText GetCameraSubmenuLabelFromLevelViewport(const TWeakPtr<::SLevelViewport>& InLevelEditorViewportClientWeak)
{
	if (TSharedPtr<::SLevelViewport> LevelViewport = InLevelEditorViewportClientWeak.Pin())
	{
		const FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();

		if (!LevelViewportClient.IsAnyActorLocked())
		{
			return UnrealEd::GetCameraSubmenuLabelFromViewportType(LevelViewportClient.GetViewportType());
		}
		else if (TStrongObjectPtr<AActor> ActorLock = LevelViewportClient.GetActiveActorLock().Pin())
		{
			return FText::FromString(ActorLock->GetActorNameOrLabel());
		}
	}

	return LOCTEXT("MissingActiveCameraLabel", "No Active Camera");
}

FSlateIcon GetCameraSubmenuIconFromLevelViewport(const TWeakPtr<::SLevelViewport>& InLevelEditorViewportClientWeak)
{
	if (TSharedPtr<::SLevelViewport> LevelViewport = InLevelEditorViewportClientWeak.Pin())
	{
		const FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();
		if (!LevelViewportClient.IsAnyActorLocked())
		{
			const FName IconName =
				UnrealEd::GetCameraSubmenuIconFNameFromViewportType(LevelViewportClient.GetViewportType());
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), IconName);
		}
		else if (TStrongObjectPtr<AActor> LockedActor = LevelViewportClient.GetActorLock().LockedActor.Pin())
		{
			if (!LockedActor->IsA<ACameraActor>() && !LockedActor->IsA<ASceneCapture>())
			{
				return FSlateIconFinder::FindIconForClass(LockedActor->GetClass());
			}
		}
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CameraComponent");
}

FToolMenuEntry CreateActorSnapCheckboxMenu()
{
	FNewToolMenuDelegate MakeMenuDelegate = FNewToolMenuDelegate::CreateLambda(
		[](UToolMenu* Submenu)
		{
			// Add "Actor snapping" widget.
			{
				FToolMenuSection& ActorSnappingSection =
					Submenu->FindOrAddSection("ActorSnapping", LOCTEXT("ActorSnappingLabel", "Actor Snapping"));
				const FText Label = LOCTEXT("ActorSnapDistanceLabel", "Snap Distance");
					const FText Tooltip = LOCTEXT("ActorSnapDistanceTooltip", "The amount of offset to apply when snapping to surfaces");
					const FMargin WidgetsMargin(2.0f, 0.0f, 3.0f, 0.0f);
					FToolMenuEntry SnapDistance = FToolMenuEntry::InitMenuEntry(
						"ActorSnapDistance",
						FUIAction(
							FExecuteAction(),
							FCanExecuteAction::CreateLambda(
								[]()
								{
									return !!GetDefault<ULevelEditorViewportSettings>()->bEnableActorSnap;
								}
							)
						),
						// clang-format off
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(WidgetsMargin)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(Label)
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(WidgetsMargin)
						.AutoWidth()
						[
							SNew(SBox).Padding(WidgetsMargin).MinDesiredWidth(100.0f)
							[
								// TODO: Check how to improve performance for this widget OnValueChanged.
								// Same functionality in LevelEditorToolBar.cpp seems to have better performance
								SNew(SNumericEntryBox<float>)
								.ToolTipText(Tooltip)
								.MinValue(0.0f)
								.MaxValue(1.0f)
								.MaxSliderValue(1.0f)
								.AllowSpin(true)
								.MaxFractionalDigits(1)
								.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
								.OnValueChanged_Static(&FLevelEditorActionCallbacks::SetActorSnapSetting)
								.Value_Lambda([]()
								{
									return FLevelEditorActionCallbacks::GetActorSnapSetting();
								})
							]
						]
						// clang-format on
					);
				ActorSnappingSection.AddEntry(SnapDistance);
			}
		}
	);
	FToolUIAction CheckboxMenuAction;
	{
		CheckboxMenuAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
			[](const FToolMenuContext& InContext)
			{
				if (ULevelEditorViewportSettings* Settings = GetMutableDefault<ULevelEditorViewportSettings>())
				{
					Settings->bEnableActorSnap = !Settings->bEnableActorSnap;
				}
			}
		);
		CheckboxMenuAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda(
			[](const FToolMenuContext& InContext)
			{
				return GetDefault<ULevelEditorViewportSettings>()->bEnableActorSnap ? ECheckBoxState::Checked
																					: ECheckBoxState::Unchecked;
			}
		);
	}
	FToolMenuEntry ActorSnapping = FToolMenuEntry::InitSubMenu(
		"ActorSnapping",
		LOCTEXT("ActorSnapLabel", "Actor"),
		FLevelEditorCommands::Get().EnableActorSnap->GetDescription(),
		MakeMenuDelegate,
		CheckboxMenuAction,
		EUserInterfaceActionType::ToggleButton,
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.EnableActorSnap")
	);

	return ActorSnapping;
}

// Can be used to show entries only in perspective view - specialized version of UE::UnrealEd method, for LevelViewport argument
TAttribute<bool> GetIsPerspectiveAttribute(const TWeakPtr<::SLevelViewport>& LevelViewportWeak)
{
	if (TSharedPtr<::SLevelViewport> LevelViewport = LevelViewportWeak.Pin())
	{
		return UE::UnrealEd::GetIsPerspectiveAttribute(LevelViewport->GetViewportClient());
	}

	return false;
}

FToolMenuEntry CreatePilotSubmenu(TWeakPtr<::SLevelViewport> LevelViewportWeak)
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"PilotingSubmenu",
		LOCTEXT("PilotingSubmenu", "Pilot"),
		LOCTEXT("PilotingSubmenu_ToolTip", "Piloting cameras and actors"),
		FNewToolMenuDelegate::CreateLambda(
			[LevelViewportWeak](UToolMenu* InMenu)
			{
				FToolMenuSection& PilotSection = InMenu->FindOrAddSection("Pilot");

				bool bShowPilotSelectedActorEntry = false;

				AActor* SelectedActor = nullptr;
				if (TSharedPtr<::SLevelViewport> LevelViewport = LevelViewportWeak.Pin())
				{
					TArray<AActor*> SelectedActors;
					GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

					if (!SelectedActors.IsEmpty() && !LevelViewport->IsSelectedActorLocked())
					{
						SelectedActor = SelectedActors[0];
						const FLevelEditorViewportClient& ViewportClient = LevelViewport->GetLevelViewportClient();

						bShowPilotSelectedActorEntry = SelectedActor && ViewportClient.IsPerspective()
													&& !ViewportClient.IsLockedToCinematic();
					}
				}

				if (bShowPilotSelectedActorEntry)
				{
					// Pilot Selected Actor Entry
					PilotSection.AddMenuEntry(
						FLevelViewportCommands::Get().PilotSelectedActor,
						FText::Format(LOCTEXT("PilotActor", "Pilot '{0}'"), FText::FromString(SelectedActor->GetActorLabel()))
					);
				}

				// Stop Piloting Entry
				PilotSection.AddEntry(UE::LevelEditor::Private::CreateEjectActorPilotEntry());

				// Exact Camera View Entry
				{
					FToolMenuEntry& ToggleCameraView =
						PilotSection.AddMenuEntry(FLevelViewportCommands::Get().ToggleActorPilotCameraView);
					ToggleCameraView.Label = LOCTEXT("ToggleCameraViewLabel", "Exact Camera View");
					ToggleCameraView.SetShowInToolbarTopLevel(
						TAttribute<bool>::CreateLambda(
							[LevelViewportWeak]()
							{
								if (TSharedPtr<::SLevelViewport> EditorViewport = LevelViewportWeak.Pin())
								{
									return EditorViewport->IsAnyActorLocked();
								}
								return false;
							}
						)
					);
				}

				PilotSection.AddMenuEntry(FLevelViewportCommands::Get().SelectPilotedActor);
			}
		),
		false,
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelViewport.PilotSelectedActor")
	);

	Entry.Visibility = GetIsPerspectiveAttribute(LevelViewportWeak);

	return Entry;
}

FToolMenuEntry CreateCameraMovementSubmenu(const TWeakPtr<::SLevelViewport>& LevelViewportWeak)
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"CameraMovement",
		LOCTEXT("CameraMovementSubmenu", "Camera Movement"),
		LOCTEXT("CameraMovementSubmenu_ToolTip", "Camera movement options"),
		FNewToolMenuDelegate::CreateLambda(
			[LevelViewportWeak](UToolMenu* InMenu)
			{
				{
					// Camera Movement
					{
						FToolMenuSection& CameraMovementSection =
							InMenu->FindOrAddSection("CameraMovement", LOCTEXT("CameraMovementLabel", "Camera Movement"));

						// This entry is created as perspective view only
						CameraMovementSection.AddEntry(UE::UnrealEd::CreateCameraSpeedMenu());

						// Frame Selection (both perspective and orthographic)
						{
							FToolMenuEntry& FrameEntry =
								CameraMovementSection.AddMenuEntry(FEditorViewportCommands::Get().FocusViewportToSelection);
							FrameEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FrameActor");
							
							FToolMenuEntry& FrameAndClip = CameraMovementSection.AddMenuEntry(FEditorViewportCommands::Get().FocusAndClipViewportToSelection);
							FrameAndClip.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FrameActor");
						}

						// The following entries are perspective view only
						{
							FToolMenuSection& PerspectiveOnlySection =
								InMenu->FindOrAddSection("CameraMovement_Perspective");

							PerspectiveOnlySection.AddMenuEntry(FLevelEditorCommands::Get().SnapCameraToObject);
							PerspectiveOnlySection.AddMenuEntry(FLevelEditorCommands::Get().SnapObjectToCamera);
							PerspectiveOnlySection.AddMenuEntry(FLevelEditorCommands::Get().OrbitCameraAroundSelection);

							PerspectiveOnlySection.Visibility = GetIsPerspectiveAttribute(LevelViewportWeak);
						}
					}

					// Orthographic
					{
						FToolMenuSection& OrthoSection =
							InMenu->FindOrAddSection("Orthographic", LOCTEXT("OrthographicSectionLabel", "Orthographic"));

						OrthoSection.AddMenuEntry(FLevelEditorCommands::Get().LinkOrthographicViewports);
						OrthoSection.AddMenuEntry(FLevelEditorCommands::Get().OrthoZoomToCursor);
					}
				}
			}
		),
		false,
		FSlateIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CameraComponent"))
	);

	return Entry;
}

void ExtendCameraSpeedSubmenu(FName InCameraSpeedSubmenuName)
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(InCameraSpeedSubmenuName);
	
	FToolMenuSection& SettingsSection = Menu->AddSection("Settings");
	SettingsSection.AddMenuEntry(
		"OpenPreferences",
		LOCTEXT("OpenCameraSpeedPreferencesLabel", "Show Camera Speed Preferences..."),
		LOCTEXT("OpenCameraSpeedPreferencesTooltip", "Opens the Editor Preferences page for camera speed."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon"),
		FExecuteAction::CreateLambda([]
		{
			ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
			if (SettingsModule)
			{
				SettingsModule->ShowViewer("Editor", "LevelEditor", "Viewport");
			}
		})
	);
}

FToolMenuEntry CreatePreviewSelectedCamerasCheckBoxSubmenu()
{
	FNewToolMenuDelegate PreviewSelectedCamerasMenuDelegate = FNewToolMenuDelegate::CreateLambda(
		[](UToolMenu* Submenu)
		{
			FToolMenuSection& CameraPreviewSection =
				Submenu->FindOrAddSection("Camera Preview", LOCTEXT("CameraPreviewLabel", "Camera Preview"));

			constexpr float PreviewSizeMin = 1.0f;
			constexpr float PreviewSizeMax = 10.0f;
			
			TSharedPtr<::SLevelViewport> LevelViewport = ULevelViewportContext::GetLevelViewport(Submenu);

			FToolMenuEntry PreviewSizeEntry = UnrealEd::CreateNumericEntry(
				"PreviewSize",
				LOCTEXT("PreviewSizeLabel", "Preview Size"),
				LOCTEXT("PreviewSizeTooltip", "Affects the size of 'picture in picture' previews if they are enabled"),
				FCanExecuteAction(),
				UnrealEd::FNumericEntryExecuteActionDelegate::CreateLambda(
					[LevelViewportWeak = LevelViewport.ToWeakPtr()](float InValue)
					{
						if (ULevelEditorViewportSettings* const ViewportSettings =
								GetMutableDefault<ULevelEditorViewportSettings>())
						{
							ViewportSettings->CameraPreviewSize = InValue;

							// If preview is not on, we assume the user wants to turn it on as they are editing preview size, so let's toggle it
							if (!FLevelEditorActionCallbacks::IsPreviewSelectedCamerasChecked())
							{
								FLevelEditorActionCallbacks::TogglePreviewSelectedCameras(LevelViewportWeak);
							}
						}
					}
				),
				TAttribute<float>::CreateLambda(
					[]()
					{
						if (const ULevelEditorViewportSettings* const ViewportSettings =
								GetDefault<ULevelEditorViewportSettings>())
						{
							return ViewportSettings->CameraPreviewSize;
						}
						return PreviewSizeMin;
					}
				),
				PreviewSizeMin,
				PreviewSizeMax
			);

			CameraPreviewSection.AddEntry(PreviewSizeEntry);
		}
	);

	FText PreviewSelectedCamerasTooltip = LOCTEXT(
		"CameraPreviewWindowTooltip", "When enabled, selecting a camera actor will display a live 'picture in picture' preview from the camera's perspective within the current editor view port.  This can be used to easily tweak camera positioning, post-processing and other settings without having to possess the camera itself.  This feature may reduce application performance when enabled."
	);

	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"PreviewSelectedCameras",
		LOCTEXT("PreviewSelectedCamerasLabel", "Preview Selected Cameras"),
		PreviewSelectedCamerasTooltip,
		PreviewSelectedCamerasMenuDelegate,
		FToolUIAction(
			FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
			{
				FLevelEditorActionCallbacks::TogglePreviewSelectedCameras(ULevelViewportContext::GetLevelViewport(InContext));
			}),
			FToolMenuGetActionCheckState::CreateLambda([](const FToolMenuContext& InContext)
			{
				return FLevelEditorActionCallbacks::IsPreviewSelectedCamerasChecked() ? ECheckBoxState::Checked
																					  : ECheckBoxState::Unchecked;
			})
		),
		EUserInterfaceActionType::ToggleButton
	);

	return Entry;
}

FToolMenuEntry CreateMouseScrollCameraSpeedEntry()
{
	constexpr int32 SpeedMin = 1;
	constexpr int32 SpeedMax = 8;

	return UE::UnrealEd::CreateNumericEntry(
		"MouseScrollCameraSpeed",
		LOCTEXT("MouseScrollCameraSpeedLabel", "Mouse Scroll Zoom Speed"),
		LOCTEXT("MouseScrollCameraSpeedTooltip", "How fast the perspective camera moves through the world when using mouse scroll"),
		FCanExecuteAction(),
		UnrealEd::FNumericEntryExecuteActionDelegateInt32::CreateLambda(
			[](int32 InValue)
			{
				if (ULevelEditorViewportSettings* const ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
				{
					ViewportSettings->MouseScrollCameraSpeed = InValue;
				}
			}
		),
		TAttribute<int32>::CreateLambda(
			[]()
			{
				if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
				{
					return ViewportSettings->MouseScrollCameraSpeed;
				}
				return SpeedMin;
			}
		),
		SpeedMin,
		SpeedMax
	);
}

FToolMenuEntry CreateMouseSensitivityEntry()
{
	constexpr float SpeedMin = 0.01f;
	constexpr float SpeedMax = 1.0f;
	return UnrealEd::CreateNumericEntry(
		"MouseSensitivity",
		LOCTEXT("MouseSensitivityLabel", "Mouse Sensitivity"),
		LOCTEXT("MouseSensitivityTooltip", "How fast the perspective camera moves through the world when using mouse scroll"),
		FCanExecuteAction(),
		UnrealEd::FNumericEntryExecuteActionDelegate::CreateLambda(
			[](float InValue)
			{
				if (ULevelEditorViewportSettings* const ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
				{
					ViewportSettings->MouseSensitivty = InValue;
				}
			}
		),
		TAttribute<float>::CreateLambda(
			[]()
			{
				if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
				{
					return ViewportSettings->MouseSensitivty;
				}
				return SpeedMin;
			}
		),
		SpeedMin,
		SpeedMax,
		2
	);
}

TSharedRef<SWidget> CreateGestureDirectionWidget(EScrollGestureDirection& InOutScrollGestureProperty, const FName InMenuName)
{
	if (!UToolMenus::Get()->IsMenuRegistered(InMenuName))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(InMenuName, NAME_None))
		{
			FToolMenuSection& Section = Menu->AddSection(NAME_None, FText());
			auto AddGestureRadioButton =
				[&Section, &InOutScrollGestureProperty](const FText& InLabel, EScrollGestureDirection InDirection)
			{
				Section.AddMenuEntry(
					NAME_None,
					InLabel,
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda(
							[InDirection, &InOutScrollGestureProperty]()
							{
								InOutScrollGestureProperty = InDirection;
							}
						),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda(
							[InDirection, &InOutScrollGestureProperty]()
							{
								if (InOutScrollGestureProperty == InDirection)
								{
									return true;
								}
								return false;
							}
						)
					),
					EUserInterfaceActionType::RadioButton
				);
			};
			AddGestureRadioButton(
				LOCTEXT("ScrollGestureDirectionSystemSettingsLabel", "System Setting"), EScrollGestureDirection::UseSystemSetting
			);
			AddGestureRadioButton(
				LOCTEXT("ScrollGestureDirectionStandardLabel", "Standard"), EScrollGestureDirection::Standard
			);
			AddGestureRadioButton(LOCTEXT("ScrollGestureDirectionNaturalLabel", "Natural"), EScrollGestureDirection::Natural);
		}
	}
	FToolMenuContext MenuContext;
	return UToolMenus::Get()->GenerateWidget(InMenuName, MenuContext);
}

TSharedRef<SWidget> CreatePerspectiveViewportGestureDirectionWidget()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		EScrollGestureDirection& Property = ViewportSettings->ScrollGestureDirectionFor3DViewports;
		return CreateGestureDirectionWidget(Property, "GestureDirectionSubmenu_Perspective");
	}
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> CreateOrthographicViewportGestureDirectionWidget()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		EScrollGestureDirection& Property = ViewportSettings->ScrollGestureDirectionForOrthoViewports;
		return CreateGestureDirectionWidget(Property, "GestureDirectionSubmenu_Orthographic");
	}
	return SNullWidget::NullWidget;
}

FToolMenuEntry CreateChannelsSubmenu(const TWeakPtr<::SLevelViewport>& LevelViewportWeak, bool bUseFullLabel)
{
	static FText ChannelLabels[] =
	{
		LOCTEXT("ChannelRGB_Label", "RGB"),
		LOCTEXT("ChannelRed_Label", "Red"),
		LOCTEXT("ChannelGreen_Label", "Green"),
		LOCTEXT("ChannelBlue_Label", "Blue"),
		LOCTEXT("ChannelAlpha_Label", "Alpha")
	};

	static FName ChannelIcons[] =
	{
		"EditorViewport.AllChannelsMask",
		"EditorViewport.RedChannelMask",
		"EditorViewport.GreenChannelMask",
		"EditorViewport.BlueChannelMask",
		"EditorViewport.AlphaChannelMask"
	};

	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"Channels",
		TAttribute<FText>::CreateLambda([LevelViewportWeak, bUseFullLabel]
		{
			if (const TSharedPtr<::SLevelViewport> Viewport = LevelViewportWeak.Pin())
			{
				const int8 ChannelMaskIndex = (int8)Viewport->GetViewportClient()->ChannelMaskParams.ColorChannelMask;
				return bUseFullLabel
					? FText::Format(LOCTEXT("ChannelsMenu_LabelFormat", "Channels ({0})"), ChannelLabels[ChannelMaskIndex + 1])
					: ChannelLabels[ChannelMaskIndex + 1];
			}

			return FText::GetEmpty();
		}),
		LOCTEXT("ChannelsMenu_ToolTip", "Display specific color channels in this viewport"),
		FNewToolMenuDelegate::CreateLambda([LevelViewportWeak](UToolMenu* InMenu)
		{
			FToolMenuSection& ChannelsSubMenuSection =
				InMenu->AddSection("ChannelsSubMenuSection", LOCTEXT("ChannelsSubMenuSectionHeader", "Channels"));

			ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().AllChannelsMask);
			ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().RedChannelMask);
			ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().GreenChannelMask);
			ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().BlueChannelMask);
			ChannelsSubMenuSection.AddMenuEntry(FEditorViewportCommands::Get().AlphaChannelMask);

			FToolMenuSection& OptionsSubMenuSection =
				InMenu->AddSection("OptionsSubMenuSection", LOCTEXT("OptionsSubMenuSectionHeader", "Options"));

			OptionsSubMenuSection.AddMenuEntry(
				"ShowChannelsInViewportToolbar",
				LOCTEXT("ShowChannelsInToolbarLabel", "Show Channels in Viewport Toolbar"),
				LOCTEXT("ShowChannelsInToolbarToolTip", "Show the channel selector in the level editor toolbar"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([]()
					{
						if (ULevelEditorViewportSettings* Settings = GetMutableDefault<ULevelEditorViewportSettings>())
						{
							Settings->bShowChannelsInToolbar = !Settings->bShowChannelsInToolbar;
						}
					}),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([]()
					{
						return GetDefault<ULevelEditorViewportSettings>()->bShowChannelsInToolbar ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				),
				EUserInterfaceActionType::ToggleButton
			);

			OptionsSubMenuSection.AddMenuEntry(
				"InvertAlpha",
				LOCTEXT("InvertAlphaLabel", "View Alpha as Inverted"),
				LOCTEXT("InvertAlphaToolTip", "Invert the alpha value displayed when viewing the alpha channel"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([LevelViewportWeak]
					{
						if (const TSharedPtr<::SLevelViewport> Viewport = LevelViewportWeak.Pin())
						{
							Viewport->GetViewportClient()->ChannelMaskParams.bInvertAlphaChannelMask = !Viewport->GetViewportClient()->ChannelMaskParams.bInvertAlphaChannelMask;
							if (!Viewport->GetViewportClient()->ChannelMaskParams.bInvertAlphaChannelMask)
							{
								// Disable the checkerboard blend when not inverting alpha as it will be broken due to UE conventions around alpha
								Viewport->GetViewportClient()->ChannelMaskParams.bDrawAlphaBlendedCheckerboard = false;
							}
							
							Viewport->GetViewportClient()->Invalidate();
						}
					}),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([LevelViewportWeak]
					{
						bool bChecked = false;
						if (const TSharedPtr<::SLevelViewport> Viewport = LevelViewportWeak.Pin())
						{
							bChecked = Viewport->GetViewportClient()->ChannelMaskParams.bInvertAlphaChannelMask; 
						}

						return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				),
				EUserInterfaceActionType::ToggleButton
			);

			OptionsSubMenuSection.AddMenuEntry(
				"VisualizeAlphaWithCheckerboard",
				LOCTEXT("VisualizeWithCheckerboardLabel", "Visualize Alpha with Checkerboard"),
				LOCTEXT("VisualizeWithCheckerboardToolTip", "Displays a checkerboard underneath the viewport to help visualize the alpha channel"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([LevelViewportWeak]
					{
						if (const TSharedPtr<::SLevelViewport> Viewport = LevelViewportWeak.Pin())
						{
							Viewport->GetViewportClient()->ChannelMaskParams.bDrawAlphaBlendedCheckerboard = !Viewport->GetViewportClient()->ChannelMaskParams.bDrawAlphaBlendedCheckerboard;
							Viewport->GetViewportClient()->Invalidate();
						}
					}),
					FCanExecuteAction::CreateLambda([LevelViewportWeak]
					{
						FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
						const bool bEnabled = LevelEditorModule.OnGetVisualizeCheckerboardEnabled().IsBound() ?
							LevelEditorModule.OnGetVisualizeCheckerboardEnabled().Execute() : true;
						
						if (const TSharedPtr<::SLevelViewport> Viewport = LevelViewportWeak.Pin())
						{
							return bEnabled && Viewport->GetViewportClient()->ChannelMaskParams.bInvertAlphaChannelMask;
						}

						return bEnabled;
					}),
					FGetActionCheckState::CreateLambda([LevelViewportWeak]
					{
						bool bChecked = false;
						if (const TSharedPtr<::SLevelViewport> Viewport = LevelViewportWeak.Pin())
						{
							bChecked = Viewport->GetViewportClient()->ChannelMaskParams.bDrawAlphaBlendedCheckerboard; 
						}

						return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				),
				EUserInterfaceActionType::ToggleButton
			);
		}),
		/*bInOpenSubMenuOnClick=*/false,
		TAttribute<FSlateIcon>::CreateLambda(
			[LevelViewportWeak]
			{
				if (const TSharedPtr<::SLevelViewport> Viewport = LevelViewportWeak.Pin())
				{
					const int8 ChannelMaskIndex = (int8)Viewport->GetViewportClient()->ChannelMaskParams.ColorChannelMask;
					return FSlateIcon(FAppStyle::GetAppStyleSetName(), ChannelIcons[ChannelMaskIndex + 1]);
				}
				return FSlateIcon();
			})
	);

	return Entry;
}

} // namespace UE::LevelEditor::Private

namespace UE::LevelEditor
{

TSharedPtr<FExtender> GetViewModesLegacyExtenders()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	return LevelEditorModule.GetMenuExtensibilityManager()->GetAllExtenders();
}

void PopulateViewModesMenu(UToolMenu* InMenu)
{
	const TSharedPtr<::SLevelViewport> LevelViewport = ULevelViewportContext::GetLevelViewport(InMenu);
	if (!LevelViewport)
	{
		return;
	}
	
	const FToolMenuInsert InsertPosition("ViewMode", EToolMenuInsertType::After);

	{
		FToolMenuSection& Section = InMenu->AddSection(
			"LevelViewportDeferredRendering", LOCTEXT("DeferredRenderingHeader", "Deferred Rendering"), InsertPosition
		);
	}

	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddSubMenu(
			"VisualizeBufferViewMode",
			LOCTEXT("VisualizeBufferViewModeDisplayName", "Buffer Visualization"),
			LOCTEXT("BufferVisualizationMenu_ToolTip", "Select a mode for buffer visualization"),
			FNewMenuDelegate::CreateStatic(&FBufferVisualizationMenuCommands::BuildVisualisationSubMenu),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakViewport = LevelViewport.ToWeakPtr()]()
					{
						const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin();
						check(Viewport.IsValid());
						FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
						return ViewportClient.IsViewModeEnabled(VMI_VisualizeBuffer);
					}
				)
			),
			EUserInterfaceActionType::RadioButton,
			/* bInOpenSubMenuOnClick = */ false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeBufferMode")
		);
	}

	if (IsRayTracingEnabled())
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddSubMenu(
			"RayTracingDebugSubMenu",
			LOCTEXT("RayTracingDebugSubMenu", "Ray Tracing Debug"),
			LOCTEXT("RayTracing_ToolTip", "Select ray tracing buffer visualization view modes"),
			FNewToolMenuDelegate::CreateStatic(&FRayTracingDebugVisualizationMenuCommands::BuildVisualisationSubMenu),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakViewport = LevelViewport.ToWeakPtr()]()
					{
						const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin();
						check(Viewport.IsValid());
						FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
						return ViewportClient.IsViewModeEnabled(VMI_RayTracingDebug);
					}
				)
			),
			EUserInterfaceActionType::RadioButton,
			/* bInOpenSubMenuOnClick = */ false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.RayTracingDebugMode")
		);
	}

	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddSubMenu(
			"VisualizeNaniteViewMode",
			LOCTEXT("VisualizeNaniteViewModeDisplayName", "Nanite Visualization"),
			LOCTEXT("NaniteVisualizationMenu_ToolTip", "Select a mode for Nanite visualization"),
			FNewMenuDelegate::CreateStatic(&FNaniteVisualizationMenuCommands::BuildVisualisationSubMenu),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakViewport = LevelViewport.ToWeakPtr()]()
					{
						const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin();
						check(Viewport.IsValid());
						FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
						return ViewportClient.IsViewModeEnabled(VMI_VisualizeNanite);
					}
				)
			),
			EUserInterfaceActionType::RadioButton,
			/* bInOpenSubMenuOnClick = */ false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeNaniteMode")
		);
	}

	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddSubMenu(
			"VisualizeLumenViewMode",
			LOCTEXT("VisualizeLumenViewModeDisplayName", "Lumen"),
			LOCTEXT("LumenVisualizationMenu_ToolTip", "Select a mode for Lumen visualization"),
			FNewMenuDelegate::CreateStatic(&FLumenVisualizationMenuCommands::BuildVisualisationSubMenu),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakViewport = LevelViewport.ToWeakPtr()]()
					{
						const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin();
						check(Viewport.IsValid());
						FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
						return ViewportClient.IsViewModeEnabled(VMI_VisualizeLumen);
					}
				)
			),
			EUserInterfaceActionType::RadioButton,
			/* bInOpenSubMenuOnClick = */ false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeLumenMode")
		);
	}

	if (ShouldCompileMegaLightsShaders(GMaxRHIShaderPlatform))
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddSubMenu(
			"VisualizeMegaLightsViewMode",
			LOCTEXT("VisualizeMegaLightsViewModeDisplayName", "MegaLights"),
			LOCTEXT("MegaLightsVisualizationMenu_ToolTip", "Select a mode for MegaLights visualization"),
			FNewMenuDelegate::CreateLambda([WeakViewport = LevelViewport.ToWeakPtr()](FMenuBuilder& Menu)
			{
				FMegaLightsVisualizationMenuCommands::BuildVisualisationSubMenu(Menu, WeakViewport);
			}),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakViewport = LevelViewport.ToWeakPtr()]()
					{
						const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin();
						check(Viewport.IsValid());
						FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
						return ViewportClient.IsViewModeEnabled(VMI_VisualizeMegaLights);
					}
				)
			),
			EUserInterfaceActionType::RadioButton,
			/* bInOpenSubMenuOnClick = */ false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeMegaLightsMode")
		);
	}

	if (Substrate::IsSubstrateEnabled())
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddSubMenu(
			"VisualizeSubstrateViewMode",
			LOCTEXT("VisualizeSubstrateViewModeDisplayName", "Substrate"),
			LOCTEXT("SubstrateVisualizationMenu_ToolTip", "Select a mode for Substrate visualization"),
			FNewMenuDelegate::CreateStatic(&FSubstrateVisualizationMenuCommands::BuildVisualisationSubMenu),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakViewport = LevelViewport.ToWeakPtr()]()
					{
						const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin();
						check(Viewport.IsValid());
						FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
						return ViewportClient.IsViewModeEnabled(VMI_VisualizeSubstrate);
					}
				)
			),
			EUserInterfaceActionType::RadioButton,
			/* bInOpenSubMenuOnClick = */ false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeSubstrateMode")
		);
	}

	if (IsGroomEnabled())
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddEntry(FGroomVisualizationMenuCommands::BuildVisualizationSubMenuItem(LevelViewport.ToWeakPtr()));
	}

	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddSubMenu(
			"VisualizeVirtualShadowMapViewMode",
			LOCTEXT("VisualizeVirtualShadowMapViewModeDisplayName", "Virtual Shadow Map"),
			LOCTEXT(
				"VirtualShadowMapVisualizationMenu_ToolTip",
				"Select a mode for virtual shadow map visualization. Select a light component in the world outliner to "
				"visualize that light."
			),
			FNewMenuDelegate::CreateStatic(&FVirtualShadowMapVisualizationMenuCommands::BuildVisualisationSubMenu),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakViewport = LevelViewport.ToWeakPtr()]()
					{
						const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin();
						check(Viewport.IsValid());
						FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
						return GetVirtualShadowMapVisualizationData().IsVSMViewMode(ViewportClient.GetViewMode());
					}
				)
			),
			EUserInterfaceActionType::RadioButton,
			/* bInOpenSubMenuOnClick = */ false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeVirtualShadowMapMode")
		);
	}

	if (UseVirtualTexturing(GMaxRHIShaderPlatform))
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddSubMenu(
			"VirtualTextureSubMenu",
			LOCTEXT("VirtualTexture_SubMenu", "Virtual Texture"),
			LOCTEXT("VirtualTexure_ToolTip", "Select virtual texture visualization view modes"),
			FNewMenuDelegate::CreateStatic(&FVirtualTextureVisualizationMenuCommands::BuildVisualisationSubMenu),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakViewport = LevelViewport.ToWeakPtr()]()
					{
						const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin();
						check(Viewport.IsValid());
						FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
						return ViewportClient.IsViewModeEnabled(VMI_VisualizeVirtualTexture);
					}
				)
			),
			EUserInterfaceActionType::RadioButton,
			/* bInOpenSubMenuOnClick = */ false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeVirtualTextureMode")
		);
	}

	{
		auto BuildActorColorationMenu = [WeakViewport = LevelViewport.ToWeakPtr()](UToolMenu* InMenu)
		{
			FToolMenuSection& SubMenuSection =
				InMenu->AddSection("LevelViewportActorColoration", LOCTEXT("ActorColorationHeader", "Actor Coloration"));

			TArray<FActorPrimitiveColorHandler::FPrimitiveColorHandler> PrimitiveColorHandlers;
			FActorPrimitiveColorHandler::Get().GetRegisteredPrimitiveColorHandlers(PrimitiveColorHandlers);

			for (const FActorPrimitiveColorHandler::FPrimitiveColorHandler& PrimitiveColorHandler : PrimitiveColorHandlers)
			{
				if (!PrimitiveColorHandler.bAvailalbleInEditor)
				{
					continue;
				}

				SubMenuSection.AddMenuEntry(
					NAME_None,
					PrimitiveColorHandler.HandlerText,
					PrimitiveColorHandler.HandlerToolTipText,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda(
							[WeakViewport, PrimitiveColorHandler]()
							{
								if (TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin())
								{
									FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
									ViewportClient.ChangeActorColorationVisualizationMode(PrimitiveColorHandler.HandlerName);
								}
							}
						),
						FCanExecuteAction::CreateLambda(
							[WeakViewport]()
							{
								if (TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin())
								{
									return true;
								}
								return false;
							}
						),
						FGetActionCheckState::CreateLambda(
							[WeakViewport, PrimitiveColorHandler]()
							{
								if (TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin())
								{
									FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
									return ViewportClient.IsActorColorationVisualizationModeSelected(
											   PrimitiveColorHandler.HandlerName
										   )
											 ? ECheckBoxState::Checked
											 : ECheckBoxState::Unchecked;
								}

								return ECheckBoxState::Unchecked;
							}
						)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		};

		FToolMenuSection& Section = InMenu->FindOrAddSection("ViewMode");
		Section.AddSubMenu(
			"VisualizeActorColorationViewMode",
			LOCTEXT("VisualizeActorColorationViewModeDisplayName", "Actor Coloration"),
			LOCTEXT("ActorColorationVisualizationMenu_ToolTip", "Select a mode for actor coloration visualization."),
			FNewToolMenuDelegate::CreateLambda(BuildActorColorationMenu),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakViewport = LevelViewport.ToWeakPtr()]()
					{
						const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin();
						check(Viewport.IsValid());
						FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
						return ViewportClient.IsViewModeEnabled(VMI_VisualizeActorColoration);
					}
				)
			),
			EUserInterfaceActionType::RadioButton,
			/* bInOpenSubMenuOnClick = */ false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeActorColorationMode")
		);
	}

	{
		FToolMenuSection& Section =
			InMenu->AddSection("LevelViewportChannels", LOCTEXT("ChannelsHeader", "Channels"), InsertPosition);

		constexpr bool bUseFullLabel = true;
		Section.AddEntry(UE::LevelEditor::Private::CreateChannelsSubmenu(LevelViewport.ToWeakPtr(), bUseFullLabel));
	}

	{
		FToolMenuSection& Section =
			InMenu->AddSection("LevelViewportLandscape", LOCTEXT("LandscapeHeader", "Landscape"), InsertPosition);

		auto BuildLandscapeLODMenu = [WeakViewport = LevelViewport.ToWeakPtr()](UToolMenu* InMenu)
		{
			FToolMenuSection& SubMenuSection =
				InMenu->AddSection("LevelViewportLandScapeLOD", LOCTEXT("LandscapeLODHeader", "Landscape LOD"));

			auto CreateLandscapeLODAction = [WeakViewport](int8 LODValue)
			{
				FUIAction LandscapeLODAction;
				LandscapeLODAction.ExecuteAction = FExecuteAction::CreateLambda(
					[WeakViewport, LODValue]()
					{
						if (const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin())
						{
							UE::LevelEditor::Private::OnLandscapeLODChanged(Viewport->GetLevelViewportClient(), LODValue);
						}
					}
				);
				LandscapeLODAction.GetActionCheckState = FGetActionCheckState::CreateLambda(
					[WeakViewport, LODValue]() -> ECheckBoxState
					{
						bool bChecked = false;
						if (const TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin())
						{
							bChecked = UE::LevelEditor::Private::IsLandscapeLODSettingChecked(
								Viewport->GetLevelViewportClient(), LODValue
							);
						}
						return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
				);

				return LandscapeLODAction;
			};

			SubMenuSection.AddMenuEntry(
				"LandscapeLODAuto",
				LOCTEXT("LandscapeLODAuto", "Auto"),
				FText(),
				FSlateIcon(),
				CreateLandscapeLODAction(-1),
				EUserInterfaceActionType::RadioButton
			);

			SubMenuSection.AddSeparator("LandscapeLODSeparator");

			static const FText FormatString = LOCTEXT("LandscapeLODFixed", "Fixed at {0}");
			for (int8 i = 0; i < 8; ++i)
			{
				SubMenuSection.AddMenuEntry(
					NAME_None,
					FText::Format(FormatString, FText::AsNumber(i)),
					FText(),
					FSlateIcon(),
					CreateLandscapeLODAction(i),
					EUserInterfaceActionType::RadioButton
				);
			}
		};

		Section.AddSubMenu(
			"LandscapeLOD",
			LOCTEXT("LandscapeLODDisplayName", "LOD"),
			LOCTEXT("LandscapeLODMenu_ToolTip", "Override Landscape LOD in this viewport"),
			FNewToolMenuDelegate::CreateLambda(BuildLandscapeLODMenu),
			/*bInOpenSubMenuOnClick=*/false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.LOD")
		);
	}
}

void ExtendViewModesSubmenu(FName InViewModesSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InViewModesSubmenuName);

	Submenu->AddDynamicSection(
		"LevelEditorViewModesExtensionDynamicSection",
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InDynamicMenu)
			{
				PopulateViewModesMenu(InDynamicMenu);
			}
		)
	);
}

FToolMenuEntry CreatePIEViewModesSubmenu()
{
	return FToolMenuEntry::InitDynamicEntry(
		"DynamicViewModes",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				ULevelViewportContext* const Context = InDynamicSection.FindContext<ULevelViewportContext>();
				if (!Context)
				{
					return;
				}
			
				TAttribute<FText> LabelAttribute = TAttribute<FText>::CreateLambda(
					[WeakViewport = Context->LevelViewport]
					{
						if (TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin())
						{
							return UE::UnrealEd::GetViewModesSubmenuLabel(Viewport->GetPlayClient());
						}
						return FText::GetEmpty();
					}
				);

				TAttribute<FSlateIcon> IconAttribute = TAttribute<FSlateIcon>::CreateLambda(
					[WeakViewport = Context->LevelViewport]
					{
						if (TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin())
						{
							return UE::UnrealEd::GetViewModesSubmenuIcon(Viewport->GetPlayClient());
						}
						return FSlateIcon();
					}
				);
				
				FToolMenuEntry& Entry = InDynamicSection.AddSubMenu(
					"ViewModes",
					LabelAttribute,
					LOCTEXT("ViewModesSubmenuTooltip", "View mode settings for the game viewport."),
					FNewToolMenuDelegate::CreateLambda([](UToolMenu* Submenu)
					{
						UE::UnrealEd::PopulateViewModesMenu(Submenu);
					}),
					false,
					IconAttribute
				);
				Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
			}
		)
	);
}

FToolMenuEntry CreateShowFoliageSubmenu()
{
	return FToolMenuEntry::InitSubMenu(
		"ShowFoliage",
		LOCTEXT("ShowFoliageTypesMenu", "Foliage Types"),
		LOCTEXT("ShowFoliageTypesMenu_ToolTip", "Show/hide specific foliage types"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu)
			{
				TSharedPtr<::SLevelViewport> Viewport = ULevelViewportContext::GetLevelViewport(Submenu);
				if (!Viewport)
				{
					return;
				}

				if (!Viewport->GetWorld())
				{
					return;
				}

				{
					FToolMenuSection& Section = Submenu->AddSection("LevelViewportFoliageMeshes");
					// Map 'Show All' and 'Hide All' commands
					FUIAction ShowAllFoliage(
						FExecuteAction::CreateSP(Viewport.ToSharedRef(), &::SLevelViewport::ToggleAllFoliageTypes, true)
					);
					FUIAction HideAllFoliage(
						FExecuteAction::CreateSP(Viewport.ToSharedRef(), &::SLevelViewport::ToggleAllFoliageTypes, false)
					);

					Section.AddMenuEntry(
						"ShowAll", LOCTEXT("ShowAllLabel", "Show All"), FText::GetEmpty(), FSlateIcon(), ShowAllFoliage
					);
					Section.AddMenuEntry(
						"HideAll", LOCTEXT("HideAllLabel", "Hide All"), FText::GetEmpty(), FSlateIcon(), HideAllFoliage
					);
				}

				// Gather all foliage types used in this world and group them by sub-levels
				auto AllFoliageMap =
					UE::LevelEditor::Private::GroupFoliageByOuter(GEditor->GetFoliageTypesInWorld(Viewport->GetWorld()));

				for (auto& FoliagePair : AllFoliageMap)
				{
					// Name foliage group by an outer sub-level name, or empty if foliage type is an asset
					FText EntryName =
						(FoliagePair.Key == NAME_None ? FText::GetEmpty()
													  : FText::FromName(FPackageName::GetShortFName(FoliagePair.Key)));
					FToolMenuSection& Section = Submenu->AddSection(NAME_None, EntryName);

					TArray<UFoliageType*>& FoliageList = FoliagePair.Value;
					for (UFoliageType* FoliageType : FoliageList)
					{
						FName MeshName = FoliageType->GetDisplayFName();
						TWeakObjectPtr<UFoliageType> FoliageTypePtr = FoliageType;

						FUIAction Action(
							FExecuteAction::CreateSP(
								Viewport.ToSharedRef(), &::SLevelViewport::ToggleShowFoliageType, FoliageTypePtr
							),
							FCanExecuteAction(),
							FIsActionChecked::CreateSP(Viewport.ToSharedRef(), &::SLevelViewport::IsFoliageTypeVisible, FoliageTypePtr)
						);

						Section.AddMenuEntry(
							NAME_None,
							FText::FromName(MeshName),
							FText::GetEmpty(),
							FSlateIcon(),
							Action,
							EUserInterfaceActionType::ToggleButton
						);
					}
				}
			}
		),
		false,
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ShowFlagsMenu.SubMenu.FoliageTypes")
	);
}

FToolMenuEntry CreateShowLayersSubmenu()
{
	// This is a dynamic entry so we can skip adding the submenu if the context
	// indicates that the viewport's world is partitioned.
	return FToolMenuEntry::InitDynamicEntry(
		"ShowLayersDynamic",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				TSharedPtr<::SLevelViewport> Viewport = ULevelViewportContext::GetLevelViewport(InDynamicSection);
				if (!Viewport)
				{
					return;
				}

				UWorld* World = Viewport->GetWorld();
				if (!World)
				{
					return;
				}

				// Only add this submenu for non-partitioned worlds.
				if (World->IsPartitionedWorld())
				{
					return;
				}

				InDynamicSection.AddSubMenu(
					"ShowLayers",
					LOCTEXT("ShowLayersMenu", "Layers"),
					LOCTEXT("ShowLayersMenu_ToolTip", "Show layers flags"),
					FNewToolMenuDelegate::CreateStatic(
						&UE::LevelEditor::Private::PopulateShowLayersSubmenu, Viewport.ToWeakPtr()
					),
					false,
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ShowFlagsMenu.SubMenu.Layers")
				);
			}
		)
	);
}

FToolMenuEntry CreateShowSpritesSubmenu()
{
	const FLevelViewportCommands& Actions = FLevelViewportCommands::Get();
	TArray<FLevelViewportCommands::FShowMenuCommand> ShowSpritesMenu;

	// 'Show All' and 'Hide All' buttons
	ShowSpritesMenu.Add(
		FLevelViewportCommands::FShowMenuCommand(Actions.ShowAllSprites, LOCTEXT("ShowAllLabel", "Show All"))
	);
	ShowSpritesMenu.Add(
		FLevelViewportCommands::FShowMenuCommand(Actions.HideAllSprites, LOCTEXT("HideAllLabel", "Hide All"))
	);

	// Get each show flag command and put them in their corresponding groups
	ShowSpritesMenu += Actions.ShowSpriteCommands;

	return FToolMenuEntry::InitSubMenu(
		"ShowSprites",
		LOCTEXT("ShowSpritesMenu", "Sprites"),
		LOCTEXT("ShowSpritesMenu_ToolTip", "Show sprites flags"),
		FNewToolMenuDelegate::CreateStatic(&UE::LevelEditor::Private::PopulateMenuWithCommands, ShowSpritesMenu, 2),
		false,
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ShowFlagsMenu.SubMenu.Sprites")
	);
}

FToolMenuEntry CreateShowVolumesSubmenu()
{
	const FLevelViewportCommands& Actions = FLevelViewportCommands::Get();
	TArray<FLevelViewportCommands::FShowMenuCommand> ShowVolumesMenu;

	// 'Show All' and 'Hide All' buttons
	ShowVolumesMenu.Add(
		FLevelViewportCommands::FShowMenuCommand(Actions.ShowAllVolumes, LOCTEXT("ShowAllLabel", "Show All"))
	);
	ShowVolumesMenu.Add(
		FLevelViewportCommands::FShowMenuCommand(Actions.HideAllVolumes, LOCTEXT("HideAllLabel", "Hide All"))
	);

	// Get each show flag command and put them in their corresponding groups
	ShowVolumesMenu += Actions.ShowVolumeCommands;

	return FToolMenuEntry::InitSubMenu(
		"ShowVolumes",
		LOCTEXT("ShowVolumesMenu", "Volumes"),
		LOCTEXT("ShowVolumesMenu_ToolTip", "Show volumes flags"),
		FNewToolMenuDelegate::CreateStatic(&UE::LevelEditor::Private::PopulateMenuWithCommands, ShowVolumesMenu, 2),
		false,
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ShowFlagsMenu.SubMenu.Volumes")
	);
}

#if STATS
FToolMenuEntry CreateShowStatsSubmenu(bool bInAddToggleStatsCheckbox, TAttribute<FText> InLabelOverride)
{
	return FToolMenuEntry::InitDynamicEntry(
		"DynamicShowStatsEntry",
		FNewToolMenuSectionDelegate::CreateLambda(
			[bInAddToggleStatsCheckbox, InLabelOverride](FToolMenuSection& InDynamicSection)
			{
				FToolUIActionChoice CommandAction;
				if (bInAddToggleStatsCheckbox)
				{
					if (TSharedPtr<::SLevelViewport> Viewport = ULevelViewportContext::GetLevelViewport(InDynamicSection))
					{
						CommandAction =
							FToolUIActionChoice(FEditorViewportCommands::Get().ToggleStats, *Viewport->GetCommandList());
					}
				}

				const TAttribute<FText> Label = InLabelOverride.IsSet() ? InLabelOverride
																		: LOCTEXT("ShowStatsMenu", "Stat");

				InDynamicSection.AddSubMenu(
					"ShowStatsMenu",
					Label,
					LOCTEXT("ShowStatsMenu_ToolTip", "Show Stat commands"),
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* InMenu) -> void
						{
							// Hide All
							{
								FToolMenuSection& UnnamedSection = InMenu->AddSection(NAME_None);
								UnnamedSection.AddMenuEntry(
									FLevelViewportCommands::Get().HideAllStats,
									LOCTEXT("HideAllLabel", "Hide All"),
									FText(),
									FSlateIconFinder::FindIcon("Cross")
								);
							}

							// Common Stats
							// The list of Stat Commands we want to show right below the Hide All Stats
							TArray<FName> CommonStatCommandNames = { TEXT("STAT_FPS"),
																	 TEXT("STAT_UNIT"),
																	 TEXT("STATGROUP_Memory"),
																	 TEXT("STATGROUP_RHI"),
																	 TEXT("STATGROUP_SceneRendering") };

							FToolMenuSection& CommonStatsSection =
								InMenu->AddSection("CommonStats", LOCTEXT("CommonStatsLabel", "Common Stats"));

							FToolMenuSection& Section = InMenu->AddSection("Section");

							// Separate out stats into two list, those with and without submenus
							TArray<FLevelViewportCommands::FShowMenuCommand> SingleStatCommands;
							TMap<FString, TArray<FLevelViewportCommands::FShowMenuCommand>> SubbedStatCommands;
							for (auto StatCatIt = FLevelViewportCommands::Get().ShowStatCatCommands.CreateConstIterator();
								 StatCatIt;
								 ++StatCatIt)
							{
								const TArray<FLevelViewportCommands::FShowMenuCommand>& ShowStatCommands =
									StatCatIt.Value();
								const FString& CategoryName = StatCatIt.Key();

								// If no category is specified, or there's only one category, don't use submenus
								FString NoCategory = FStatConstants::NAME_NoCategory.ToString();
								NoCategory.RemoveFromStart(TEXT("STATCAT_"));
								if (CategoryName == NoCategory
									|| FLevelViewportCommands::Get().ShowStatCatCommands.Num() == 1)
								{
									for (int32 StatIndex = 0; StatIndex < ShowStatCommands.Num(); ++StatIndex)
									{
										const FLevelViewportCommands::FShowMenuCommand& StatCommand =
											ShowStatCommands[StatIndex];
										SingleStatCommands.Add(StatCommand);
									}
								}
								else
								{
									SubbedStatCommands.Add(CategoryName, ShowStatCommands);
								}

								// Search for commands to be added to the Common Stats Section
								for (const FLevelViewportCommands::FShowMenuCommand& ShowMenuCommand : ShowStatCommands)
								{
									if (TSharedPtr<FUICommandInfo> Command = ShowMenuCommand.ShowMenuItem)
									{
										for (FName StatCommandName : CommonStatCommandNames)
										{
											if (StatCommandName == Command->GetCommandName())
											{
												CommonStatsSection.AddMenuEntry(Command);
											}
										}
									}
								}
							}

							// Sort Common Stats section entries alphabetically
							CommonStatsSection.Blocks.Sort(
								[](const FToolMenuEntry& A, const FToolMenuEntry& B)
								{
									return A.Label.Get().ToLower().ToString() < B.Label.Get().ToLower().ToString();
								}
							);

							CommonStatsSection.AddSeparator("CommonStatsSeparator");

							// First add all the stats that don't have a sub menu
							for (auto StatCatIt = SingleStatCommands.CreateConstIterator(); StatCatIt; ++StatCatIt)
							{
								const FLevelViewportCommands::FShowMenuCommand& StatCommand = *StatCatIt;
								Section.AddMenuEntry(NAME_None, StatCommand.ShowMenuItem, StatCommand.LabelOverride);
							}

							// Now add all the stats that have sub menus
							for (auto StatCatIt = SubbedStatCommands.CreateConstIterator(); StatCatIt; ++StatCatIt)
							{
								const TArray<FLevelViewportCommands::FShowMenuCommand>& StatCommands = StatCatIt.Value();
								const FText CategoryName = FText::FromString(StatCatIt.Key());

								FFormatNamedArguments Args;
								Args.Add(TEXT("StatCat"), CategoryName);
								const FText CategoryDescription =
									FText::Format(NSLOCTEXT("UICommands", "StatShowCatName", "Show {StatCat} stats"), Args);

								Section.AddSubMenu(
									NAME_None,
									CategoryName,
									CategoryDescription,
									FNewToolMenuDelegate::CreateStatic(
										&UE::LevelEditor::Private::PopulateMenuWithCommands, StatCommands, 0
									)
								);
							}
						}
					),
					CommandAction,
					bInAddToggleStatsCheckbox ? EUserInterfaceActionType::ToggleButton : EUserInterfaceActionType::Button,
					false,
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.SubMenu.Stats")
				);
			}
		)
	);
}
#endif

FToolMenuEntry CreateDynamicChannelsSubmenu()
{
	// This has to be a dynamic entry for the ViewModes submenu's label to be able to access the context.
	return FToolMenuEntry::InitDynamicEntry(
		"DynamicChannels",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				if (ULevelViewportContext* const LevelViewportContext = InDynamicSection.FindContext<ULevelViewportContext>())
				{
					constexpr bool bUseFullLabel = false;
					FToolMenuEntry& Entry = InDynamicSection.AddEntry(UE::LevelEditor::Private::CreateChannelsSubmenu(LevelViewportContext->LevelViewport, bUseFullLabel));
					Entry.Visibility = TAttribute<bool>::CreateLambda([]
					{
						return (bool)GetDefault<ULevelEditorViewportSettings>()->bShowChannelsInToolbar;
					});
					Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
				}
			}
		)
	);
}

FToolMenuEntry CreateShowSubmenu()
{
	return UE::UnrealEd::CreateShowSubmenu(
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu) -> void
		{
			{
				FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);

				UnnamedSection.AddMenuEntry(FLevelViewportCommands::Get().UseDefaultShowFlags);

				UnnamedSection.AddSeparator("ViewportStatsSeparator");

#if STATS
				// Override the label of the stats submenu for the new viewport toolbar.
				UnnamedSection.AddEntry(
					UE::LevelEditor::CreateShowStatsSubmenu(true, LOCTEXT("ViewportStatsLabel", "Viewport Stats"))
				);
#endif
			}

			// Starting from commonly used flags
			UE::UnrealEd::AddDefaultShowFlags(InMenu);

			// Add Level Editor specific entries to the All Show Flags Section
			{
				FToolMenuSection& AllShowFlagsSection =
					InMenu->FindOrAddSection("AllShowFlags", LOCTEXT("AllShowFlagsLabel", "All Show Flags"));

				// Show Foliage
				{
					FToolMenuEntry ShowFoliageSubmenu = CreateShowFoliageSubmenu();
					ShowFoliageSubmenu.Label = LOCTEXT("ShowFoliageLabel", "Foliage");
					ShowFoliageSubmenu.InsertPosition.Position = EToolMenuInsertType::First;
					AllShowFlagsSection.AddEntry(ShowFoliageSubmenu);
				}

				// Show Layers
				{
					FToolMenuEntry ShowLayersSubmenu = CreateShowLayersSubmenu();
					ShowLayersSubmenu.InsertPosition.Position = EToolMenuInsertType::First;
					AllShowFlagsSection.AddEntry(ShowLayersSubmenu);
				}

				// Show Sprites
				{
					FToolMenuEntry ShowSpriteSubmenu = CreateShowSpritesSubmenu();
					ShowSpriteSubmenu.InsertPosition.Position = EToolMenuInsertType::First;
					AllShowFlagsSection.AddEntry(ShowSpriteSubmenu);
				}

				// Show Volumes
				{
					FToolMenuEntry ShowVolumesSubmenu = CreateShowVolumesSubmenu();
					ShowVolumesSubmenu.InsertPosition.Position = EToolMenuInsertType::First;
					AllShowFlagsSection.AddEntry(ShowVolumesSubmenu);
				}
			}

			// Adds show flags sections for backward compatibility with the old viewport toolbar.
			//  If your entries end up in this section, you should move it to the new "CommonShowFlags" section instead.
			InMenu->FindOrAddSection(
				"ShowFlagsMenuSectionCommon",
				LOCTEXT("ShowFlagsMenuSectionCommonLabel", "Common Show Flags (Deprecated section)")
			);

			// If your entries end up in these sections, you should move them to the above "AllShowFlags" section instead.
			InMenu->FindOrAddSection(
				"LevelViewportShowFlags", LOCTEXT("LevelViewportShowFlagsLabel", "All Show Flags (Deprecated section)")
			);
			InMenu->FindOrAddSection(
				"LevelViewportEditorShow", LOCTEXT("LevelViewportEditorShowLabel", "Editor (Deprecated section)")
			);
		}
	));
}

FToolMenuEntry CreatePIEShowSubmenu()
{
	return UE::UnrealEd::CreateShowSubmenu(FNewToolMenuDelegate::CreateLambda(
		[](UToolMenu* InMenu) -> void
		{
			{
				FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);
				UnnamedSection.AddMenuEntry(FLevelViewportCommands::Get().UseDefaultShowFlags);
			}

			UE::UnrealEd::AddDefaultShowFlags(InMenu);
		}
	));
}

FToolMenuEntry CreateFeatureLevelPreviewSubmenu()
{
	return FToolMenuEntry::InitSubMenu(
		"FeatureLevelPreview",
		NSLOCTEXT("LevelToolBarViewMenu", "PreviewPlatformSubMenu", "Preview Platform"),
		NSLOCTEXT("LevelToolBarViewMenu", "PreviewPlatformSubMenu_ToolTip", "Sets the preview platform used by the main editor"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InMenu) -> void
			{
				FToolMenuSection& Section =
					InMenu->AddSection("EditorPreviewMode", LOCTEXT("EditorPreviewModePlatforms", "Preview Platforms"));

				if (FLevelEditorCommands::Get().DisablePlatformPreview.IsValid())
				{
					Section.AddMenuEntry(FLevelEditorCommands::Get().DisablePlatformPreview);
				}
				Section.AddSeparator("DisablePlatformPreviewSeparator");

				for (auto Iter = FLevelEditorCommands::Get().PlatformToPreviewPlatformOverrides.CreateConstIterator(); Iter;
					 ++Iter)
				{
					FName PlatformName = Iter.Key();
					const TArray<FLevelEditorCommands::PreviewPlatformCommand>& CommandList = Iter.Value();
					const TArray<FLevelEditorCommands::PreviewPlatformCommand>* CommandListJson =
						FLevelEditorCommands::Get().PlatformToPreviewJsonPlatformOverrides.Find(PlatformName);

					Section.AddSubMenu(
						FName(PlatformName),
						FText::FromString(PlatformName.ToString()),
						FText(),
						FNewToolMenuDelegate::CreateLambda(
							[CommandList, CommandListJson, PlatformName](UToolMenu* InSubMenu)
							{
								for (const FLevelEditorCommands::PreviewPlatformCommand& Command : CommandList)
								{
									FToolMenuSection& Section = InSubMenu->FindOrAddSection(
										Command.SectionName,
										FText::Format(LOCTEXT("PreviewShaderPlatform", "{0}"), FText::FromName(Command.SectionName))
									);
									Section.AddMenuEntry(Command.CommandInfo);
								}

								bool bJsonPreviewEnabled = FLevelEditorActionCallbacks::IsPreviewJsonVisible(PlatformName) || FLevelEditorActionCallbacks::IsGeneratePreviewJsonVisible(PlatformName);
								if (bJsonPreviewEnabled && CommandListJson != nullptr)
								{
									FToolMenuSection& SectionJson = InSubMenu->FindOrAddSection(
										"PreviewWithJson", LOCTEXT("PreviewWithJsonLabel", "Preview With Json")
									);
									TArray < TSharedPtr<FUICommandInfo>> CommandArray;
									for (const FLevelEditorCommands::PreviewPlatformCommand& PreviewJsonPlatform :
										 *CommandListJson)
									{
										if (PreviewJsonPlatform.bIsGeneratingJsonCommand)
										{
											SectionJson.AddMenuEntry(PreviewJsonPlatform.CommandInfo);
										}
										else
										{
											CommandArray.Add(PreviewJsonPlatform.CommandInfo);
										}
									}

									SectionJson.AddSubMenu(
										PlatformName,
										FText::Format(
											LOCTEXT("PreviewJsonLabel", "Preview {0} with Json"), FText::FromName(PlatformName)
										),
										FText::Format(
											LOCTEXT("PreviewJsonTooltip", "Preview {0} with Json"), FText::FromName(PlatformName)
										),
										FNewToolMenuDelegate::CreateLambda(
											[CommandArray](UToolMenu* InSubMenu)
											{
												FToolMenuSection& Section = InSubMenu->AddSection(NAME_None);
												for (const TSharedPtr<FUICommandInfo>& Command : CommandArray)
												{
													Section.AddMenuEntry(Command);
												}
											}
										)
									);
								}
							}
						)
					);
				}
			}
		),
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.PreviewPlatform")
	);
}

FToolMenuEntry CreateMaterialQualityLevelSubmenu()
{
	return FToolMenuEntry::InitSubMenu(
		"MaterialQualityLevel",
		NSLOCTEXT("LevelToolBarViewMenu", "MaterialQualityLevelSubMenu", "Material Quality Level"),
		NSLOCTEXT(
			"LevelToolBarViewMenu",
			"MaterialQualityLevelSubMenu_ToolTip",
			"Sets the value of the CVar \"r.MaterialQualityLevel\" (low=0, high=1, medium=2, Epic=3). This affects "
			"materials via the QualitySwitch material expression."
		),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InMenu) -> void
			{
				FToolMenuSection& Section = InMenu->AddSection(
					"LevelEditorMaterialQualityLevel",
					NSLOCTEXT("LevelToolBarViewMenu", "MaterialQualityLevelHeading", "Material Quality Level")
				);
				Section.AddMenuEntry(FLevelEditorCommands::Get().MaterialQualityLevel_Low);
				Section.AddMenuEntry(FLevelEditorCommands::Get().MaterialQualityLevel_Medium);
				Section.AddMenuEntry(FLevelEditorCommands::Get().MaterialQualityLevel_High);
				Section.AddMenuEntry(FLevelEditorCommands::Get().MaterialQualityLevel_Epic);
			}
		),
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.MaterialQuality")
	);
}

FToolMenuEntry CreatePerformanceAndScalabilitySubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"PerformanceAndScalability",
		LOCTEXT("PerformanceAndScalabilityLabel", "Performance & Scalability"),
		LOCTEXT("PerformanceAndScalabilityTooltip", "Performance and scalability tools tied to this viewport."),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				{
					FToolMenuSection& UnnamedSection = Submenu->FindOrAddSection(NAME_None);

					UnnamedSection.AddEntry(UE::UnrealEd::CreateToggleRealtimeEntry());
					UnnamedSection.AddEntry(UE::UnrealEd::CreateResetScalabilitySubmenu());

					if (const UUnrealEdViewportToolbarContext* Context =
							Submenu->FindContext<UUnrealEdViewportToolbarContext>())
					{
						UnnamedSection.AddEntry(UE::UnrealEd::CreateRemoveRealtimeOverrideEntry(Context->Viewport));
					}

					FToolMenuEntry& MenuEntryAllowConstrainedAspectRatio = UnnamedSection.AddMenuEntry(FLevelViewportCommands::Get().ToggleAllowConstrainedAspectRatioInPreview);
					MenuEntryAllowConstrainedAspectRatio.SetShowInToolbarTopLevel(true);
					FToolMenuEntry& MenuEntryFlipConstrainedAspectRatio = UnnamedSection.AddMenuEntry(FLevelViewportCommands::Get().ToggleFlipConstrainedAspectRatioInPreview);
					MenuEntryFlipConstrainedAspectRatio.SetShowInToolbarTopLevel(true);
				}

				{
					FToolMenuSection& PerformanceAndScalabilitySection = Submenu->FindOrAddSection(
						"PerformanceAndScalability",
						LOCTEXT("PerformanceAndScalabilitySectionLabel", "Performance & Scalability")
					);

					PerformanceAndScalabilitySection.AddEntry(CreateFeatureLevelPreviewSubmenu());

					PerformanceAndScalabilitySection.AddSeparator("PerformanceAndScalabilitySettings");

					PerformanceAndScalabilitySection.AddEntry(UE::UnrealEd::CreateScalabilitySubmenu());

					PerformanceAndScalabilitySection.AddEntry(CreateMaterialQualityLevelSubmenu());

					PerformanceAndScalabilitySection.AddEntry(UE::UnrealEd::CreateScreenPercentageSubmenu());
				}
			}
		)
	);
	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Scalability");
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
	return Entry;
}

FToolMenuEntry CreatePIEPerformanceAndScalabilitySubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"PerformanceAndScalability",
		LOCTEXT("PIEPerformanceAndScalabilityLabel", "Performance & Scalability"),
		LOCTEXT("PIEPerformanceAndScalabilityTooltip", "Performance and scalability tools tied to this viewport."),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				{
					FToolMenuSection& UnnamedSection = Submenu->FindOrAddSection(NAME_None);

					FToolMenuEntry& MenuEntryAllowConstrainedAspectRatio = UnnamedSection.AddMenuEntry(FLevelViewportCommands::Get().ToggleAllowConstrainedAspectRatioInPreview);
					MenuEntryAllowConstrainedAspectRatio.SetShowInToolbarTopLevel(true);
					FToolMenuEntry& MenuEntryAllowFlipConstrainedAspectRatio = UnnamedSection.AddMenuEntry(FLevelViewportCommands::Get().ToggleFlipConstrainedAspectRatioInPreview);
					MenuEntryAllowFlipConstrainedAspectRatio.SetShowInToolbarTopLevel(true);

					UnnamedSection.AddEntry(UE::UnrealEd::CreateResetScalabilitySubmenu());
				}

				{
					FToolMenuSection& PerformanceAndScalabilitySection = Submenu->FindOrAddSection(
						"PerformanceAndScalability",
						LOCTEXT("PerformanceAndScalabilitySectionLabel", "Performance & Scalability")
					);

					PerformanceAndScalabilitySection.AddEntry(CreateFeatureLevelPreviewSubmenu());
					PerformanceAndScalabilitySection.AddEntry(UE::UnrealEd::CreateScalabilitySubmenu());
				}
			}
		)
	);
	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Scalability");
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
	return Entry;
}

void GenerateViewportLayoutsMenu(UToolMenu* InMenu)
{
	const TSharedPtr<::SLevelViewport> LevelViewport = ULevelViewportContext::GetLevelViewport(InMenu);
	if (!LevelViewport)
	{
		return;
	}
	TSharedPtr<FUICommandList> CommandList = LevelViewport->GetCommandList();

	// Disable searching in this menu because it only contains visual representations of
	// viewport layouts without any searchable text.
	InMenu->bSearchable = false;

	{
		FToolMenuSection& Section =
			InMenu->AddSection("LevelViewportOnePaneConfigs", LOCTEXT("OnePaneConfigHeader", "One Pane"));

		FSlimHorizontalToolBarBuilder OnePaneButton(CommandList, FMultiBoxCustomization::None);
		OnePaneButton.SetLabelVisibility(EVisibility::Collapsed);
		OnePaneButton.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

		OnePaneButton.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_OnePane);

		Section.AddEntry(FToolMenuEntry::InitWidget(
			"LevelViewportOnePaneConfigs",
			// clang-format off
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				OnePaneButton.MakeWidget()
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNullWidget::NullWidget
			],
			// clang-format on
			FText::GetEmpty(),
			true
		));
	}

	{
		FToolMenuSection& Section =
			InMenu->AddSection("LevelViewportTwoPaneConfigs", LOCTEXT("TwoPaneConfigHeader", "Two Panes"));
		FSlimHorizontalToolBarBuilder TwoPaneButtons(CommandList, FMultiBoxCustomization::None);
		TwoPaneButtons.SetLabelVisibility(EVisibility::Collapsed);
		TwoPaneButtons.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

		TwoPaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_TwoPanesH, NAME_None, FText());
		TwoPaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_TwoPanesV, NAME_None, FText());

		Section.AddEntry(FToolMenuEntry::InitWidget(
			"LevelViewportTwoPaneConfigs",
			// clang-format off
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				TwoPaneButtons.MakeWidget()
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNullWidget::NullWidget
			],
			// clang-format on
			FText::GetEmpty(),
			true
		));
	}

	{
		FToolMenuSection& Section =
			InMenu->AddSection("LevelViewportThreePaneConfigs", LOCTEXT("ThreePaneConfigHeader", "Three Panes"));
		FSlimHorizontalToolBarBuilder ThreePaneButtons(CommandList, FMultiBoxCustomization::None);
		ThreePaneButtons.SetLabelVisibility(EVisibility::Collapsed);
		ThreePaneButtons.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

		ThreePaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_ThreePanesLeft, NAME_None, FText());
		ThreePaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_ThreePanesRight, NAME_None, FText());
		ThreePaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_ThreePanesTop, NAME_None, FText());
		ThreePaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_ThreePanesBottom, NAME_None, FText());

		Section.AddEntry(FToolMenuEntry::InitWidget(
			"LevelViewportThreePaneConfigs",
			// clang-format off
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				ThreePaneButtons.MakeWidget()
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNullWidget::NullWidget
			],
			// clang-format on
			FText::GetEmpty(),
			true
		));
	}

	{
		FToolMenuSection& Section =
			InMenu->AddSection("LevelViewportFourPaneConfigs", LOCTEXT("FourPaneConfigHeader", "Four Panes"));
		FSlimHorizontalToolBarBuilder FourPaneButtons(CommandList, FMultiBoxCustomization::None);
		FourPaneButtons.SetLabelVisibility(EVisibility::Collapsed);
		FourPaneButtons.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

		FourPaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_FourPanes2x2, NAME_None, FText());
		FourPaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_FourPanesLeft, NAME_None, FText());
		FourPaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_FourPanesRight, NAME_None, FText());
		FourPaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_FourPanesTop, NAME_None, FText());
		FourPaneButtons.AddToolBarButton(FLevelViewportCommands::Get().ViewportConfig_FourPanesBottom, NAME_None, FText());

		Section.AddEntry(FToolMenuEntry::InitWidget(
			"LevelViewportFourPaneConfigs",
			// clang-format off
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				FourPaneButtons.MakeWidget()
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNullWidget::NullWidget
			],
			// clang-format on
			FText::GetEmpty(),
			true
		));
	}
}

TSharedRef<SWidget> BuildVolumeControlCustomWidget()
{
	// clang-format off
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(0.9f)
		.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SVolumeControl)
				.ToolTipText_Static(&FLevelEditorActionCallbacks::GetAudioVolumeToolTip)
				.Volume_Static(&FLevelEditorActionCallbacks::GetAudioVolume)
				.OnVolumeChanged_Static(&FLevelEditorActionCallbacks::OnAudioVolumeChanged)
				.Muted_Static(&FLevelEditorActionCallbacks::GetAudioMuted)
				.OnMuteChanged_Static(&FLevelEditorActionCallbacks::OnAudioMutedChanged)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(0.1f);
	// clang-format on
}

FAudioDeviceHandle GetAudioFromViewport(const TWeakPtr<::SLevelViewport>& WeakViewport)
{
	if (TSharedPtr<::SLevelViewport> Viewport = WeakViewport.Pin())
	{
		if (UGameViewportClient* Client = Viewport->GetPlayClient())
		{
			return Client->GetWorld()->GetAudioDevice();
		}
	}
	return FAudioDeviceHandle();
}

TSharedRef<SWidget> BuildPIEVolumeControlCustomWidget(const TWeakPtr<::SLevelViewport>& Viewport)
{
	// clang-format off
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(0.9f)
		.MinWidth(100.0f)
		.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SVolumeControl)
				.ToolTipText_Lambda([WeakViewport = Viewport]
				{
					if (FAudioDeviceHandle Audio = GetAudioFromViewport(WeakViewport))
					{
						if (Audio->IsAudioDeviceMuted())
						{
							return LOCTEXT("PIEMuted", "Muted");
						}
								
						const float Volume = Audio->GetTransientPrimaryVolume() * 100.0f;
						return FText::AsNumber(FMath::RoundToInt(Volume));
					}
					return LOCTEXT("PIEVolume", "PIE Volume");
				})
				.Volume_Lambda([WeakViewport = Viewport]
				{
					if (FAudioDeviceHandle Audio = GetAudioFromViewport(WeakViewport))
					{
						return Audio->GetTransientPrimaryVolume();
					}
					return 0.0f;
				})
				.OnVolumeChanged_Lambda([WeakViewport = Viewport](float Volume)
				{
					if (FAudioDeviceHandle Audio = GetAudioFromViewport(WeakViewport))
					{
						Audio->SetTransientPrimaryVolume(Volume);
					}
				})
				.Muted_Lambda([WeakViewport = Viewport]
				{
					if (FAudioDeviceHandle Audio = GetAudioFromViewport(WeakViewport))
					{
						return Audio->IsAudioDeviceMuted();
					}
					return false;
				})
				.OnMuteChanged_Lambda([WeakViewport = Viewport](bool bMuted)
				{
					if (FAudioDeviceHandle Audio = GetAudioFromViewport(WeakViewport))
					{
						Audio->SetDeviceMuted(bMuted);
					}
				})
		]
		+ SHorizontalBox::Slot()
		.FillWidth(0.1f);
	// clang-format on
}

FToolMenuEntry CreateSettingsSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"Settings",
		LOCTEXT("SettingsSubmenuLabel", "Settings"),
		LOCTEXT("SettingsSubmenuTooltip", "Viewport-related settings"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				{
					{
						FToolMenuSection& SettingsSection = Submenu->FindOrAddSection("Settings", LOCTEXT("SettingsSectionLabel", "Settings"));
						SettingsSection.AddEntry(FToolMenuEntry::InitWidget(
							"Level Editor Volume (dB)",
							BuildVolumeControlCustomWidget(),
							LOCTEXT("VolumeControlLabel", "Level Editor Volume (dB)"), 
							false, true, false,
							LOCTEXT("VolumeControlToolTip", "Sets the level editor's preview volume of audio placed on actors in the level editor (e.g. Ambient Actors).")
						));
					}
					// Mouse Control
					{
						FToolMenuSection& ControlsSection =
							Submenu->FindOrAddSection("Controls", LOCTEXT("ControlsSectionLabel", "Controls"));
						ControlsSection.AddEntry(UE::LevelEditor::Private::CreateMouseSensitivityEntry());
						ControlsSection.AddEntry(UE::LevelEditor::Private::CreateMouseScrollCameraSpeedEntry());
						ControlsSection.AddMenuEntry(FLevelEditorCommands::Get().InvertMiddleMousePan);
						ControlsSection.AddMenuEntry(FLevelEditorCommands::Get().InvertOrbitYAxis);
						ControlsSection.AddMenuEntry(FLevelEditorCommands::Get().InvertRightMouseDollyYAxis);
						ControlsSection.AddSubMenu(
							"ScrollGestures",
							LOCTEXT("ScrollGesturesLabel", "Scroll Gestures"),
							LOCTEXT("ScrollGesturesTooltip", "Scroll Gestures Options"),
							FNewToolMenuDelegate::CreateLambda(
								[](UToolMenu* InMenu)
								{
									// Perspective Scroll Gesture
									{
										FToolMenuSection& PerspectiveSection = InMenu->AddSection(
											"PerspectiveScrollGestureDirection",
											LOCTEXT("PerspectiveScrollGestureDirectionLabel", "Perspective Scroll Gesture Direction")
										);
										PerspectiveSection.AddEntry(
											FToolMenuEntry::InitWidget(
												"PerspectiveScrollGestureDirectionWidget",
												UE::LevelEditor::Private::CreatePerspectiveViewportGestureDirectionWidget(),
												FText()
											)
										);
									}

									// Orthographic Scroll Gesture
									{
										FToolMenuSection& OrthographicSection = InMenu->AddSection(
											"OrthographicScrollGestureDirection",
											LOCTEXT("OrthographicScrollGestureDirectionLabel", "Ortho Scroll Gesture Direction")
										);
										OrthographicSection.AddEntry(
											FToolMenuEntry::InitWidget(
												"OrthographicScrollGestureDirectionWidget",
												UE::LevelEditor::Private::CreateOrthographicViewportGestureDirectionWidget(),
												FText()
											)
										);
									}
								}
							)
						);
					}
					// Viewport advanced settings
					{
						FToolMenuSection& UnnamedSection = Submenu->FindOrAddSection(NAME_None, FText());
						UnnamedSection.AddSeparator(NAME_None);
						const FLevelViewportCommands& LevelViewportActions = FLevelViewportCommands::Get();
						UnnamedSection.AddMenuEntry(LevelViewportActions.AdvancedSettings);
					}

					// Cascade
					{
						FToolMenuSection& CascadeSection =
							Submenu->FindOrAddSection("Cascade", LOCTEXT("CascadeSectionLabel", "Cascade"));

						constexpr bool bInOpenSubMenuOnClick = false;
						CascadeSection.AddSubMenu(
							"CascadeSubmenu",
							LOCTEXT("CascadeLabel", "Cascade"),
							LOCTEXT("CascadeTooltip", "Cascade Options"),
							FNewToolMenuDelegate::CreateLambda(
								[](UToolMenu* InMenu)
								{
									FToolMenuSection& Section =
										InMenu->AddSection("Cascade", LOCTEXT("CascadeLabel", "Cascade"));
									Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleParticleSystemLOD);
									Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleParticleSystemHelpers);
									Section.AddMenuEntry(FLevelEditorCommands::Get().ToggleFreezeParticleSimulation);
								}
							),
							bInOpenSubMenuOnClick,
							FSlateIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.ParticleSystemComponent"))
						);
					}
				}
			}
		)
	);

	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings");
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.ClippingPriority = 1000;
	return Entry;
}

FToolMenuEntry CreatePIESettingsSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"Settings",
		LOCTEXT("PIESettingsSubmenuLabel", "Settings"),
		LOCTEXT("PIESettingsSubmenuTooltip", "Viewport-related settings"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				{
					if (ULevelViewportContext* Context = Submenu->FindContext<ULevelViewportContext>())
					{
						FToolMenuSection& SettingsSection = Submenu->FindOrAddSection("Settings", LOCTEXT("PIESettingsSectionLabel", "Settings"));
						SettingsSection.AddEntry(FToolMenuEntry::InitWidget(
							"Volume",
							BuildPIEVolumeControlCustomWidget(Context->LevelViewport),
							LOCTEXT("PIEVolumeControlLabel", "Volume")
						));
						FToolMenuEntry EditorShortcutsEntry = FToolMenuEntry::InitMenuEntry(FLevelViewportCommands::Get().EnableEditorShortcutsInPIE,
							TAttribute<FText>(), TAttribute<FText>(),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.BrowseViewportControls"));
						EditorShortcutsEntry.SetShowInToolbarTopLevel(true);
						SettingsSection.AddEntry(EditorShortcutsEntry);
					}
					
					// Viewport advanced settings
					{
						FToolMenuSection& UnnamedSection = Submenu->FindOrAddSection(NAME_None, FText());
						UnnamedSection.AddSeparator(NAME_None);
						const FLevelViewportCommands& LevelViewportActions = FLevelViewportCommands::Get();
						UnnamedSection.AddMenuEntry(LevelViewportActions.PlaySettings);
					}
				}
			}
		)
	);

	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings");
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.ClippingPriority = 1000;
	return Entry;
}

FToolMenuEntry CreateViewportSizingSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"ViewportSizing",
		LOCTEXT("ViewportSizingLabel", "..."),
		LOCTEXT("ViewportSizingTooltip", "Viewport-sizing settings"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				if (const TSharedPtr<::SLevelViewport> LevelViewport = ULevelViewportContext::GetLevelViewport(Submenu))
				{
					GenerateViewportLayoutsMenu(Submenu);
				}

				{
					FToolMenuSection& MaximizeSection = Submenu->FindOrAddSection("MaximizeSection");

					MaximizeSection.AddSeparator("MaximizeSeparator");

					MaximizeSection.AddMenuEntry(FLevelViewportCommands::Get().ToggleImmersive);

					MaximizeSection.AddDynamicEntry(
						"MaximizeRestoreDynamicEntry",
						FNewToolMenuSectionDelegate::CreateLambda(
							[](FToolMenuSection& InnerSection) -> void
							{
								ULevelViewportContext* const LevelViewportContext =
									InnerSection.FindContext<ULevelViewportContext>();
								if (!LevelViewportContext)
								{
									return;
								}

								const TAttribute<FText> Label = TAttribute<FText>::CreateLambda(
									[WeakLevelViewport = LevelViewportContext->LevelViewport]() -> FText
									{
										if (const TSharedPtr<::SLevelViewport> LevelViewport = WeakLevelViewport.Pin())
										{
											if (!LevelViewport->IsMaximized())
											{
												return LOCTEXT("MaximizeRestoreLabel_Maximize", "Maximize Viewport");
											}
										}
										return LOCTEXT("MaximizeRestoreLabel_Restore", "Restore All Viewports");
									}
								);

								const TAttribute<FText> Tooltip = TAttribute<FText>::CreateLambda(
									[WeakLevelViewport = LevelViewportContext->LevelViewport]() -> FText
									{
										if (const TSharedPtr<::SLevelViewport> LevelViewport = WeakLevelViewport.Pin())
										{
											if (!LevelViewport->IsMaximized())
											{
												return LOCTEXT("MaximizeRestoreTooltip_Maximize", "Maximizes this viewport");
											}
										}
										return LOCTEXT(
											"MaximizeRestoreTooltip_Restore", "Restores the layout to show all viewports"
										);
									}
								);

								const TAttribute<FSlateIcon> Icon = TAttribute<FSlateIcon>::CreateLambda(
									[WeakLevelViewport = LevelViewportContext->LevelViewport]() -> FSlateIcon
									{
										if (const TSharedPtr<::SLevelViewport> LevelViewport = WeakLevelViewport.Pin())
										{
											if (!LevelViewport->IsMaximized())
											{
												return FSlateIcon(
													FAppStyle::GetAppStyleSetName(), "EditorViewportToolBar.Maximize.Normal"
												);
											}
										}

										return FSlateIcon(
											FAppStyle::GetAppStyleSetName(), "EditorViewportToolBar.Maximize.Checked"
										);
									}
								);
								
								FToolMenuEntry& MaximizeRestoreEntry = InnerSection.AddMenuEntry(FLevelViewportCommands::Get().ToggleMaximize, Label, Tooltip, Icon);
								MaximizeRestoreEntry.SetShowInToolbarTopLevel(true);
								MaximizeRestoreEntry.ToolBarData.ResizeParams.AllowClipping = false;
								MaximizeRestoreEntry.StyleNameOverride = FName("ViewportToolbarViewportSizingSubmenu");
							}
						)
					);
				}
			}
		)
	);

	Entry.StyleNameOverride = FName("ViewportToolbarViewportSizingSubmenu");
	Entry.InsertPosition.Position = EToolMenuInsertType::Last;
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.AllowClipping = false;

	return Entry;
}

void CreateCameraSpawnMenu(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Section");
	const FLevelViewportCommands& Actions = FLevelViewportCommands::Get();

	for (TSharedPtr<FUICommandInfo> Camera : Actions.CreateCameras)
	{
		Section.AddMenuEntry(NAME_None, Camera);
	}
}

void CreateBookmarksMenu(UToolMenu* InMenu)
{
	const TSharedPtr<::SLevelViewport> LevelViewport = ULevelViewportContext::GetLevelViewport(InMenu);
	if (!LevelViewport)
	{
		return;
	}

	// Add a menu entry for each bookmark
	const FLevelEditorViewportClient& ViewportClient = LevelViewport->GetLevelViewportClient();

	FToolMenuSection& ManageBookmarksSection =
		InMenu->FindOrAddSection("ManageBookmarks", LOCTEXT("ManageBookmarkSectionName", "Manage Bookmarks"));

	bool bFoundBookmarks = Private::AddJumpToBookmarkMenu(InMenu, &ViewportClient);

	// Manage Bookmarks Section
	{
		// Set Bookmark Submenu
		{
			const int32 NumberOfBookmarks =
				static_cast<int32>(IBookmarkTypeTools::Get().GetMaxNumberOfBookmarks(&ViewportClient));
			const int32 NumberOfMappedBookmarks = FMath::Min<int32>(AWorldSettings::NumMappedBookmarks, NumberOfBookmarks);

			ManageBookmarksSection.AddSubMenu(
				"SetBookmark",
				LOCTEXT("SetBookmarkSubMenu", "Set Bookmark"),
				LOCTEXT("SetBookmarkSubMenu_ToolTip", "Setting bookmarks"),
				FNewToolMenuDelegate::CreateLambda(
					[NumberOfMappedBookmarks](UToolMenu* InMenu)
					{
						const FLevelViewportCommands& Actions = FLevelViewportCommands::Get();

						FToolMenuSection& SetBookmarksSection =
							InMenu->FindOrAddSection("SetBookmark", LOCTEXT("SetBookmarkSectionName", "Set Bookmark"));

						for (int32 BookmarkIndex = 0; BookmarkIndex < NumberOfMappedBookmarks; ++BookmarkIndex)
						{
							SetBookmarksSection.AddMenuEntry(
								NAME_None,
								Actions.SetBookmarkCommands[BookmarkIndex],
								FBookmarkUI::GetPlainLabel(BookmarkIndex),
								FBookmarkUI::GetSetTooltip(BookmarkIndex),
								FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelViewport.ToggleActorPilotCameraView")
							);
						}
					}
				),
				false,
				FSlateIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelViewport.ToggleActorPilotCameraView"))
			);
		}

		// Manage Bookmarks Submenu
		{
			if (bFoundBookmarks)
			{
				ManageBookmarksSection.AddSubMenu(
					"ManageBookmarks",
					LOCTEXT("ManageBookmarksSubMenu", "Manage Bookmarks"),
					LOCTEXT("ManageBookmarksSubMenu_ToolTip", "Bookmarks related actions"),
					FNewToolMenuDelegate::CreateLambda(
						[bFoundBookmarks, LevelViewportWeak = LevelViewport.ToWeakPtr()](UToolMenu* InMenu)
						{
							if (!bFoundBookmarks)
							{
								return;
							}

							const FLevelViewportCommands& Actions = FLevelViewportCommands::Get();

							FToolMenuSection& ManageBookmarksSubsection = InMenu->FindOrAddSection(
								"ManageBookmarks", LOCTEXT("ManageBookmarkSectionName", "Manage Bookmarks")
							);

							ManageBookmarksSubsection.AddSubMenu(
								"ClearBookmark",
								LOCTEXT("ClearBookmarkSubMenu", "Clear Bookmark"),
								LOCTEXT("ClearBookmarkSubMenu_ToolTip", "Clear viewport bookmarks"),
								FNewToolMenuDelegate::CreateLambda(&Private::AddClearBookmarkMenu, LevelViewportWeak),
								false,
								FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.SubMenu.Bookmarks")
							);

							FToolMenuEntry& CompactBookmarks =
								ManageBookmarksSubsection.AddMenuEntry(Actions.CompactBookmarks);
							CompactBookmarks.Icon =
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimationEditor.ApplyCompression");

							FToolMenuEntry& ClearBookmarks =
								ManageBookmarksSubsection.AddMenuEntry(Actions.ClearAllBookmarks);
							ClearBookmarks.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Clean");
						}
					),
					false,
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.SubMenu.Bookmarks")
				);
			}
		}
	}
}

FToolMenuEntry CreateFOVMenu(TWeakPtr<::SLevelViewport> InLevelViewportWeak)
{
	constexpr float FOVMin = 5.0f;
	constexpr float FOVMax = 170.0f;

	return UnrealEd::CreateNumericEntry(
		"FOVAngle",
		LOCTEXT("FOVAngle", "Field of View"),
		LOCTEXT("FOVAngleTooltip", "Field of View"),
		FCanExecuteAction(),
		UnrealEd::FNumericEntryExecuteActionDelegate::CreateLambda(
			[InLevelViewportWeak](float InValue)
			{
				if (TSharedPtr<::SLevelViewport> LevelViewport = InLevelViewportWeak.Pin())
				{
					Private::SetLevelViewportFOV(LevelViewport.ToSharedRef(), InValue);
				}
			}
		),
		TAttribute<float>::CreateLambda(
			[InLevelViewportWeak]()
			{
				if (TSharedPtr<::SLevelViewport> Viewport = InLevelViewportWeak.Pin())
				{
					return Private::GetLevelViewportFOV(Viewport.ToSharedRef());
				}

				return FOVMin;
			}
		),
		FOVMin,
		FOVMax,
		1
	);
}

FToolMenuEntry CreateFarViewPlaneMenu(TWeakPtr<::SLevelViewport> InLevelViewportWeak)
{
	constexpr float FarMin = 0.0f;
	constexpr float FarMax = 100000.0f;

	return UnrealEd::CreateNumericEntry(
		"FarViewPlane",
		LOCTEXT("FarViewPlane", "Far View Plane"),
		LOCTEXT("FarViewPlaneTooltip", "Far View Plane"),
		FCanExecuteAction(),
		UnrealEd::FNumericEntryExecuteActionDelegate::CreateLambda(
			[InLevelViewportWeak](float InValue)
			{
				if (TSharedPtr<::SLevelViewport> LevelViewport = InLevelViewportWeak.Pin())
				{
					Private::SetFarViewPlaneValue(LevelViewport.ToSharedRef(), InValue);
				}
			}
		),
		TAttribute<float>::CreateLambda(
			[InLevelViewportWeak]()
			{
				if (TSharedPtr<::SLevelViewport> Viewport = InLevelViewportWeak.Pin())
				{
					return Private::GetFarViewPlaneValue(Viewport.ToSharedRef());
				}

				return FarMax;
			}
		),
		FarMin,
		FarMax,
		1
	);
}

void AddCameraActorSelectSection(UToolMenu* InMenu)
{
	TSharedPtr<::SLevelViewport> LevelViewport = ULevelViewportContext::GetLevelViewport(InMenu);
	if (!LevelViewport)
	{
		return;
	}

	TArray<AActor*> LookThroughActors;

	if (UWorld* World = LevelViewport->GetWorld())
	{
		for (TActorIterator<ACameraActor> It(World); It; ++It)
		{
			LookThroughActors.Add(Cast<AActor>(*It));
		}

		for (TActorIterator<ASceneCapture> It(World); It; ++It)
		{
			LookThroughActors.Add(Cast<AActor>(*It));
		}
	}

	FText CameraActorsHeading = LOCTEXT("CameraActorsHeading", "Cameras");

	FToolMenuInsert InsertPosition("LevelViewportCameraType_Perspective", EToolMenuInsertType::After);

	FToolMenuSection& Section = InMenu->AddSection("CameraActors");
	Section.InsertPosition = InsertPosition;

	// Don't add too many cameras to the top level menu or else it becomes too large
	constexpr uint32 MaxCamerasInTopLevelMenu = 10;
	if (LookThroughActors.Num() > MaxCamerasInTopLevelMenu)
	{
		FToolMenuEntry& Entry = Section.AddSubMenu(
			"CameraActors",
			CameraActorsHeading,
			LOCTEXT("LookThroughPlacedCameras_ToolTip", "Look through and pilot placed cameras"),
			FNewToolMenuDelegate::CreateLambda(
				[LookThroughActors, LevelViewportWeak = LevelViewport.ToWeakPtr()](UToolMenu* InMenu)
				{
					if (TSharedPtr<::SLevelViewport> LevelViewport = LevelViewportWeak.Pin())
					{
						FToolMenuSection& Section = InMenu->FindOrAddSection(NAME_None);
						UE::LevelEditor::Private::GeneratePlacedCameraMenuEntries(Section, LookThroughActors, LevelViewport);
					}
				}
			)
		);
		Entry.Icon = FSlateIconFinder::FindIconForClass(ACameraActor::StaticClass());
	}
	else if (!LookThroughActors.IsEmpty())
	{
		Section.AddSeparator(NAME_None);
		UE::LevelEditor::Private::GeneratePlacedCameraMenuEntries(Section, LookThroughActors, LevelViewport);
	}

	TWeakObjectPtr<AActor> LockedActorWeak = LevelViewport->GetLevelViewportClient().GetActorLock().LockedActor;

	if (TStrongObjectPtr<AActor> LockedActor = LockedActorWeak.Pin())
	{
		if (!LockedActor->IsA<ACameraActor>() && !LockedActor->IsA<ASceneCapture>())
		{
			UE::LevelEditor::Private::GeneratePlacedCameraMenuEntries(Section, { LockedActor.Get() }, LevelViewport);
		}
	}
}

void ExtendCameraSubmenu(FName InCameraOptionsSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InCameraOptionsSubmenuName);

	Submenu->AddDynamicSection(
		"LevelEditorCameraExtensionDynamicSection",
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InDynamicMenu)
			{
				TWeakPtr<::SLevelViewport> LevelViewportWeak = ULevelViewportContext::GetLevelViewport(InDynamicMenu);

				// Camera Selection elements
				{
					AddCameraActorSelectSection(InDynamicMenu);
				}

				// Movement Menus
				{
					FToolMenuSection& MovementSection = InDynamicMenu->FindOrAddSection("Movement");
					
					MovementSection.AddEntry(Private::CreatePilotSubmenu(LevelViewportWeak));
					MovementSection.AddEntry(Private::CreateCameraMovementSubmenu(LevelViewportWeak));
				}
				
				UE::UnrealEd::GenerateViewportTypeMenu(InDynamicMenu);

				// Create Section
				{
					FToolMenuSection& CreateSection =
						InDynamicMenu->FindOrAddSection("Create", LOCTEXT("CreateLabel", "Create"));

					CreateSection.AddSubMenu(
						"CreateCamera",
						LOCTEXT("CameraSubMenu", "Create Camera"),
						LOCTEXT("CameraSubMenu_ToolTip", "Select a camera type to create at current viewport's location"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* InMenu)
							{
								CreateCameraSpawnMenu(InMenu);
							}
						),
						false,
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.SubMenu.CreateCamera")
					);

					CreateSection.AddSubMenu(
						"Bookmarks",
						LOCTEXT("BookmarksSubMenu", "Bookmarks"),
						LOCTEXT("BookmarksSubMenu_ToolTip", "Bookmarks related actions"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* InMenu)
							{
								CreateBookmarksMenu(InMenu);
							}
						),
						false,
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.SubMenu.Bookmarks")
					);
				}

				// Options Section
				{
					FToolMenuSection& OptionsSection =
						InDynamicMenu->FindOrAddSection("CameraOptions", LOCTEXT("OptionsLabel", "Options"));
					// add Cinematic Viewport
					// add Allow Cinematic Control
					// add Game View

					FToolMenuEntry AllowCinematicControl =
						FToolMenuEntry::InitMenuEntry(FLevelViewportCommands::Get().ToggleCinematicPreview);
					AllowCinematicControl.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;
					OptionsSection.AddEntry(AllowCinematicControl);

					FToolMenuEntry ToggleGameView =
						FToolMenuEntry::InitMenuEntry(FLevelViewportCommands::Get().ToggleGameView);
					ToggleGameView.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;
					OptionsSection.AddEntry(ToggleGameView);

					// This additional options section is used to force certain elements to appear after extensions
					{
						FToolMenuSection& AdditionalOptions = InDynamicMenu->FindOrAddSection("AdditionalOptions");
						AdditionalOptions.AddEntry(UE::LevelEditor::Private::CreatePreviewSelectedCamerasCheckBoxSubmenu());
						AdditionalOptions.AddSeparator("AdditionalOptionsSeparator");

						FToolMenuEntry HighResolutionScreenshot =
							FToolMenuEntry::InitMenuEntry(FLevelViewportCommands::Get().HighResScreenshot);
						HighResolutionScreenshot.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;
						AdditionalOptions.AddEntry(HighResolutionScreenshot);
					}
				}
			}
		)
	);
	
	Private::ExtendCameraSpeedSubmenu(UToolMenus::JoinMenuPaths(InCameraOptionsSubmenuName, "CameraMovement.CameraSpeed"));
}

void ExtendTransformSubmenu(FName InTransformSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InTransformSubmenuName);
	if (!Submenu)
	{
		return;
	}

	// Gizmo
	{
		FToolMenuSection& GizmoSection = Submenu->FindOrAddSection("Gizmo", LOCTEXT("GizmoLabel", "Gizmo"));
		FToolMenuEntry& ShowTransformGizmoEntry = GizmoSection.AddMenuEntry(FLevelEditorCommands::Get().ShowTransformWidget);
		ShowTransformGizmoEntry.InsertPosition.Position = EToolMenuInsertType::First;

		GizmoSection.AddMenuEntry(FLevelEditorCommands::Get().AllowArcballRotation);
		GizmoSection.AddMenuEntry(FLevelEditorCommands::Get().AllowScreenspaceRotation);
	}

	// Selection
	{
		FToolMenuSection& SelectionSection = Submenu->FindOrAddSection("Selection", LOCTEXT("SelectionLabel", "Selection"));
		SelectionSection.AddMenuEntry(FLevelEditorCommands::Get().AllowTranslucentSelection);
		SelectionSection.AddMenuEntry(FLevelEditorCommands::Get().AllowGroupSelection);
		SelectionSection.AddMenuEntry(FLevelEditorCommands::Get().StrictBoxSelect);
		SelectionSection.AddMenuEntry(FLevelEditorCommands::Get().TransparentBoxSelect);
		SelectionSection.AddMenuEntry(FLevelEditorCommands::Get().ShowSelectionSubcomponents);
		SelectionSection.AddMenuEntry(FLevelEditorCommands::Get().EnableViewportHoverFeedback);

		// Marquee selection options
		{
			FToolMenuEntry& MarqueeMode = SelectionSection.AddSubMenu(
				"StrictBoxSelect", // Overwrite the other setting
				LOCTEXT("MarqueeSelectionLabel", "Marquee Selection"),
				LOCTEXT("MarqueeSelectionTooltip", "Define how objects are selected by marquee selection"),
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* Menu)
					{
						FProperty* Property = ULevelEditorViewportSettings::StaticClass()->FindPropertyByName(
							GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, MarqueeSelectionMode)
						);
						if (!Property || !Property->IsA<FEnumProperty>())
						{
							return;
						}

						FEnumProperty* EnumProperty = StaticCast<FEnumProperty*>(Property);
						UEnum* Enum = EnumProperty->GetEnum();
						if (!Enum)
						{
							return;
						}

						TWeakObjectPtr<UEnum> WeakEnum = Enum;

						FToolMenuSection& Section = Menu->AddSection("Mode", LOCTEXT("MarqueeModeSection", "Mode"));

						const int32 MaxIndex = Enum->NumEnums() - (Enum->ContainsExistingMax() ? 1 : 0);
						for (int32 Index = 0; Index < MaxIndex; Index++)
						{
							Section.AddMenuEntry(
								NAME_None,
								Enum->GetDisplayNameTextByIndex(Index),
								Enum->GetToolTipTextByIndex(Index),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda(
										[WeakEnum, Index]
										{
											if (UEnum* Enum = WeakEnum.Get())
											{
												ULevelEditorViewportSettings* Settings =
													GetMutableDefault<ULevelEditorViewportSettings>();
												Settings->MarqueeSelectionMode =
													static_cast<EMarqueeSelectionMode>(Enum->GetValueByIndex(Index));
												Settings->SaveConfig();
											}
										}
									),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda(
										[WeakEnum, Index]
										{
											if (UEnum* Enum = WeakEnum.Get())
											{
												const ULevelEditorViewportSettings* Settings =
													GetDefault<ULevelEditorViewportSettings>();
												return Settings->MarqueeSelectionMode
													== static_cast<EMarqueeSelectionMode>(Enum->GetValueByIndex(Index));
											}
											return false;
										}
									)
								),
								EUserInterfaceActionType::RadioButton
							);
						}
					}
				)

			);
			MarqueeMode.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.StrictBoxSelect");
			MarqueeMode.InsertPosition = FToolMenuInsert("TransparentBoxSelect", EToolMenuInsertType::Before);
			MarqueeMode.Visibility = TAttribute<bool>::CreateLambda([]()
			{
				// ITF Drag Tools are automatically enabled by new TRS Gizmos
				return GLevelEditorModeTools().GetInteractiveToolsContext()->UsesViewportInteractions();
			});
		}
	}
}

void ExtendSnappingSubmenu(FName InSnappingSubmenuName)
{
	if (UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InSnappingSubmenuName))
	{
		FToolMenuSection& SnappingSection = Submenu->FindOrAddSection("Snapping");

		// Actor Snapping
		SnappingSection.AddEntry(UE::LevelEditor::Private::CreateActorSnapCheckboxMenu());

		// Socket Snapping
		SnappingSection.AddMenuEntry(FLevelEditorCommands::Get().ToggleSocketSnapping);

		// Vertex Snapping
		SnappingSection.AddMenuEntry(FLevelEditorCommands::Get().EnableVertexSnap);
	}
}

FToolMenuEntry CreateToolbarCameraSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitDynamicEntry(
		"DynamicCameraOptions",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				if (ULevelViewportContext* const LevelViewportContext =
						InDynamicSection.FindContext<ULevelViewportContext>())
				{
					const TAttribute<FText> Label = TAttribute<FText>::CreateLambda(
						[LevelViewportWeak = LevelViewportContext->LevelViewport]()
						{
							return UE::LevelEditor::Private::GetCameraSubmenuLabelFromLevelViewport(LevelViewportWeak);
						}
					);

					const TAttribute<FSlateIcon> Icon = TAttribute<FSlateIcon>::CreateLambda(
						[LevelViewportWeak = LevelViewportContext->LevelViewport]()
						{
							return UE::LevelEditor::Private::GetCameraSubmenuIconFromLevelViewport(LevelViewportWeak);
						}
					);

					FToolMenuEntry& Entry = InDynamicSection.AddSubMenu(
						"Camera",
						Label,
						LOCTEXT("CameraSubmenuTooltip", "Camera options"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* Submenu) -> void
							{
								UnrealEd::PopulateCameraMenu(Submenu, UnrealEd::FViewportCameraMenuOptions().ShowLensControls());
							}
						),
						false,
						Icon
					);
					Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
				}
			}
		)
	);

	return Entry;
}

} // namespace UE::LevelEditor

#undef LOCTEXT_NAMESPACE
