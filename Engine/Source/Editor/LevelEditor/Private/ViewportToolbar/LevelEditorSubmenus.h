// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "ToolMenuEntry.h"

class FExtender;
class FLevelEditorViewportClient;
class SLevelViewport;
class SWidget;
class UToolMenu;
struct FToolMenuSection;

namespace UE::LevelEditor
{
void CreateCameraSpawnMenu(UToolMenu* InMenu);
void CreateBookmarksMenu(UToolMenu* InMenu);
void AddCameraActorSelectSection(UToolMenu* InMenu);

FToolMenuEntry CreateFOVMenu(TWeakPtr<SLevelViewport> InLevelViewportWeak);
FToolMenuEntry CreateFarViewPlaneMenu(TWeakPtr<SLevelViewport> InLevelViewportWeak);
FToolMenuEntry CreateCameraSpeedSlider(TWeakPtr<SLevelViewport> InLevelViewportWeak);
FToolMenuEntry CreateCameraSpeedScalarSlider(TWeakPtr<SLevelViewport> InLevelViewportWeak);
FToolMenuEntry CreateToolbarCameraSubmenu();

TSharedPtr<FExtender> GetViewModesLegacyExtenders();
void PopulateViewModesMenu(UToolMenu* InMenu);
void ExtendViewModesSubmenu(FName InViewModesSubmenuName);
FToolMenuEntry CreatePIEViewModesSubmenu();

FToolMenuEntry CreateShowFoliageSubmenu();
FToolMenuEntry CreateShowHLODsSubmenu();
FToolMenuEntry CreateShowLayersSubmenu();
FToolMenuEntry CreateShowSpritesSubmenu();
#if STATS
FToolMenuEntry CreateShowStatsSubmenu(
	bool bInAddToggleStatsCheckbox = false, TAttribute<FText> InLabelOverride = TAttribute<FText>()
);
#endif
FToolMenuEntry CreateDynamicChannelsSubmenu();
FToolMenuEntry CreateShowVolumesSubmenu();
FToolMenuEntry CreateShowSubmenu();
FToolMenuEntry CreatePIEShowSubmenu();

FToolMenuEntry CreateFeatureLevelPreviewSubmenu();
FToolMenuEntry CreateMaterialQualityLevelSubmenu();
FToolMenuEntry CreatePerformanceAndScalabilitySubmenu();
FToolMenuEntry CreatePIEPerformanceAndScalabilitySubmenu();

void GenerateViewportLayoutsMenu(UToolMenu* InMenu);
TSharedRef<SWidget> BuildVolumeControlCustomWidget();
TSharedRef<SWidget> BuildPIEVolumeControlCustomWidget(const TWeakPtr<SLevelViewport>& Viewport);
FToolMenuEntry CreateSettingsSubmenu();
FToolMenuEntry CreatePIESettingsSubmenu();

FToolMenuEntry CreateViewportSizingSubmenu();

void ExtendCameraSubmenu(FName InCameraOptionsSubmenuName);
void ExtendTransformSubmenu(FName InTransformSubmenuName);
void ExtendSnappingSubmenu(FName InSnappingSubmenuName);
} // namespace UE::LevelEditor
