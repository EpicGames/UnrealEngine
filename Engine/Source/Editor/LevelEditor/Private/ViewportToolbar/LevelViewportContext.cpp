// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelViewportContext.h"

#include "Editor/LevelEditor/Private/LevelEditorInternalTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelViewportContext)

FLevelEditorViewportClient* ULegacyLevelViewportToolbarContext::GetLevelViewportClient() const
{
	if (TSharedPtr<SLevelViewport> Viewport = LevelViewport.Pin())
	{
		return &Viewport->GetLevelViewportClient();
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Super::GetLevelViewportClient();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
