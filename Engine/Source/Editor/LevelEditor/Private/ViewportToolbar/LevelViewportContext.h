// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "SLevelViewport.h"
#include "LevelEditorMenuContext.h"

#include "LevelViewportContext.generated.h"


#define UE_API LEVELEDITOR_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Customizes ULevelViewportToolBarContext for backwards compatability
// TODO: Remove this once the old toolbars are deprecated.
UCLASS(MinimalAPI)
class ULegacyLevelViewportToolbarContext : public ULevelViewportToolBarContext
{
	GENERATED_BODY()
	
public:
	TWeakPtr<SLevelViewport> LevelViewport;
	
	UE_API virtual FLevelEditorViewportClient* GetLevelViewportClient() const override;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
