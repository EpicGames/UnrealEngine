// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DlgDeltaTransform.h: UnrealEd dialog for moving assets.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

class SDlgDeltaTransform;
class SWindow;

/** 
 * FDlgDeltaTransform
 * 
 * Wrapper class for SDlgDeltaTransform. This class creates and launches the dialog then awaits the
 * Result to return to the user. 
 */
class FDlgDeltaTransform
{
public:

	/** 
	 * Constructs A move Asset Dialog
	 *
	 * @param bInLegacyOrMapPackage		-	Used to handle moving into or out of map packages, or legacy packages.
	 * @param InPackage					-	Package information of the current asset
	 * @param InGroup					-	Group information of the current asset
	 * @param InName					-	Name information of the current asset
	 */
	FDlgDeltaTransform();

	/** Custom result types */
	enum EResult
	{
		Cancel = 0,
		OK,
	};

	/** 
	 * Displays the dialog in a blocking fashion
	 *
	 * returns User response in EResult form.
	 */
	EResult ShowModal();

private:

	/** Cached pointer to the modal window */
	TSharedPtr<SWindow> DeltaTransformWindow;

	/** Cached pointer to the message box held within the window */
	TSharedPtr<class SDlgDeltaTransform> DeltaTransformWidget;
};
