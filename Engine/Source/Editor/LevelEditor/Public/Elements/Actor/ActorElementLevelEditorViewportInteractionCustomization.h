// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Actor/ActorElementEditorViewportInteractionCustomization.h"
#include "Elements/Framework/TypedElementAssetEditorLevelEditorViewportClientMixin.h"

#define UE_API LEVELEDITOR_API

class FActorElementLevelEditorViewportInteractionCustomization : public FActorElementEditorViewportInteractionCustomization, public FTypedElementAssetEditorLevelEditorViewportClientMixin
{
public:
	UE_API virtual void GizmoManipulationStarted(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode) override;
	UE_API virtual void GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation) override;
	UE_API virtual void GizmoManipulationStopped(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType) override;
	UE_API virtual void PostGizmoManipulationStopped(TArrayView<const FTypedElementHandle> InElementHandles, const UE::Widget::EWidgetMode InWidgetMode) override;

private:
	UE_API void ModifyScale(AActor* InActor, const EAxisList::Type InDragAxis, FVector& ScaleDelta, bool bCheckSmallExtent);
};

#undef UE_API
