// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Component/ComponentElementEditorViewportInteractionCustomization.h"
#include "Elements/Framework/TypedElementAssetEditorLevelEditorViewportClientMixin.h"

#define UE_API LEVELEDITOR_API

class FLevelEditorViewportClient;

class FComponentElementLevelEditorViewportInteractionCustomization : public FComponentElementEditorViewportInteractionCustomization, public FTypedElementAssetEditorLevelEditorViewportClientMixin
{
public:
	UE_API virtual void GizmoManipulationStarted(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode) override;
	UE_API virtual void GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation) override;
	UE_API virtual void GizmoManipulationStopped(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType) override;

	static UE_API void ValidateScale(const FVector& InOriginalPreDragScale, const EAxisList::Type InDragAxis, const FVector& InCurrentScale, const FVector& InBoxExtent, FVector& InOutScaleDelta, bool bInCheckSmallExtent);
	static UE_API FProperty* GetEditTransformProperty(const UE::Widget::EWidgetMode InWidgetMode);

private:
	UE_API void ModifyScale(USceneComponent* InComponent, const EAxisList::Type InDragAxis, FVector& ScaleDelta);
};

#undef UE_API
