// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FLevelEditorViewportClient;

/**
 * Mixin for asset editor customizations that provides access to the owner level editor viewport client.
 */
class FTypedElementAssetEditorLevelEditorViewportClientMixin
{
public:
	/**
	 * Get the owner level editor viewport client.
	 */
	const FLevelEditorViewportClient* GetLevelEditorViewportClient() const
	{
		return LevelEditorViewportClient;
	}

	/**
	 * Get the owner level editor viewport client.
	 */
	FLevelEditorViewportClient* GetMutableLevelEditorViewportClient()
	{
		return LevelEditorViewportClient;
	}

	/**
	 * Set the owner level editor viewport client.
	 */
	void SetLevelEditorViewportClient(FLevelEditorViewportClient* InLevelEditorViewportClient)
	{
		LevelEditorViewportClient = InLevelEditorViewportClient;
	}

private:
	/** The owner level editor viewport client. */
	FLevelEditorViewportClient* LevelEditorViewportClient = nullptr;
};
