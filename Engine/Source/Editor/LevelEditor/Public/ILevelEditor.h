// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Editor/UnrealEdTypes.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Commands/UICommandList.h"
#include "AssetThumbnail.h"
#include "Toolkits/IToolkitHost.h"

class FDetailsViewObjectFilter;
class IDetailRootObjectCustomization;
class ISceneOutliner;
class IAssetViewport;
class SLevelViewport;
class UTypedElementSelectionSet;
struct FSceneOutlinerInitializationOptions;
namespace UE::WorldHierarchy { class IWorldHierarchy; }

/**
 * Public interface to SLevelEditor
 */
class ILevelEditor : public SCompoundWidget, public IToolkitHost
{

public:

	/** Get the element selection set used by this level editor */
	virtual const UTypedElementSelectionSet* GetElementSelectionSet() const = 0;
	virtual UTypedElementSelectionSet* GetMutableElementSelectionSet() = 0;

	/** Summons a context menu for this level editor at the mouse cursor's location */
	virtual void SummonLevelViewportContextMenu(const FTypedElementHandle& HitProxyElement = FTypedElementHandle()) = 0;

	/** Gets the title for the context menu for this level editor */
	virtual FText GetLevelViewportContextMenuTitle() const = 0;

	/** Summons a context menu for view options */
	virtual void SummonLevelViewportViewOptionMenu(ELevelViewportType ViewOption) = 0;

	/** Returns a list of all of the toolkits that are currently hosted by this toolkit host */
	virtual const TArray< TSharedPtr< IToolkit > >& GetHostedToolkits() const = 0;

	/** Gets an array of all viewports in this level editor */
	virtual TArray< TSharedPtr< SLevelViewport > > GetViewports() const = 0;

	/** 
	 * Gets a list of all the viewport tabs in this level editor mapped to arrays of all viewports in that tab. 
	 * There are 4 viewport tabs, mapped to 0 to 3. Standalone viewports are mapped to index 4. 
	 */
	virtual TMap< int32, TArray< TSharedPtr< SLevelViewport > > > GetViewportsByTab() const = 0;
	
	/** Gets the active level viewport for this level editor */
	virtual TSharedPtr<SLevelViewport> GetActiveViewportInterface() = 0;

	/** Access the level editor's action command list */
	virtual const TSharedPtr< FUICommandList >& GetLevelEditorActions() const = 0;

	/** Called to process a key down event in a viewport when in immersive mode */
	virtual FReply OnKeyDownInViewport( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) = 0;

	/** Append commands to the command list for the level editor */
	virtual void AppendCommands( const TSharedRef<FUICommandList>& InCommandsToAppend ) = 0;

	/** After spawning a new level viewport outside of the editor's tab system, this function must be called so that
	    the editor can keep track of that viewport */
	virtual void AddStandaloneLevelViewport( const TSharedRef<SLevelViewport>& LevelViewport ) = 0;

	/** Spawns an Actor Details widget */
	virtual TSharedRef<SWidget> CreateActorDetails( const FName TabIdentifier ) = 0;

	/** Set the filter that should be used to determine the set of objects that should be shown in a details panel when an actor in the level editor is selected */
	virtual void SetActorDetailsRootCustomization(TSharedPtr<FDetailsViewObjectFilter> ActorDetailsObjectFilter, TSharedPtr<IDetailRootObjectCustomization> ActorDetailsRootCustomization) = 0;

	/** Adds the UI customization of the SCSEditor inside the level editor details panel. */
	virtual void AddActorDetailsSCSEditorUICustomization(TSharedPtr<class ISCSEditorUICustomization> ActorDetailsSCSEditorUICustomization) = 0;
	
	/** Remove the UI customization of the SCSEditor inside the level editor details panel. */
	virtual void RemoveActorDetailsSCSEditorUICustomization(TSharedPtr<class ISCSEditorUICustomization> ActorDetailsSCSEditorUICustomization) = 0;

	UE_DEPRECATED(5.5, "Use AddActorDetailsSCSEditorUICustomization/RemoveActorDetailsSCSEditorUICustomization instead.")
	virtual void SetActorDetailsSCSEditorUICustomization(TSharedPtr<class ISCSEditorUICustomization> ActorDetailsSCSEditorUICustomization) {};
	
	/** Gets the default FSceneOutlinerInitializationOptions for this Level Editor's Scene Outliners */
	virtual void GetSceneOutlinerInitializationOptions(const FName TabIdentifier, FSceneOutlinerInitializationOptions& OutInitOptions) = 0;

	/** Get an array containing weak pointers to all 4 Scene Outliners which could be potentially active */
	virtual TArray<TWeakPtr<ISceneOutliner>> GetAllSceneOutliners() const = 0;

	/** Set the outliner with the given name as the most recently interacted with */
	virtual void SetMostRecentlyUsedSceneOutliner(FName OutlinerIdentifier) = 0;

	/** Return the most recently interacted with Outliner */
	virtual TSharedPtr<ISceneOutliner> GetMostRecentlyUsedSceneOutliner() = 0;

	/** @return The widget displayed in the Levels tab, if the tab is summoned. */
	virtual TWeakPtr<UE::WorldHierarchy::IWorldHierarchy> GetWorldHierarchy() = 0;
};


