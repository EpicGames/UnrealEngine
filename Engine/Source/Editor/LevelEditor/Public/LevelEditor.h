// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleInterface.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Docking/LayoutService.h"
#include "ILevelEditor.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ViewportTypeDefinition.h"
#include "LevelEditorOutlinerSettings.h"

#define UE_API LEVELEDITOR_API

class AActor;
class IAssetViewport;
class SDockTab;
class SLevelEditor;
class UAnimSequence;
class USkeletalMeshComponent;
class UTypedElementSelectionSet;

enum class EMapChangeType : uint8;

extern const FName LevelEditorApp;

DECLARE_DELEGATE_RetVal_OneParam(bool, FAreObjectsEditable, const TArray<TWeakObjectPtr<UObject>>&);

class LevelEditorTabIds
{
public:
	static UE_API const FName LevelEditorViewport;
	static UE_API const FName LevelEditorViewport_Clone1;
	static UE_API const FName LevelEditorViewport_Clone2;
	static UE_API const FName LevelEditorViewport_Clone3;
	static UE_API const FName LevelEditorViewport_Clone4;
	static UE_API const FName LevelEditorToolBox;
	static UE_API const FName LevelEditorSelectionDetails;
	static UE_API const FName LevelEditorSelectionDetails2;
	static UE_API const FName LevelEditorSelectionDetails3;
	static UE_API const FName LevelEditorSelectionDetails4;
	static UE_API const FName PlacementBrowser;
	static UE_API const FName LevelEditorBuildAndSubmit;
	static UE_API const FName LevelEditorSceneOutliner;
	static UE_API const FName LevelEditorSceneOutliner2;
	static UE_API const FName LevelEditorSceneOutliner3;
	static UE_API const FName LevelEditorSceneOutliner4;
	static UE_API const FName LevelEditorStatsViewer;
	static UE_API const FName LevelEditorLayerBrowser;
	static UE_API const FName LevelEditorDataLayerBrowser;
	static UE_API const FName Sequencer;
	static UE_API const FName SequencerGraphEditor;
	static UE_API const FName WorldSettings;
	static UE_API const FName WorldBrowserComposition;
	static UE_API const FName WorldBrowserPartitionEditor;
	static UE_API const FName WorldBrowserHierarchy;
	static UE_API const FName WorldBrowserDetails;
	static UE_API const FName LevelEditorHierarchicalLODOutliner;
	static UE_API const FName OutputLog;
	static UE_API const FName LevelEditorEnvironmentLightingViewer;
};

/**
 * Level editor module
 */
class FLevelEditorModule : public IModuleInterface,
	public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/** Constructor, set up console commands and variables **/
	FLevelEditorModule();

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

	/**
	 * Override this to set whether your module is allowed to be unloaded on the fly
	 *
	 * @return Whether the module supports shutdown separate from the rest of the engine.
	 */
	virtual bool SupportsDynamicReloading() override
	{
		// @todo: Eventually, this should probably not be allowed.
		return true;
	}

	/**
	 * Spawns a new property viewer
	 * @todo This only works with the first level editor. Fix it.
	 */
	virtual void SummonSelectionDetails();

	/**
	 * Spawns a new build and submit widget
	 * @todo This only works with the first level editor. Fix it.
	 */
	virtual void SummonBuildAndSubmit();
	
	virtual void SummonWorldBrowserHierarchy();
	virtual void SummonWorldBrowserDetails();
	virtual void SummonWorldBrowserComposition();
	
	// @todo remove when world-centric mode is added
	/**
	 * Spawns a new sequencer tab if one doesn't exist already
	 * @todo This only works with the first level editor. Fix it.
	 */
	virtual TSharedPtr<SDockTab> AttachSequencer(TSharedPtr<SWidget> SequencerWidget, TSharedPtr<class IAssetEditorInstance> SequencerAssetEditor );

	/**
	 * Starts a play in editor session using the active viewport
	 */
	virtual void StartPlayInEditorSession();

	/**
	 * Takes the first active viewport and switches it to immersive mode.  Designed to be called at editor startup.  Optionally forces the viewport into game view mode too.
	 */
	virtual void GoImmersiveWithActiveLevelViewport( const bool bForceGameView );

	/**
	 * Toggles immersive mode on the currently active level viewport
	 */
	virtual void ToggleImmersiveOnActiveLevelViewport();

	/** @return Returns the first Level Editor that we currently know about */
	virtual TSharedPtr< class ILevelEditor > GetFirstLevelEditor() const;

	/** @return the dock tab in which the level editor currently resides. */
	virtual TSharedPtr<SDockTab> GetLevelEditorTab() const;

	/**
	 * Gets the first active viewport of all the viewports.
	 *
	 * @todo This only works with the first level editor. Fix it.
	 */
	virtual TSharedPtr<class IAssetViewport> GetFirstActiveViewport();

	/**
	* Gets the first active viewport of all the viewports.
	*
	* @todo This only works with the first level editor. Fix it.
	*/
	virtual TSharedPtr<class SLevelViewport> GetFirstActiveLevelViewport();

	/**
	 * Called to focus the level editor that has a play in editor viewport
	 */
	virtual void FocusPIEViewport();

	/**
	 * Called to focus the level editor viewport
	 */
	virtual void FocusViewport();

	/**
	 * @return The list of bound level editor commands that are common to all level editors
	 */
	virtual const TSharedRef<FUICommandList> GetGlobalLevelEditorActions() const { return GlobalLevelEditorActions.ToSharedRef(); }

	/**
	 * Method for getting level editor commands outside of this module                   
	 */
	virtual const class FLevelEditorCommands& GetLevelEditorCommands() const;

	/**
	 * Method for getting level editor modes commands outside of this module                   
	 */
	virtual const class FLevelEditorModesCommands& GetLevelEditorModesCommands() const;

	/**
	 * Method for getting level viewport commands outside of this module                   
	 */
	virtual const class FLevelViewportCommands& GetLevelViewportCommands() const;

	/* @return The pointer to the current level Editor instance */
	virtual TWeakPtr<class ILevelEditor> GetLevelEditorInstance() const;

	/* @return The pointer to the level editor tab */
	virtual TWeakPtr<class SDockTab> GetLevelEditorInstanceTab() const;

	/* @return The pointer to the level editor tab manager */
	virtual TSharedPtr<FTabManager> GetLevelEditorTabManager() const;

	/* Set the pointer to the current level Editor instance */
	virtual void SetLevelEditorInstance( TWeakPtr<class SLevelEditor> LevelEditor );

	/* Set the pointer to the level editor tab */
	virtual void SetLevelEditorInstanceTab( TWeakPtr<class SDockTab> LevelEditorTab );

	/* Create a tab manager for the level editor based on the given tab (or clears the tab manager if OwnerTab is null) */
	virtual void SetLevelEditorTabManager( const TSharedPtr<SDockTab>& OwnerTab );

	/** Called when the tab manager is changed */
	DECLARE_EVENT(FLevelEditorModule, FTabManagerChangedEvent);
	virtual FTabManagerChangedEvent& OnTabManagerChanged() { return TabManagerChangedEvent; }

	/** Called when the tab content is changed */
	DECLARE_EVENT(FLevelEditorModule, FTabContentChangedEvent);
	virtual FTabContentChangedEvent& OnTabContentChanged() { return TabContentChangedEvent; }

	/** Called when a level editor widget has been created */
	DECLARE_EVENT_OneParam(FLevelEditorModule, FOnLevelEditorCreated, TSharedPtr<ILevelEditor>);
	virtual FOnLevelEditorCreated& OnLevelEditorCreated() { return LevelEditorCreatedEvent; }
	
	/**
	 * Called when actor selection changes
	 * 
	 * @param NewSelection	List of objects that are now selected
	 */
	virtual void BroadcastActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh=false);
	
	/**
	 * Called when element selection changes
	 *
	 * @param SelectionSet	Set of selected elements
	 */
	virtual void BroadcastElementSelectionChanged(const UTypedElementSelectionSet* SelectionSet, bool bForceRefresh=false);

	/**
	 * Called to set property editors to show the given actors, even if those actors aren't in the current selection set
	 * 
	 * @param NewSelection	List of actors to edit the properties of
	 */
	virtual void BroadcastOverridePropertyEditorSelection(const TArray<AActor*>& NewSelection, bool bForceRefresh=false);

	/**
	 * Called by the engine when level editing viewports need to be redrawn
	 * 
	 * @param bInvalidateHitProxies	true to invalidate hit proxies
	 */
	virtual void BroadcastRedrawViewports( bool bInvalidateHitProxies );
	
	/**
	 * Called by the engine when level editing viewports need to take a high res screenshot
	 * 
	 */
	virtual void BroadcastTakeHighResScreenShots( );

	/**
	 * Called by the engine when a new map is loaded
	 * 
	 * @param World	The new world
	 */
	virtual void BroadcastMapChanged( UWorld* World, EMapChangeType MapChangeType );

	/** Called when an edit command is executed on one or more components in the world */
	virtual void BroadcastComponentsEdited();

	/** Called when actor selection changes */
	DECLARE_EVENT_TwoParams(FLevelEditorModule, FActorSelectionChangedEvent, const TArray<UObject*>&, bool);
	virtual FActorSelectionChangedEvent& OnActorSelectionChanged() { return ActorSelectionChangedEvent; }

	/** Called when element selection changes */
	DECLARE_EVENT_TwoParams(FLevelEditorModule, FElementSelectionChangedEvent, const UTypedElementSelectionSet*, bool);
	virtual FElementSelectionChangedEvent& OnElementSelectionChanged() { return ElementSelectionChangedEvent; }

	/** Called to set property editors to show the given actors, even if those actors aren't in the current selection set */
	DECLARE_EVENT_TwoParams(FLevelEditorModule, FOverridePropertyEditorSelectionEvent, const TArray<AActor*>&, bool);
	virtual FOverridePropertyEditorSelectionEvent& OnOverridePropertyEditorSelection() { return OverridePropertyEditorSelectionEvent; }

	/** Called when level editor viewports should be redrawn */
	DECLARE_EVENT_OneParam( FLevelEditorModule, FRedrawLevelEditingViewportsEvent, bool );
	virtual FRedrawLevelEditingViewportsEvent& OnRedrawLevelEditingViewports() { return RedrawLevelEditingViewportsEvent; }

	/** Called when a new map is loaded */
	DECLARE_EVENT_TwoParams( FLevelEditorModule, FMapChangedEvent, UWorld*, EMapChangeType );
	virtual FMapChangedEvent& OnMapChanged() { return MapChangedEvent; }

	/** Called when an edit command is executed on components in the world */
	DECLARE_EVENT(FLevelEditorModule, FComponentsEditedEvent);
	virtual FComponentsEditedEvent& OnComponentsEdited() { return ComponentsEditedEvent; }

	/** Delegates to be called to extend the level viewport menus */
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<FExtender>, FLevelEditorMenuExtender, const TSharedRef<FUICommandList>);
	DECLARE_DELEGATE_RetVal_TwoParams( TSharedRef<FExtender>, FLevelViewportMenuExtender_SelectedObjects, const TSharedRef<FUICommandList>, const TArray<UObject*>);
	DECLARE_DELEGATE_RetVal_TwoParams( TSharedRef<FExtender>, FLevelViewportMenuExtender_SelectedActors, const TSharedRef<FUICommandList>, const TArray<AActor*>);
	virtual TArray<FLevelViewportMenuExtender_SelectedObjects>& GetAllLevelViewportDragDropContextMenuExtenders() {return LevelViewportDragDropContextMenuExtenders;}
	virtual TArray<FLevelViewportMenuExtender_SelectedActors>& GetAllLevelViewportContextMenuExtenders() {return LevelViewportContextMenuExtenders;}
	virtual TArray<FLevelEditorMenuExtender>& GetAllLevelViewportOptionsMenuExtenders() {return LevelViewportOptionsMenuExtenders;}
	virtual TArray<FLevelEditorMenuExtender>& GetAllLevelViewportShowMenuExtenders() { return LevelViewportShowMenuExtenders; }
	virtual TArray<FLevelEditorMenuExtender>& GetAllLevelEditorToolbarViewMenuExtenders() {return LevelEditorToolbarViewMenuExtenders;}
	virtual TArray<FLevelEditorMenuExtender>& GetAllLevelEditorToolbarBuildMenuExtenders() {return LevelEditorToolbarBuildMenuExtenders;}
	virtual TArray<FLevelEditorMenuExtender>& GetAllLevelEditorToolbarCompileMenuExtenders() {return LevelEditorToolbarCompileMenuExtenders;}
	virtual TArray<FLevelEditorMenuExtender>& GetAllLevelEditorToolbarSourceControlMenuExtenders() { return LevelEditorToolbarSourceControlMenuExtenders; }
	virtual TArray<FLevelEditorMenuExtender>& GetAllLevelEditorToolbarCreateMenuExtenders() { return LevelEditorToolbarCreateMenuExtenders; }
	virtual TArray<FLevelEditorMenuExtender>& GetAllLevelEditorToolbarPlayMenuExtenders() { return LevelEditorToolbarPlayMenuExtenders; }
	virtual TArray<TSharedPtr<FExtender>>& GetAllLevelEditorToolbarBlueprintsMenuExtenders() { return LevelEditorToolbarBlueprintsMenuExtenders; }
	virtual TArray<TSharedPtr<FExtender>>& GetAllLevelEditorToolbarCinematicsMenuExtenders() {return LevelEditorToolbarCinematicsMenuExtenders;}
	virtual TArray<FLevelEditorMenuExtender>& GetAllLevelEditorLevelMenuExtenders() { return LevelEditorLevelMenuExtenders; }
	
	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override {return MenuExtensibilityManager;}
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override {return ToolBarExtensibilityManager;}
	virtual TSharedPtr<FExtensibilityManager> GetModeBarExtensibilityManager() {return ModeBarExtensibilityManager;}

	virtual TSharedPtr<FExtender> AssembleExtenders(TSharedRef<FUICommandList>& InCommandList, TArray<FLevelEditorMenuExtender>& MenuExtenderDelegates) const;

	DECLARE_EVENT_OneParam(ILevelEditor, FOnRegisterTabs, TSharedPtr<FTabManager>);
	FOnRegisterTabs& OnRegisterTabs() { return RegisterTabs; }

	DECLARE_EVENT_OneParam(ILevelEditor, FOnRegisterLayoutExtensions, FLayoutExtender&);
	FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() { return RegisterLayoutExtensions; }

	/** Add / Remove title bar item */
	struct FTitleBarItem
	{
		/* Visiblility of the title bar item. */
		TAttribute<EVisibility> Visibility;
		/* Label of the title bar item. */
		TAttribute<FText> Label;
		/* Value of the title bar item. */
		TAttribute<FText> Value;
	};
	const TMap<FName, FTitleBarItem>& GetTitleBarItems() const { return TitleBarItems; }
	virtual void AddTitleBarItem(FName InTitleBarIdentifier, const FTitleBarItem& InTitleBarITem);
	virtual void RemoveTitleBarItem(FName InTitleBarIdentifier);

	/** Called when a new map is loaded */
	DECLARE_EVENT( FLevelEditorModule, FTitleBarMessagesChanged );
	virtual FTitleBarMessagesChanged& OnTitleBarMessagesChanged() { return TitleBarMessagesChangedEvent; }

	virtual void BroadcastTitleBarMessagesChanged() { TitleBarMessagesChangedEvent.Broadcast(); }

	/** Called when a high res screenshot is requested. */
	DECLARE_EVENT( FLevelEditorModule, FTakeHighResScreenShotsEvent );
	virtual FTakeHighResScreenShotsEvent& OnTakeHighResScreenShots() { return TakeHighResScreenShotsEvent; }

	/** Delegate used to capture skeletal meshes to single-frame animations when 'keeping simulation changes' */
	DECLARE_DELEGATE_RetVal_OneParam(UAnimSequence*, FCaptureSingleFrameAnimSequence, USkeletalMeshComponent* /*Component*/);
	virtual FCaptureSingleFrameAnimSequence& OnCaptureSingleFrameAnimSequence() { return CaptureSingleFrameAnimSequenceDelegate; }

	/** Delegate used to determine if users should be able to enable the alpha-blended checkerboard for the level viewport */
	DECLARE_DELEGATE_RetVal(bool, FGetVisualizeCheckerboardEnabled);
	virtual FGetVisualizeCheckerboardEnabled& OnGetVisualizeCheckerboardEnabled() { return GetVisualizeCheckerboardEnabled; }
	
	/**  Add a custom filter to the outliner filter bar. These are all AND'd together
	 * @see FLevelEditorOutlinerSettings
	 *   @see FGenericFilter on how to create generic filters
	 *   NOTE: Currently does not support adding filters dynamically, they must be added before the Level Editor is init
	 */
	UE_DEPRECATED(5.2, "Use the AddCustomFilterToOutliner override that takes in a delegate instead so each Outliner can have a unique instance of the filter")
	virtual void AddCustomFilterToOutliner(TSharedRef<FFilterBase<const ISceneOutlinerTreeItem&>> InCustomFilter);

	/**  Add a custom filter to the outliner filter bar. These are all AND'd together
	 *	 This function takes in a delegate that will be called to create an instance of the custom filter for each Outliner that
	 *	 the level editor creates.
	 *   @see FGenericFilter on how to create generic filters
	 */
	virtual void AddCustomFilterToOutliner(FLevelEditorOutlinerSettings::FOutlinerFilterFactory InCreateCustomFilter);

	/**  Add a custom class filter to the outliner filter bar. These represent asset/actor type filters and are OR'd
	 *   @see FLevelEditorOutlinerSettings
	 *   Can be created using IAssetTypeActions or UClass (@see constructor)
	 *   NOTE: Currently does not support adding filters dynamically, they must be added before the Level Editor is init
	 */
	virtual void AddCustomClassFilterToOutliner(TSharedRef<FCustomClassFilterData> InCustomClassFilterData);
	
	/** Get the FFilterCategory attached to the given category name. Use this to add filters to the built in categories
	 *  @see FLevelEditorOutlinerBuiltInCategories
	 */
	virtual TSharedPtr<FFilterCategory> GetOutlinerFilterCategory(const FName& CategoryName) const;

public:

	/** Add a delegate that will get called to check whether the specified objects should be editable on the details panel or not */
	void AddEditableObjectPredicate(const FAreObjectsEditable& InPredicate)
	{
		check(InPredicate.IsBound());
		AreObjectsEditableDelegates.Add(InPredicate);
	}

	/** Remove a delegate that was added via AddEditableObjectPredicate */
	void RemoveEditableObjectPredicate(FDelegateHandle InPredicateHandle)
	{
		AreObjectsEditableDelegates.RemoveAll([=](const FAreObjectsEditable& P){ return P.GetHandle() == InPredicateHandle; });
	}

	/** Check whether the specified objects are editable */
	bool AreObjectsEditable(const TArray<TWeakObjectPtr<UObject>>& InObjects) const
	{
		for (const FAreObjectsEditable& Predicate : AreObjectsEditableDelegates)
		{
			if (!Predicate.Execute(InObjects))
			{
				return false;
			}
		}
		return true;
	}

	/** Return the settings (containing the outliner filters) used to create the level editor's outliners  */
	virtual TSharedPtr<FLevelEditorOutlinerSettings> GetLevelEditorOutlinerSettings() const;
	
	/** Refresh all of the level editor's outliners (close and recreate them) */
	virtual void RefreshLevelEditorOutliners() const;

public:
	
	/** Register a viewport type for the level editor */
	void RegisterViewportType(FName InLayoutName, const FViewportTypeDefinition& InDefinition)
	{
		CustomViewports.Add(InLayoutName, InDefinition);
	}

	/** Unregister a previously registered viewport type */
	void UnregisterViewportType(FName InLayoutName)
	{
		CustomViewports.Remove(InLayoutName);
	}

	/** Iterate all registered viewport types */
	void IterateViewportTypes(TFunctionRef<void(FName, const FViewportTypeDefinition&)> Iter)
	{
		for (auto& Pair : CustomViewports)
		{
			Iter(Pair.Key, Pair.Value);
		}
	}

	/** Create an instance of a custom viewport from the specified viewport type name */
	UE_DEPRECATED(4.26, "Use FactoryViewport from inside of a FLevelViewportTabContent.")
	TSharedRef<ILevelViewportLayoutEntity> FactoryViewport(FName InTypeName, const FAssetEditorViewportConstructionArgs& ConstructionArgs) const;

private:
	/**
	 * Binds all global level editor commands to delegates
	 */
	void BindGlobalLevelEditorCommands();

	// Callback for persisting the Level Editor's layout.
	void HandleTabManagerPersistLayout( const TSharedRef<FTabManager::FLayout>& LayoutToSave )
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToSave);
	}

	/** Spawn the main level editor tab */
	TSharedRef<SDockTab> SpawnLevelEditor( const FSpawnTabArgs& InArgs );

	/** Extend the dock tab context menu */
	void OnExtendTabContextMenu(FMenuBuilder& InMenuBuilder);

	/** Is the active level an asset that can be browsed to */
	bool OnCanBrowseLevelInContentBrowser();

	/** Browse to active level in content browser */
	void OnBrowseLevelInContentBrowser();

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ModeBarExtensibilityManager;
	TSharedPtr<FExtensibilityManager> NotificationBarExtensibilityManager;

	TMap<FName, FTitleBarItem> TitleBarItems;
	FTitleBarMessagesChanged TitleBarMessagesChangedEvent;

	/** 
	 * A command list that can be passed around and isn't bound to an instance of a level editor. 
	 * Only commands that arent bound to an instance of SLevelEditor should be used here (like level context menu commands)
	 */
	TSharedPtr<FUICommandList> GlobalLevelEditorActions;

	FAutoConsoleCommand ToggleImmersiveConsoleCommand;

	/** Multicast delegate executed when the tab manager is changed */
	FTabManagerChangedEvent TabManagerChangedEvent;

	/** Multicast delegate executed when the tab content is changed */
	FTabContentChangedEvent TabContentChangedEvent;

	/** Multicast delegate executed when a level editor has been created */
	FOnLevelEditorCreated LevelEditorCreatedEvent;

	/** Multicast delegate executed when actor selection changes */
	FActorSelectionChangedEvent ActorSelectionChangedEvent;

	/** Multicast delegate executed when element selection changes */
	FElementSelectionChangedEvent ElementSelectionChangedEvent;

	/** Multicast delegate executed to set property editors to show the given actors, even if those actors aren't in the current selection set */
	FOverridePropertyEditorSelectionEvent OverridePropertyEditorSelectionEvent;

	/** Multicast delegate executed when viewports should be redrawn */
	FRedrawLevelEditingViewportsEvent RedrawLevelEditingViewportsEvent;

	/** Multicast delegate executed after components are edited in the world */
	FComponentsEditedEvent ComponentsEditedEvent;

	/** Multicast delegate executed when viewports should be redrawn */
	FTakeHighResScreenShotsEvent TakeHighResScreenShotsEvent;

	/** Multicast delegate executed when a map is changed (loaded,saved,new map, etc) */
	FMapChangedEvent MapChangedEvent;

	/** Delegate used to capture skeletal meshes to single-frame animations when 'keeping simulation changes' */
	FCaptureSingleFrameAnimSequence CaptureSingleFrameAnimSequenceDelegate;

	/** Delegate used to determine if 'Visualize Alpha with Checkerboard' checkbox is enabled in the level viewports  */
	FGetVisualizeCheckerboardEnabled GetVisualizeCheckerboardEnabled;
	
	/** All extender delegates for the level viewport menus */
	TArray<FLevelViewportMenuExtender_SelectedObjects> LevelViewportDragDropContextMenuExtenders;
	TArray<FLevelViewportMenuExtender_SelectedActors> LevelViewportContextMenuExtenders;
	TArray<FLevelEditorMenuExtender> LevelViewportOptionsMenuExtenders;
	TArray<FLevelEditorMenuExtender> LevelViewportShowMenuExtenders;
	TArray<FLevelEditorMenuExtender> LevelEditorToolbarViewMenuExtenders;
	TArray<FLevelEditorMenuExtender> LevelEditorToolbarBuildMenuExtenders;
	TArray<FLevelEditorMenuExtender> LevelEditorToolbarCompileMenuExtenders;
	TArray<FLevelEditorMenuExtender> LevelEditorToolbarSourceControlMenuExtenders;
	TArray<FLevelEditorMenuExtender> LevelEditorToolbarCreateMenuExtenders;
	TArray<FLevelEditorMenuExtender> LevelEditorToolbarPlayMenuExtenders;
	TArray<TSharedPtr<FExtender>> LevelEditorToolbarCinematicsMenuExtenders;
	TArray<TSharedPtr<FExtender>> LevelEditorToolbarBlueprintsMenuExtenders;
	TArray<FLevelEditorMenuExtender> LevelEditorLevelMenuExtenders;

	/* Pointer to the current level Editor instance */
	TWeakPtr<class SLevelEditor> LevelEditorInstancePtr;

	/* Pointer to the level editor tab */
	TWeakPtr<class SDockTab> LevelEditorInstanceTabPtr;

	/* Holds the Editor's tab manager */
	TSharedPtr<FTabManager> LevelEditorTabManager;

	/** Map of named viewport types to factory functions */
	TMap<FName, FViewportTypeDefinition> CustomViewports;

	/** Register layout extensions event */
	FOnRegisterLayoutExtensions RegisterLayoutExtensions;

	/** Register external tabs event */
	FOnRegisterTabs RegisterTabs;

	/** Array of delegates that are used to check if the specified objects should be editable on the details panel */
	TArray<FAreObjectsEditable> AreObjectsEditableDelegates;

	/** The settings used to create any Outliners in the Level Editor */
	TSharedPtr<FLevelEditorOutlinerSettings> OutlinerSettings;
};

#undef UE_API
