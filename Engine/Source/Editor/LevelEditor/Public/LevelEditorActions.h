// Copyright Epic Games, Inc. All Rights Reserved.



#pragma once

#include "CoreMinimal.h"
#include "UnrealWidgetFwd.h"
#include "SceneTypes.h"
#include "Framework/Commands/Commands.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "Toolkits/IToolkit.h"
#include "Styling/AppStyle.h"
#include "TexAligner/TexAligner.h"
#include "LightmapResRatioAdjust.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"

#define UE_API LEVELEDITOR_API

class SLevelViewport;
class FLightingBuildOptions;
class SLevelEditor;
class UActorFactory;
class UTypedElementSelectionSet;

/**
 * Unreal level editor actions
 */
class FLevelEditorCommands : public TCommands<FLevelEditorCommands>
{

public:
	FLevelEditorCommands();

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

	LEVELEDITOR_API FORCENOINLINE static const FLevelEditorCommands& Get();

public:
	
	TSharedPtr< FUICommandInfo > BrowseDocumentation;
	TSharedPtr< FUICommandInfo > BrowseViewportControls;

	/** Level file commands */
	TSharedPtr<FUICommandInfo> NewLevel;
	UE_DEPRECATED(5.5, "This command has been moved to FGlobalEditorCommonCommands and is no longer registered.")
	TSharedPtr<FUICommandInfo> OpenLevel;
	TSharedPtr<FUICommandInfo> Save;
	TSharedPtr<FUICommandInfo> SaveAs;
	TSharedPtr<FUICommandInfo> SaveAllLevels;
	TSharedPtr<FUICommandInfo> BrowseLevel;

	static const int32 MaxRecentFiles = 10;
	TArray< TSharedPtr< FUICommandInfo > > OpenRecentFileCommands;
	static const int32 MaxFavoriteFiles = 20;
	TArray< TSharedPtr< FUICommandInfo > > OpenFavoriteFileCommands;
	
	TSharedPtr< FUICommandInfo > ClearRecentFiles;

	TSharedPtr< FUICommandInfo > ToggleFavorite;

	/** Import Scene */
	TSharedPtr< FUICommandInfo > ImportScene;

	/** Export All */
	TSharedPtr< FUICommandInfo > ExportAll;

	/** Export Selected */
	TSharedPtr< FUICommandInfo > ExportSelected;

	
	/** Build commands */
	static constexpr int32 MaxExternalBuildTypes = 10;
	TArray<TSharedPtr< FUICommandInfo >> ExternalBuildTypeCommands;

	TSharedPtr< FUICommandInfo > Build;
	TSharedPtr< FUICommandInfo > BuildAndSubmitToSourceControl;
	TSharedPtr< FUICommandInfo > BuildLightingOnly;
	TSharedPtr< FUICommandInfo > BuildReflectionCapturesOnly;
	TSharedPtr< FUICommandInfo > BuildLightingOnly_VisibilityOnly;
	TSharedPtr< FUICommandInfo > LightingBuildOptions_UseErrorColoring;
	TSharedPtr< FUICommandInfo > LightingBuildOptions_ShowLightingStats;
	TSharedPtr< FUICommandInfo > BuildGeometryOnly;
	TSharedPtr< FUICommandInfo > BuildGeometryOnly_OnlyCurrentLevel;
	TSharedPtr< FUICommandInfo > BuildPathsOnly;
	TSharedPtr< FUICommandInfo > BuildHLODs;
	TSharedPtr< FUICommandInfo > BuildHLODs_SelectedActors;
	TSharedPtr< FUICommandInfo > BuildMinimap;
	TSharedPtr< FUICommandInfo > BuildLandscapeSplineMeshes;
	TSharedPtr< FUICommandInfo > BuildTextureStreamingOnly;
	TSharedPtr< FUICommandInfo > BuildVirtualTextureOnly;
	TSharedPtr< FUICommandInfo > BuildAllLandscape;
	TSharedPtr< FUICommandInfo > LightingQuality_Production;
	TSharedPtr< FUICommandInfo > LightingQuality_High;
	TSharedPtr< FUICommandInfo > LightingQuality_Medium;
	TSharedPtr< FUICommandInfo > LightingQuality_Preview;
	TSharedPtr< FUICommandInfo > LightingDensity_RenderGrayscale;
	TSharedPtr< FUICommandInfo > LightingResolution_CurrentLevel;
	TSharedPtr< FUICommandInfo > LightingResolution_SelectedLevels;
	TSharedPtr< FUICommandInfo > LightingResolution_AllLoadedLevels;
	TSharedPtr< FUICommandInfo > LightingResolution_SelectedObjectsOnly;
	TSharedPtr< FUICommandInfo > LightingStaticMeshInfo;
	TSharedPtr< FUICommandInfo > SceneStats;
	TSharedPtr< FUICommandInfo > TextureStats;
	TSharedPtr< FUICommandInfo > MapCheck;

	/** Recompile */
	TSharedPtr< FUICommandInfo > RecompileLevelEditor;
	TSharedPtr< FUICommandInfo > ReloadLevelEditor;
	TSharedPtr< FUICommandInfo > RecompileGameCode;

#if WITH_LIVE_CODING
	TSharedPtr< FUICommandInfo > LiveCoding_Enable;
	TSharedPtr< FUICommandInfo > LiveCoding_StartSession;
	TSharedPtr< FUICommandInfo > LiveCoding_ShowConsole;
	TSharedPtr< FUICommandInfo > LiveCoding_Settings;
#endif

	/**
	 * Level context menu commands.  These are shared between all viewports
	 * and rely on GCurrentLevelEditingViewport
	 * @todo Slate: Do these belong in their own context?
	 */

	/** Edits associated asset(s), prompting for confirmation if there is more than one selected */
	TSharedPtr< FUICommandInfo > EditAsset;

	/** Edits associated asset(s) */
	TSharedPtr< FUICommandInfo > EditAssetNoConfirmMultiple;

	/** Opens the associated asset(s) in the property matrix */
	TSharedPtr< FUICommandInfo > OpenSelectionInPropertyMatrix;

	/** Moves the camera to the current mouse position */
	TSharedPtr< FUICommandInfo > GoHere;
	
	/** Plays from the current position (selected actor or clicked) */
	TSharedPtr< FUICommandInfo > PlayFromHere;

	/** Snaps the camera to the selected object. */
	TSharedPtr< FUICommandInfo > SnapCameraToObject;

	/** Snaps the selected actor to the camera. */
	TSharedPtr< FUICommandInfo > SnapObjectToCamera;

	/** Copy the file path where the actor is saved. */
	TSharedPtr< FUICommandInfo > CopyActorFilePathtoClipboard;

	/** Opens the reference viewer for the selected actor. */
	TSharedPtr< FUICommandInfo > OpenActorInReferenceViewer;

	/** Save the selected actor. */
	TSharedPtr< FUICommandInfo > SaveActor;

	/** Opens the source control panel. */
	TSharedPtr< FUICommandInfo > OpenSourceControl;

	/** Shows the history of the file containing the actor. */
	TSharedPtr< FUICommandInfo > ShowActorHistory;

	/** Goes to the source code for the selected actor's class. */
	TSharedPtr< FUICommandInfo > GoToCodeForActor;

	/** Goes to the documentation for the selected actor's class. */
	TSharedPtr< FUICommandInfo > GoToDocsForActor;

	/** Customize the script behavior of an instance. */
	TSharedPtr< FUICommandInfo > AddScriptBehavior;

	/** Paste actor at click location*/
	TSharedPtr< FUICommandInfo > PasteHere;

	/**
	 * Actor Transform Commands                   
	 */

	/** Snaps the actor to the grid at its pivot*/
	TSharedPtr< FUICommandInfo > SnapOriginToGrid;

	/** Snaps each selected actor separately to the grid at its pivot*/
	TSharedPtr< FUICommandInfo > SnapOriginToGridPerActor;

	/** Aligns the actor to the grid at its pivot*/
	TSharedPtr< FUICommandInfo > AlignOriginToGrid;

	/** Snaps the actor to the 2D layer */
	TSharedPtr< FUICommandInfo > SnapTo2DLayer;

	/** Moves the selected actors up one 2D layer (changing the active layer at the same time) */
	TSharedPtr< FUICommandInfo > MoveSelectionUpIn2DLayers;

	/** Moves the selected actors down one 2D layer (changing the active layer at the same time) */
	TSharedPtr< FUICommandInfo > MoveSelectionDownIn2DLayers;

	/** Moves the selected actors to the top 2D layer (changing the active layer at the same time) */
	TSharedPtr< FUICommandInfo > MoveSelectionToTop2DLayer;

	/** Moves the selected actors to the bottom 2D layer (changing the active layer at the same time) */
	TSharedPtr< FUICommandInfo > MoveSelectionToBottom2DLayer;

	/** Changes the active 2D layer to one above the current one */
	TSharedPtr< FUICommandInfo > Select2DLayerAbove;

	/** Changes the active 2D layer to one below the current one */
	TSharedPtr< FUICommandInfo > Select2DLayerBelow;

	/** Snaps the actor to the floor*/
	TSharedPtr< FUICommandInfo > SnapToFloor;

	/** Aligns the actor with the floor */
	TSharedPtr< FUICommandInfo > AlignToFloor;

	/** Snaps the actor to the floor at its pivot*/
	TSharedPtr< FUICommandInfo > SnapPivotToFloor;

	/** Aligns the actor to the floor at its pivot */
	TSharedPtr< FUICommandInfo > AlignPivotToFloor;

	/** Snaps the actor to the floor at its bottom center bounds position*/
	TSharedPtr< FUICommandInfo > SnapBottomCenterBoundsToFloor;

	/** Aligns the actor to the floor at its bottom center bounds position*/
	TSharedPtr< FUICommandInfo > AlignBottomCenterBoundsToFloor;

	/** Snaps the actor to another actor at its pivot*/
	TSharedPtr< FUICommandInfo > SnapOriginToActor;

	/** Aligns the actor to another actor at its pivot*/
	TSharedPtr< FUICommandInfo > AlignOriginToActor;

	/** Snaps the actor to another actor */
	TSharedPtr< FUICommandInfo > SnapToActor;

	/** Aligns the actor with another actor */
	TSharedPtr< FUICommandInfo > AlignToActor;

	/** Snaps the actor to another actor at its pivot */
	TSharedPtr< FUICommandInfo > SnapPivotToActor;

	/** Aligns the actor to another actor at its pivot */
	TSharedPtr< FUICommandInfo > AlignPivotToActor;

	/** Snaps the actor to the Actor at its bottom center bounds position*/
	TSharedPtr< FUICommandInfo > SnapBottomCenterBoundsToActor;

	/** Aligns the actor to the Actor at its bottom center bounds position*/
	TSharedPtr< FUICommandInfo > AlignBottomCenterBoundsToActor;

	/** Apply delta transform to selected actors */	
	TSharedPtr< FUICommandInfo > DeltaTransformToActors;

	/** Mirros the actor along the x axis */	
	TSharedPtr< FUICommandInfo > MirrorActorX;
	 
	/** Mirros the actor along the y axis */	
	TSharedPtr< FUICommandInfo > MirrorActorY;

	/** Mirros the actor along the z axis */	
	TSharedPtr< FUICommandInfo > MirrorActorZ;

	/** Locks the actor so it cannot be moved */
	TSharedPtr< FUICommandInfo > LockActorMovement;

	/** Saves the pivot to the pre-pivot */
	TSharedPtr< FUICommandInfo > SavePivotToPrePivot;

	/** Resets the pre-pivot */
	TSharedPtr< FUICommandInfo > ResetPrePivot;

	/** Resets the pivot */
	TSharedPtr< FUICommandInfo > ResetPivot;

	/** Moves the pivot to the click location */
	TSharedPtr< FUICommandInfo > MovePivotHere;

	/** Moves the pivot to the click location and snap it to the grid */
	TSharedPtr< FUICommandInfo > MovePivotHereSnapped;

	/** Moves the pivot to the center of the selection */
	TSharedPtr< FUICommandInfo > MovePivotToCenter;

	/** Detach selected actor(s) from any parent */
	TSharedPtr< FUICommandInfo > DetachFromParent;

	TSharedPtr< FUICommandInfo > AttachSelectedActors;

	TSharedPtr< FUICommandInfo > AttachActorIteractive;

	TSharedPtr< FUICommandInfo > CreateNewOutlinerFolder;

	TSharedPtr< FUICommandInfo > HoldToEnableVertexSnapping;
	TSharedPtr< FUICommandInfo > HoldToEnablePivotVertexSnapping;

	/**
	 * Brush Commands                   
	 */

	/** Put the selected brushes first in the draw order */
	TSharedPtr< FUICommandInfo > OrderFirst;

	/** Put the selected brushes last in the draw order */
	TSharedPtr< FUICommandInfo > OrderLast;

	/** Converts the brush to an additive brush */
	TSharedPtr< FUICommandInfo > ConvertToAdditive;

	/** Converts the brush to a subtractive brush */
	TSharedPtr< FUICommandInfo > ConvertToSubtractive;
	
	/** Make the brush solid */
	TSharedPtr< FUICommandInfo > MakeSolid;

	/** Make the brush semi-solid */
	TSharedPtr< FUICommandInfo > MakeSemiSolid;

	/** Make the brush non-solid */
	TSharedPtr< FUICommandInfo > MakeNonSolid;

	/** Merge bsp polys into as few faces as possible*/
	TSharedPtr< FUICommandInfo > MergePolys;

	/** Reverse a merge */
	TSharedPtr< FUICommandInfo > SeparatePolys;

	/** Align brush vertices to the grid */
	TSharedPtr<FUICommandInfo> AlignBrushVerticesToGrid;

	/**
	 * Actor group commands
	 */

	/** Group or regroup the selected actors depending on context*/
	TSharedPtr< FUICommandInfo > RegroupActors;
	/** Groups selected actors */
	TSharedPtr< FUICommandInfo > GroupActors;
	/** Ungroups selected actors */
	TSharedPtr< FUICommandInfo > UngroupActors;
	/** Adds the selected actors to the selected group */
	TSharedPtr< FUICommandInfo > AddActorsToGroup;
	/** Removes selected actors from the group */
	TSharedPtr< FUICommandInfo > RemoveActorsFromGroup;
	/** Locks the selected group */
	TSharedPtr< FUICommandInfo > LockGroup;
	/** Unlocks the selected group */
	TSharedPtr< FUICommandInfo > UnlockGroup;
		
	/**
	 * Visibility commands                   
	 */
	/** Shows all actors */
	TSharedPtr< FUICommandInfo > ShowAll;

	/** Shows only selected actors */
	TSharedPtr< FUICommandInfo > ShowSelectedOnly;

	/** Unhides selected actors */
	TSharedPtr< FUICommandInfo > ShowSelected;

	/** Hides selected actors */
	TSharedPtr< FUICommandInfo > HideSelected;
	
	/** Hides or shows selected actors */
	TSharedPtr< FUICommandInfo > ToggleSelectedVisibility;

	/** Unhides selected actors and their children */
	TSharedPtr< FUICommandInfo > ShowSelectedHierarchy;
	
	/** Hides selected actors and their children */
	TSharedPtr< FUICommandInfo > HideSelectedHierarchy;

	/** Hides or shows selected actors and their children */
	TSharedPtr< FUICommandInfo > ToggleSelectedHierarchyVisibility;

	/** Shows all actors at startup */
	TSharedPtr< FUICommandInfo > ShowAllStartup;

	/** Shows selected actors at startup */
	TSharedPtr< FUICommandInfo > ShowSelectedStartup;

	/** Hides selected actors at startup */
	TSharedPtr< FUICommandInfo > HideSelectedStartup;

	/** Shows selected actors and their current children at startup */
	TSharedPtr< FUICommandInfo > ShowSelectedHierarchyStartup;

	/** Hides selected actors and their current children at startup */
	TSharedPtr< FUICommandInfo > HideSelectedHierarchyStartup;

	/** Cycles through all navigation data to show one at a time */
	TSharedPtr< FUICommandInfo > CycleNavigationDataDrawn;

	/**
	 * Selection commands                    
	 */

	/** Select nothing */
	TSharedPtr< FUICommandInfo > SelectNone;

	/** Invert the current selection */
	TSharedPtr< FUICommandInfo > InvertSelection;

	/** Selects all direct children of the current selection */
	TSharedPtr< FUICommandInfo > SelectImmediateChildren;

	/** Selects all descendants of the current selection */
	TSharedPtr< FUICommandInfo > SelectAllDescendants;

	/** Selects all actors of the same class as the current selection */
	TSharedPtr< FUICommandInfo > SelectAllActorsOfSameClass;

	/** Selects all actors of the same class and archetype as the current selection */
	TSharedPtr< FUICommandInfo > SelectAllActorsOfSameClassWithArchetype;

	/** Selects the actor that owns the currently selected component(s) */
	TSharedPtr< FUICommandInfo > SelectComponentOwnerActor;

	/** Selects all lights relevant to the current selection */
	TSharedPtr< FUICommandInfo > SelectRelevantLights;

	/** Selects all actors using the same static mesh(es) as the current selection */
	TSharedPtr< FUICommandInfo > SelectStaticMeshesOfSameClass;

	/** Selects all actors using the same static mesh(es) and same actor class as the current selection */
	TSharedPtr< FUICommandInfo > SelectStaticMeshesAllClasses;

	/** Selects the HLOD cluster (ALODActor), if available, that has this actor as one of its SubActors */
	TSharedPtr< FUICommandInfo > SelectOwningHierarchicalLODCluster;

	/** Selects all actors using the same skeletal mesh(es) as the current selection */
	TSharedPtr< FUICommandInfo > SelectSkeletalMeshesOfSameClass;

	/** Selects all actors using the same skeletal mesh(es) and same actor class as the current selection */
	TSharedPtr< FUICommandInfo > SelectSkeletalMeshesAllClasses;

	/** Selects all actors using the same material(s) as the current selection */
	TSharedPtr< FUICommandInfo > SelectAllWithSameMaterial;

	/** Selects all emitters using the same particle system as the current selection */
	TSharedPtr< FUICommandInfo > SelectMatchingEmitter;

	/** Selects all lights */
	TSharedPtr< FUICommandInfo > SelectAllLights;

	/** Selects all lights exceeding the overlap limit */
	TSharedPtr< FUICommandInfo > SelectStationaryLightsExceedingOverlap;

	/** Selects all additive brushes */
	TSharedPtr< FUICommandInfo > SelectAllAddditiveBrushes;

	/** Selects all subtractive brushes */
	TSharedPtr< FUICommandInfo > SelectAllSubtractiveBrushes;

	/**
	 * Surface commands                   
	 */
	TSharedPtr< FUICommandInfo > SelectAllSurfaces;

	/** Select all surfaces in the same brush as the current surface selection */
	TSharedPtr< FUICommandInfo > SurfSelectAllMatchingBrush;

	/** Select all surfaces using the same material as current surface selection */
	TSharedPtr< FUICommandInfo > SurfSelectAllMatchingTexture;

	/** Select all surfaces adjacent to current surface selection */
	TSharedPtr< FUICommandInfo > SurfSelectAllAdjacents;

	/** Select all surfaces adjacent and coplanar to current surface selection */
	TSharedPtr< FUICommandInfo > SurfSelectAllAdjacentCoplanars;

	/** Select all surfaces adjacent to to current surface selection that are walls*/
	TSharedPtr< FUICommandInfo > SurfSelectAllAdjacentWalls;

	/** Select all surfaces adjacent to to current surface selection that are floors(normals pointing up)*/
	TSharedPtr< FUICommandInfo > SurfSelectAllAdjacentFloors;

	/** Select all surfaces adjacent to to current surface selection that are slants*/
	TSharedPtr< FUICommandInfo > SurfSelectAllAdjacentSlants;

	/** Invert current surface selection */
	TSharedPtr< FUICommandInfo > SurfSelectReverse;

	/** Memorize current surface selection */
	TSharedPtr< FUICommandInfo > SurfSelectMemorize;

	/** Recall previously memorized selection */
	TSharedPtr< FUICommandInfo > SurfSelectRecall;

	/** Replace the current selection with only the surfaces which are both currently selected and contained within the saved selection in memory */
	TSharedPtr< FUICommandInfo > SurfSelectOr;
	
	/**	Add the selection of surfaces saved in memory to the current selection */
	TSharedPtr< FUICommandInfo > SurfSelectAnd;

	/** Replace the current selection with only the surfaces that are not in both the current selection and the selection saved in memory */
	TSharedPtr< FUICommandInfo > SurfSelectXor;

	/** Unalign surface texture */
	TSharedPtr< FUICommandInfo > SurfUnalign;

	/** Auto align surface texture */
	TSharedPtr< FUICommandInfo > SurfAlignPlanarAuto;

	/** Align surface texture like its a wall */
	TSharedPtr< FUICommandInfo > SurfAlignPlanarWall;

	/** Align surface texture like its a floor */
	TSharedPtr< FUICommandInfo > SurfAlignPlanarFloor;

	/** Align surface texture using box */
	TSharedPtr< FUICommandInfo > SurfAlignBox;

	/** Best fit surface texture alignment */
	TSharedPtr< FUICommandInfo > SurfAlignFit;

	/** Apply the currently selected material to the currently selected surfaces */
	TSharedPtr< FUICommandInfo > ApplyMaterialToSurface;

	/**
	 * Static mesh commands                   
	 */

	/** Create a blocking volume from the meshes bounding box */
	TSharedPtr< FUICommandInfo > CreateBoundingBoxVolume;

	/** Create a blocking volume from the meshes using a heavy convex shape */
	TSharedPtr< FUICommandInfo > CreateHeavyConvexVolume;

	/** Create a blocking volume from the meshes using a normal convex shape */
	TSharedPtr< FUICommandInfo > CreateNormalConvexVolume;

	/** Create a blocking volume from the meshes using a light convex shape */
	TSharedPtr< FUICommandInfo > CreateLightConvexVolume;

	/** Create a blocking volume from the meshes using a rough convex shape */
	TSharedPtr< FUICommandInfo > CreateRoughConvexVolume;

	/** Set the actors collision to block all */
	TSharedPtr< FUICommandInfo > SetCollisionBlockAll;

	/** Set the actors collision to block only weapons */
	TSharedPtr< FUICommandInfo > SetCollisionBlockWeapons;

	/** Set the actors collision to block nothing */
	TSharedPtr< FUICommandInfo > SetCollisionBlockNone;

	/**
	 * Simulation commands
	 */

	/** Pushes properties of the selected actor back to its EditorWorld counterpart */
	TSharedPtr< FUICommandInfo > KeepSimulationChanges;


	/**
	 * Level commands
	 */

	/** Makes the actor level the current level */
	TSharedPtr< FUICommandInfo > MakeActorLevelCurrent;

	/** Move all the selected actors to the current level */
	TSharedPtr< FUICommandInfo > MoveSelectedToCurrentLevel;

	/** Finds the level of the selected actors in the content browser */
	TSharedPtr< FUICommandInfo > FindActorLevelInContentBrowser;

	/** Finds the levels of the selected actors in the level browser */
	TSharedPtr< FUICommandInfo > FindLevelsInLevelBrowser;

	/** Add levels of the selected actors to the level browser selection */
	TSharedPtr< FUICommandInfo > AddLevelsToSelection;

	/** Remove levels of the selected actors from the level browser selection */
	TSharedPtr< FUICommandInfo > RemoveLevelsFromSelection;

	/**
	 * Level Script Commands
	 */
	TSharedPtr< FUICommandInfo > FindActorInLevelScript;

	/**
	 * Level Menu
	 */

	TSharedPtr< FUICommandInfo > WorldProperties;
	TSharedPtr< FUICommandInfo > OpenPlaceActors;
	TSharedPtr< FUICommandInfo > OpenContentBrowser;
	TSharedPtr< FUICommandInfo > ToggleVR;
	TSharedPtr< FUICommandInfo > ImportContent;

	/**
	 * Blueprints commands
	 */
	TSharedPtr< FUICommandInfo > OpenLevelBlueprint;
	TSharedPtr< FUICommandInfo > CheckOutProjectSettingsConfig;
	TSharedPtr< FUICommandInfo > CreateBlankBlueprintClass;
	TSharedPtr< FUICommandInfo > ConvertSelectionToBlueprint;

	/** Editor mode commands */
	TArray< TSharedPtr< FUICommandInfo > > EditorModeCommands;

	/**
	 * View commands
	 */
	TSharedPtr< FUICommandInfo > ShowTransformWidget;
	TSharedPtr< FUICommandInfo > AllowTranslucentSelection;
	TSharedPtr< FUICommandInfo > AllowGroupSelection;
	TSharedPtr< FUICommandInfo > ShowSelectionSubcomponents;

	TSharedPtr< FUICommandInfo > StrictBoxSelect;
	TSharedPtr< FUICommandInfo > TransparentBoxSelect;
	TSharedPtr< FUICommandInfo > DrawBrushMarkerPolys;
	TSharedPtr< FUICommandInfo > OnlyLoadVisibleInPIE;

	TSharedPtr< FUICommandInfo > ToggleSocketSnapping; 
	TSharedPtr< FUICommandInfo > ToggleParticleSystemLOD;
	TSharedPtr< FUICommandInfo > ToggleParticleSystemHelpers;
	TSharedPtr< FUICommandInfo > ToggleFreezeParticleSimulation;
	TSharedPtr< FUICommandInfo > ToggleLODViewLocking;
	TSharedPtr< FUICommandInfo > LevelStreamingVolumePrevis;

	TSharedPtr< FUICommandInfo > EnableActorSnap;
	TSharedPtr< FUICommandInfo > EnableVertexSnap;

	TSharedPtr< FUICommandInfo > ToggleHideViewportUI;

	TSharedPtr< FUICommandInfo > MaterialQualityLevel_Low;
	TSharedPtr< FUICommandInfo > MaterialQualityLevel_Medium;
	TSharedPtr< FUICommandInfo > MaterialQualityLevel_High;
	TSharedPtr< FUICommandInfo > MaterialQualityLevel_Epic;

	TSharedPtr< FUICommandInfo > TogglePreviewPlatform;

	TArray<TSharedPtr<FUICommandInfo>> PreviewPlatformOverrides;

	struct PreviewPlatformCommand
	{
		PreviewPlatformCommand()
			: bIsGeneratingJsonCommand(false)
			, SectionName(NAME_None)
		{
		}

		bool bIsGeneratingJsonCommand;
		FName SectionName;
		TSharedPtr<FUICommandInfo> CommandInfo;
		FString FilePath;
		TSet<FName> PreviewShaderPlatformNames;
	};
	
	TSharedPtr< FUICommandInfo > DisablePlatformPreview;
	TMap<FName, TArray<PreviewPlatformCommand>> PlatformToPreviewPlatformOverrides;
	TMap<FName, TArray<PreviewPlatformCommand>> PlatformToPreviewJsonPlatformOverrides;

	/**
	 * Camera Preferences
	 */
	TSharedPtr<FUICommandInfo> OrbitCameraAroundSelection;
	TSharedPtr<FUICommandInfo> LinkOrthographicViewports;
	TSharedPtr<FUICommandInfo> OrthoZoomToCursor;

	///**
	// * Mode Commands                   
	// */
	//TSharedPtr< FUICommandInfo > BspMode;
	//TSharedPtr< FUICommandInfo > MeshPaintMode;
	//TSharedPtr< FUICommandInfo > LandscapeMode;
	//TSharedPtr< FUICommandInfo > FoliageMode;

	/**
	 * Misc Commands
	 */
	TSharedPtr< FUICommandInfo > ShowSelectedDetails;
	TSharedPtr< FUICommandInfo > RecompileShaders;
	TSharedPtr< FUICommandInfo > ProfileGPU;
	TSharedPtr< FUICommandInfo > DumpGPU;

	TSharedPtr< FUICommandInfo > ResetAllParticleSystems;
	TSharedPtr< FUICommandInfo > ResetSelectedParticleSystem;
	TSharedPtr< FUICommandInfo > SelectActorsInLayers;

        // Open merge actor command
	TSharedPtr< FUICommandInfo > OpenMergeActor;

	TSharedPtr< FUICommandInfo > FixupGroupActor;

	TSharedPtr<FUICommandInfo> AllowArcballRotation;
	TSharedPtr<FUICommandInfo> AllowScreenspaceRotation;
	TSharedPtr<FUICommandInfo> EnableViewportHoverFeedback;
	/**
	 * Controls
	 */

	// Mouse Controls
	TSharedPtr<FUICommandInfo> InvertMiddleMousePan;
	TSharedPtr<FUICommandInfo> InvertOrbitYAxis;
	TSharedPtr<FUICommandInfo> InvertRightMouseDollyYAxis;
};

/**
 * Implementation of various level editor action callback functions
 */
class FLevelEditorActionCallbacks
{
public:

	/**
	 * The default can execute action for all commands unless they override it
	 * By default commands cannot be executed if the application is in K2 debug mode.
	 */
	static UE_API bool DefaultCanExecuteAction();

	/** Opens the global documentation homepage */
	static UE_API void BrowseDocumentation();

	/** Opens the viewport controls page*/
	static UE_API void BrowseViewportControls();

	/** Creates a new level */
	static UE_API void NewLevel();
	static UE_API void NewLevel(bool& bOutLevelCreated);
	DECLARE_DELEGATE_OneParam(FNewLevelOverride, bool& /*bOutLevelCreated*/);
	static UE_API FNewLevelOverride NewLevelOverride;
	static UE_API bool NewLevel_CanExecute();

	/** Opens an existing level */
	static UE_API void OpenLevel();
	static UE_API bool OpenLevel_CanExecute();

	/** Toggles VR mode */
	static UE_API void ToggleVR();
	static UE_API bool ToggleVR_CanExecute();
	static UE_API bool ToggleVR_IsButtonActive();
	static UE_API bool ToggleVR_IsChecked();

	/** Opens delta transform */
	static UE_API void DeltaTransform();

	/**
	 * Opens a recent file
	 *
	 * @param	RecentFileIndex		Index into our MRU list of recent files that can be opened
	 */
	static UE_API void OpenRecentFile( int32 RecentFileIndex );

	/** Clear the list of recent files. */
	static UE_API void ClearRecentFiles();

	/**
	 * Opens a favorite file
	 *
	 * @param	FavoriteFileIndex		Index into our list of favorite files that can be opened
	 */
	static UE_API void OpenFavoriteFile( int32 FavoriteFileIndex );

	static UE_API void ToggleFavorite();

	/**
	 * Remove a favorite file from the favorites list
	 *
	 * @param	FavoriteFileIndex		Index into our list of favorite files to be removed
	 */
	static UE_API void RemoveFavorite( int32 FavoriteFileIndex );

	static UE_API bool ToggleFavorite_CanExecute();
	static UE_API bool ToggleFavorite_IsChecked();

	/** Determine whether the level can be saved at this moment */
	static UE_API bool CanSaveWorld();
	static UE_API bool CanSaveUnpartitionedWorld();

	/** Save the current level as... */
	static UE_API bool CanSaveCurrentAs();
	static UE_API void SaveCurrentAs();

	/** Saves the current map */
	static UE_API void Save();

	/** Saves all unsaved maps (but not packages) */
	static UE_API void SaveAllLevels();

	/** Browses to the current map */
	static UE_API void Browse();
	static UE_API bool CanBrowse();


	/**
	 * Called when import scene is selected
	 */
	static UE_API void ImportScene_Clicked();
	
	/**
	* Called When Preview Json is selected in the Platforms Preview Sub Menu
	*/
	static UE_API void PreviewJson_Clicked(FName PlatformName, FString JsonFile);

	/**
	* Is Preview Json visible in the Platforms Preview Sub Menu
	*/
	static UE_API bool IsPreviewJsonVisible(FName PlatformName);

	/**
	* Called When Generate Preview Json is selected in the Platforms Preview Sub Menu
	*/
	static UE_API void GeneratePreviewJson_Clicked(FString PlatformName);
	
	/**
	* Is Generate Preview Json visible in the Platforms Preview Sub Menu
	*/
	static UE_API bool IsGeneratePreviewJsonVisible(FName PlatformName);

	/**
	 * Called when export all is selected
	 */
	static UE_API void ExportAll_Clicked();


	/**
	 * Called when export selected is clicked
	 */
	static UE_API void ExportSelected_Clicked();


	/**
	 * @return	True if the export selected option is available to execute
	 */
	static UE_API bool ExportSelected_CanExecute();


	static UE_API void ConfigureLightingBuildOptions( const FLightingBuildOptions& Options );

	static UE_API bool CanBuildLighting();
	static UE_API bool CanBuildReflectionCaptures();

	/**
	 * Build callbacks
	 */
	static UE_API void Build_Execute();
	static UE_API bool Build_CanExecute();
	static UE_API void BuildAndSubmitToSourceControl_Execute();
	static UE_API void BuildLightingOnly_Execute();
	static UE_API bool BuildLighting_CanExecute();
	static UE_API void BuildReflectionCapturesOnly_Execute();
	static UE_API bool BuildReflectionCapturesOnly_CanExecute();
	static UE_API void BuildLightingOnly_VisibilityOnly_Execute();
	static UE_API bool LightingBuildOptions_UseErrorColoring_IsChecked();
	static UE_API void LightingBuildOptions_UseErrorColoring_Toggled();
	static UE_API bool LightingBuildOptions_ShowLightingStats_IsChecked();
	static UE_API void LightingBuildOptions_ShowLightingStats_Toggled();
	static UE_API void BuildGeometryOnly_Execute();
	static UE_API void BuildGeometryOnly_OnlyCurrentLevel_Execute();
	static UE_API void BuildPathsOnly_Execute();
	static UE_API bool IsWorldPartitionEnabled();
	static UE_API bool IsWorldPartitionStreamingEnabled();
	static UE_API void BuildHLODs_Execute();
	static UE_API void BuildHLODs_SelectedActors_Execute();
	static UE_API void BuildMinimap_Execute();
	static UE_API void BuildLandscapeSplineMeshes_Execute();
	static UE_API void BuildTextureStreamingOnly_Execute();
	static UE_API void BuildVirtualTextureOnly_Execute();
	static UE_API void BuildAllLandscape_Execute();
	static UE_API bool BuildExternalType_CanExecute( int32 Index );
	static UE_API void BuildExternalType_Execute( int32 Index );
	static UE_API void SetLightingQuality( ELightingBuildQuality NewQuality );
	static UE_API bool IsLightingQualityChecked( ELightingBuildQuality TestQuality );
	static UE_API float GetLightingDensityIdeal();
	static UE_API void SetLightingDensityIdeal( float Value );
	static UE_API float GetLightingDensityMaximum();
	static UE_API void SetLightingDensityMaximum( float Value );
	static UE_API float GetLightingDensityColorScale();
	static UE_API void SetLightingDensityColorScale( float Value );
	static UE_API float GetLightingDensityGrayscaleScale();
	static UE_API void SetLightingDensityGrayscaleScale( float Value );
	static UE_API void SetLightingDensityRenderGrayscale();
	static UE_API bool IsLightingDensityRenderGrayscaleChecked();
	static UE_API void SetLightingResolutionStaticMeshes( ECheckBoxState NewCheckedState );
	static UE_API ECheckBoxState IsLightingResolutionStaticMeshesChecked();
	static UE_API void SetLightingResolutionBSPSurfaces( ECheckBoxState NewCheckedState );
	static UE_API ECheckBoxState IsLightingResolutionBSPSurfacesChecked();
	static UE_API void SetLightingResolutionLevel( FLightmapResRatioAdjustSettings::AdjustLevels NewLevel );
	static UE_API bool IsLightingResolutionLevelChecked( FLightmapResRatioAdjustSettings::AdjustLevels TestLevel );
	static UE_API void SetLightingResolutionSelectedObjectsOnly();
	static UE_API bool IsLightingResolutionSelectedObjectsOnlyChecked();
	static UE_API float GetLightingResolutionMinSMs();
	static UE_API void SetLightingResolutionMinSMs( float Value );
	static UE_API float GetLightingResolutionMaxSMs();
	static UE_API void SetLightingResolutionMaxSMs( float Value );
	static UE_API float GetLightingResolutionMinBSPs();
	static UE_API void SetLightingResolutionMinBSPs( float Value );
	static UE_API float GetLightingResolutionMaxBSPs();
	static UE_API void SetLightingResolutionMaxBSPs( float Value );
	static UE_API int32 GetLightingResolutionRatio();
	static UE_API void SetLightingResolutionRatio( int32 Value );
	static UE_API void SetLightingResolutionRatioCommit( int32 Value, ETextCommit::Type CommitInfo);
	static UE_API void ShowLightingStaticMeshInfo();
	static UE_API void AttachToActor(AActor* ParentActorPtr );
	static UE_API void AttachToSocketSelection(FName SocketName, AActor* ParentActorPtr);
	static UE_API void SetMaterialQualityLevel( EMaterialQualityLevel::Type NewQualityLevel );
	static UE_API bool IsMaterialQualityLevelChecked( EMaterialQualityLevel::Type TestQualityLevel );
	static UE_API void TogglePreviewPlatform();
	static UE_API bool IsPreviewPlatformEnabled();
	static UE_API bool IsPreviewPlatformActive();
	static UE_API bool IsPreviewModeButtonVisible();

	UE_DEPRECATED(5.8, "Use TogglePreviewPlatform instead")
	static void ToggleFeatureLevelPreview() { TogglePreviewPlatform(); }
	UE_DEPRECATED(5.8, "Use IsPreviewPlatformEnabled instead")
	static bool IsFeatureLevelPreviewEnabled() { return IsPreviewPlatformEnabled(); }
	UE_DEPRECATED(5.8, "Use IsPreviewPlatformActive instead")
	static bool IsFeatureLevelPreviewActive() { return IsPreviewPlatformActive(); }
	static UE_API void SetPreviewPlatform(FPreviewPlatformInfo NewPreviewPlatform);
	static UE_API bool CanExecutePreviewPlatform(FPreviewPlatformInfo NewPreviewPlatform);
	static UE_API bool IsPreviewPlatformChecked(FPreviewPlatformInfo NewPreviewPlatform);
	static UE_API void GeometryCollection_SelectAllGeometry();
	static UE_API void GeometryCollection_SelectNone();
	static UE_API void GeometryCollection_SelectInverseGeometry();
	static UE_API bool GeometryCollection_IsChecked();

	static UE_API void ToggleAllowArcballRotation();
	static UE_API bool IsAllowArcballRotationChecked();

	static UE_API void ToggleAllowScreenspaceRotation();
	static UE_API bool IsAllowScreenspaceRotationChecked();

	static UE_API void ToggleEnableViewportHoverFeedback();
	static UE_API bool IsEnableViewportHoverFeedbackChecked();

	static UE_API void TogglePreviewSelectedCameras(const TWeakPtr<SLevelViewport>& InLevelViewportWeak = {});
	static UE_API bool IsPreviewSelectedCamerasChecked();

	static UE_API void ToggleOrbitCameraAroundSelection();
	static UE_API bool IsOrbitCameraAroundSelectionChecked();

	static UE_API void ToggleLinkOrthographicViewports();
	static UE_API bool IsLinkOrthographicViewportsChecked();

	static UE_API void ToggleOrthoZoomToCursor();
	static UE_API bool IsOrthoZoomToCursorChecked();

	static UE_API void ToggleInvertMiddleMousePan();
	static UE_API bool IsInvertMiddleMousePanChecked();
	
	static UE_API void ToggleInvertOrbitYAxis();
	static UE_API bool IsInvertOrbitYAxisChecked();
	
	static UE_API void ToggleInvertRightMouseDollyYAxis();
	static UE_API bool IsInvertRightMouseDollyYAxisChecked();

	/**
	 * Called when the Scene Stats button is clicked.  Invokes the Primitive Stats dialog.
	 */
	static UE_API void ShowSceneStats();

	/**
	 * Called when the Texture Stats button is clicked.  Invokes the Texture Stats dialog.
	 */
	static UE_API void ShowTextureStats();

	/**
	 * Called when the Map Check button is clicked.  Invokes the Map Check dialog.
	 */
	static UE_API void MapCheck_Execute();

	/** @return True if actions that should only be visible when source code is thought to be available */
	static UE_API bool CanShowSourceCodeActions();

	/**
	 * Called when the recompile buttons are clicked.
	 */
	static UE_API void RecompileGameCode_Clicked();
	static UE_API bool Recompile_CanExecute();

#if WITH_LIVE_CODING
	/**
	 * Enables live coding mode
	 */
	static UE_API void LiveCoding_ToggleEnabled();

	/**
	 * Determines if live coding is enabled
	 */
	static UE_API bool LiveCoding_IsEnabled();

	/**
	 * Starts live coding (in manual mode)
	 */
	static UE_API void LiveCoding_StartSession_Clicked();

	/**
	 * Determines whether we can manually start live coding for the current session
	 */
	static UE_API bool LiveCoding_CanStartSession();

	/**
	 * Shows the console
	 */
	static UE_API void LiveCoding_ShowConsole_Clicked();

	/**
	 * Determines whether the console can be shown
	 */
	static UE_API bool LiveCoding_CanShowConsole();

	/**
	 * Shows the settings panel
	 */
	static UE_API void LiveCoding_Settings_Clicked();
#endif

	/**
	 * Called when requesting connection to source control
	 */
	static UE_API void ConnectToSourceControl_Clicked();

	/**
	 * Called when Check Out Modified Files is clicked
	 */
	static UE_API void CheckOutModifiedFiles_Clicked();
	static UE_API bool CheckOutModifiedFiles_CanExecute();

	/**
	 * Called when Submit to Source Control is clicked
	 */
	static UE_API void SubmitToSourceControl_Clicked();
	static UE_API bool SubmitToSourceControl_CanExecute();

	/**
	 * Called when the FindInContentBrowser command is executed
	 */
	static UE_API void FindInContentBrowser_Clicked();
	static UE_API bool FindInContentBrowser_CanExecute();

	/** Called to when "Edit Asset" is clicked */
	static UE_API void EditAsset_Clicked( const EToolkitMode::Type ToolkitMode, TWeakPtr< class SLevelEditor > LevelEditor, bool bAskMultiple );
	static UE_API bool EditAsset_CanExecute();

	/** Called to when "Open Selection in Property Matrix" is clicked */
	static UE_API void OpenSelectionInPropertyMatrix_Clicked();
	static UE_API bool OpenSelectionInPropertyMatrix_IsVisible();


	/** Called when 'detach' is clicked */
	static UE_API void DetachActor_Clicked();
	static UE_API bool DetachActor_CanExecute();

	/** Called when attach selected actors is pressed */
	static UE_API void AttachSelectedActors();

	/** Called when the actor picker needs to be used to select a new parent actor */
	static UE_API void AttachActorIteractive();

	/** @return true if the selected actor can be attached to the given parent actor */
	static UE_API bool IsAttachableActor( const AActor* const ParentActor );

	/** Called when create new outliner folder is clicked */
	static UE_API void CreateNewOutlinerFolder_Clicked();

	/** Called when the go here command is clicked 
	 * 
	 * @param Point	- Specified point to go to.  If null, a point will be calculated from current mouse position
	 */
	static UE_API void GoHere_Clicked( const FVector* Point );

	/** Called when selected actor can be used to start a play session */
	static UE_API void PlayFromHere_Clicked(bool bFloatingWindow);
	static UE_API bool PlayFromHere_IsVisible();
	
	/** If an actor is selected, plays from its position (AActor::bCanPlayFromHere must be true) otherwise it will play from last clicked position */
	static UE_API void PlayFromHere();

	/** Called when 'Go to Code for Actor' is clicked */
	static UE_API void GoToCodeForActor_Clicked();
	static UE_API bool GoToCodeForActor_CanExecute();
	static UE_API bool GoToCodeForActor_IsVisible();

	/** Called when 'Go to Documentation for Actor' is clicked */
	static UE_API void GoToDocsForActor_Clicked();

/**
	 * Called when the LockActorMovement command is executed
	 */
	static UE_API void LockActorMovement_Clicked();

		
	/**
	 * @return true if the lock actor menu option should appear checked
	 */
	static UE_API bool LockActorMovement_IsChecked();

	/**
	 * Called when the AddActor command is executed
	 *
	 * @param ActorFactory		The actor factory to use when adding the actor
	 * @param bUsePlacement		Whether to use the placement editor. If not, the actor will be placed at the last click.
	 * @param ActorLocation		[opt] If NULL, positions the actor at the mouse location, otherwise the location specified. Default is true.
	 */
	static UE_API void AddActor_Clicked( UActorFactory* ActorFactory, FAssetData AssetData);
	static UE_API AActor* AddActor( UActorFactory* ActorFactory, const FAssetData& AssetData, const FTransform* ActorLocation );

	/**
	 * Called when the AddActor command is executed and a class is selected in the actor browser
	 *
	 * @param ActorClass		The class of the actor to add
	 */
	static UE_API void AddActorFromClass_Clicked( UClass* ActorClass );
	static UE_API AActor* AddActorFromClass( UClass* ActorClass );

	/**
	 * Replaces currently selected actors with an actor from the given actor factory
	 *
	 * @param ActorFactory	The actor factory to use in replacement
	 */
	static UE_API void ReplaceActors_Clicked( UActorFactory* ActorFactory, FAssetData AssetData );
	static UE_API AActor* ReplaceActors( UActorFactory* ActorFactory, const FAssetData& AssetData, bool bCopySourceProperties = true );

	/**
	 * Called when the ReplaceActor command is executed and a class is selected in the actor browser
	 *
	 * @param ActorClass	The class of the actor to replace
	 */
	static UE_API void ReplaceActorsFromClass_Clicked( UClass* ActorClass );

	/**
	 * Called to check to see if the Edit commands can be executed
	 *
	 * @return true, if the operation can be performed
	 */
	static UE_API bool Duplicate_CanExecute();
	static UE_API bool Delete_CanExecute();
	static UE_API void Rename_Execute();
	static UE_API bool Rename_CanExecute();
	static UE_API bool Cut_CanExecute();
	static UE_API bool Copy_CanExecute();
	static UE_API bool Paste_CanExecute();
	static UE_API bool PasteHere_CanExecute();

	/**
	 * Called when many of the menu items in the level editor context menu are clicked
	 *
	 * @param Command	The command to execute
	 */
	static UE_API void ExecuteExecCommand( FString Command );
	
	/**
	 * Called when selecting all actors of the same class that is selected
	 *
	 * @param bArchetype	true to also check that the archetype is the same
	 */
	static UE_API void OnSelectAllActorsOfClass( bool bArchetype );

	/** Called to see if all selected actors are the same class */
	static UE_API bool CanSelectAllActorsOfClass();

	/** Called when selecting the actor that owns the currently selected component(s) */
	static UE_API void OnSelectComponentOwnerActor();

	/** Called to see if any components are selected */
	static UE_API bool CanSelectComponentOwnerActor();

	/** Called to see if selected actors can be hidden */
	static UE_API bool CanHideSelectedActors();
	
	/** Called to see if selected actors can be hidden */
	static UE_API bool CanShowSelectedActors();
	
	/** Called to see if the visibility of seleced actors can be toggled */
	static UE_API bool CanToggleSelectedActorsVisibility();

	/**
	 * Called to select all lights
	 */
	static UE_API void OnSelectAllLights();

	/** Selects stationary lights that are exceeding the overlap limit. */
	static UE_API void OnSelectStationaryLightsExceedingOverlap();

	/**
	* Called when selecting an Actor's (if available) owning HLOD cluster
	*/
	static UE_API void OnSelectOwningHLODCluster();
	
	/**
	 * Called to change bsp surface alignment
	 *
	 * @param AlignmentMode	The new alignment mode
	 */
	static UE_API void OnSurfaceAlignment( ETexAlign AligmentMode );

	/**
	 * Called to apply a material to selected surfaces
	 */
	static UE_API void OnApplyMaterialToSurface();

	/**
	 * Checks to see if the selected actors can be grouped
	 *	@return true if it can execute.
	 */
	static UE_API bool GroupActors_CanExecute();
	
	/**
	 * Called when the RegroupActor command is executed
	 */
	static UE_API void RegroupActor_Clicked();
		
	/**
	 * Called when the UngroupActor command is executed
	 */
	static UE_API void UngroupActor_Clicked();
		
	/**
	 * Called when the LockGroup command is executed
	 */
	static UE_API void LockGroup_Clicked();
	
	/**
	 * Called when the UnlockGroup command is executed
	 */
	static UE_API void UnlockGroup_Clicked();
	
	/**
	 * Called when the AddActorsToGroup command is executed
	 */
	static UE_API void AddActorsToGroup_Clicked();
	
	/**
	 * Called when the RemoveActorsFromGroup command is executed
	 */
	static UE_API void RemoveActorsFromGroup_Clicked();

	/**
	 * Called when the location grid snap is toggled off and on
	 */
	static UE_API void LocationGridSnap_Clicked();

	/**
	 * @return Returns whether or not location grid snap is enabled
	 */
	static UE_API bool LocationGridSnap_IsChecked();

	/**
	 * Called when the rotation grid snap is toggled off and on
	 */
	static UE_API void RotationGridSnap_Clicked();

	/**
	 * @return Returns whether or not rotation grid snap is enabled
	 */
	static UE_API bool RotationGridSnap_IsChecked();

	/**
	 * Called when the scale grid snap is toggled off and on
	 */
	static UE_API void ScaleGridSnap_Clicked();

	/**
	 * @return Returns whether or not scale grid snap is enabled
	 */
	static UE_API bool ScaleGridSnap_IsChecked();


	/** Called when "Keep Simulation Changes" is clicked in the viewport right click menu */
	static UE_API void OnKeepSimulationChanges();

	/** @return Returns true if 'Keep Simulation Changes' can be used right now */
	static UE_API bool CanExecuteKeepSimulationChanges();
		
		
	/**
	 * Makes the currently selected actors level the current level
	 * If multiple actors are selected they must all be in the same level
	 */
	static UE_API void OnMakeSelectedActorLevelCurrent();

	/**
	 * Moves the currently selected actors to the current level                   
	 */
	static UE_API void OnMoveSelectedToCurrentLevel();

	/** Finds the currently selected actor(s) level in the content browser */
	static UE_API void OnFindActorLevelInContentBrowser();

	/** @return Returns true if all selected actors are from the same level and hence can browse to it in the content browser */
	static UE_API bool CanExecuteFindActorLevelInContentBrowser();

	/**
	 * Selects the currently selected actor(s) levels in the level browser
	 * Deselecting everything else first
	 */
	static UE_API void OnFindLevelsInLevelBrowser();

	/**
	 * Selects the currently selected actor(s) levels in the level browser
	 */
	static UE_API void OnSelectLevelInLevelBrowser();

	/**
	 * Deselects the currently selected actor(s) levels in the level browser
	 */
	static UE_API void OnDeselectLevelInLevelBrowser();

	/**
	 * Finds references to the currently selected actor(s) in level scripts
	 */
	static UE_API void OnFindActorInLevelScript();

	/** Select the world info actor and show the properties */
	static UE_API void OnShowWorldProperties( TWeakPtr< SLevelEditor > LevelEditor );

	/** Focuses the outliner on the selected actors */
	static UE_API void OnFocusOutlinerToSelection(TWeakPtr<SLevelEditor> LevelEditor);

	/** Focuses the outliner on the context folder if it's currently set */
	static UE_API void OnFocusOutlinerToContextFolder(TWeakPtr<SLevelEditor> LevelEditor);

	/** Open the Place Actors Panel */
	static UE_API void OpenPlaceActors();

	/** Open the Content Browser */
	static UE_API void OpenContentBrowser();

	/** Import content into a chosen location*/
	static UE_API void ImportContent();

	/** Checks out the Project Settings config */
	static UE_API void CheckOutProjectSettingsConfig();

	/** Open the level's blueprint in Kismet2 */
	static UE_API void OpenLevelBlueprint( TWeakPtr< SLevelEditor > LevelEditor );

	/** Returns TRUE if the user can edit the game mode Blueprint, this requires the DefaultEngine config file to be writable */
	static UE_API bool CanSelectGameModeBlueprint();

	/** Helps the user create a Blueprint class */
	static UE_API void CreateBlankBlueprintClass();

	/** Can the selected actors be converted to a blueprint class in any of the supported ways? */
	static UE_API bool CanConvertSelectedActorsIntoBlueprintClass();

	/** Bring up the convert actors to blueprint UI */
	static UE_API void ConvertSelectedActorsIntoBlueprintClass();

	/** Shows only selected actors, hiding any unselected actors and unhiding any selected hidden actors. */
	static UE_API void OnShowOnlySelectedActors();

	/**
	 * View callbacks
	 */ 
	static UE_API void OnToggleTransformWidgetVisibility();
	static UE_API bool OnGetTransformWidgetVisibility();
	static UE_API void OnToggleShowSelectionSubcomponents();
	static UE_API bool OnGetShowSelectionSubcomponents();
	static UE_API void OnAllowTranslucentSelection();
	static UE_API bool OnIsAllowTranslucentSelectionEnabled();	
	static UE_API void OnAllowGroupSelection();
	static UE_API bool OnIsAllowGroupSelectionEnabled(); 
	static UE_API void OnToggleStrictBoxSelect();
	static UE_API bool OnIsStrictBoxSelectEnabled(); 
	static UE_API void OnToggleTransparentBoxSelect();
	static UE_API bool OnIsTransparentBoxSelectEnabled();
	static UE_API void OnDrawBrushMarkerPolys();
	static UE_API bool OnIsDrawBrushMarkerPolysEnabled();
	static UE_API void OnToggleOnlyLoadVisibleInPIE();
	static UE_API bool OnIsOnlyLoadVisibleInPIEEnabled(); 
	static UE_API void OnToggleSocketSnapping();
	static UE_API bool OnIsSocketSnappingEnabled(); 
	static UE_API void OnToggleParticleSystemLOD();
	static UE_API bool OnIsParticleSystemLODEnabled(); 
	static UE_API void OnToggleFreezeParticleSimulation();
	static UE_API bool OnIsParticleSimulationFrozen();
	static UE_API void OnToggleParticleSystemHelpers();
	static UE_API bool OnIsParticleSystemHelpersEnabled();
	static UE_API void OnToggleLODViewLocking();
	static UE_API bool OnIsLODViewLockingEnabled(); 
	static UE_API void OnToggleLevelStreamingVolumePrevis();
	static UE_API bool OnIsLevelStreamingVolumePrevisEnabled(); 
	
	static UE_API FText GetAudioVolumeToolTip();
	static UE_API float GetAudioVolume();
	static UE_API void OnAudioVolumeChanged(float Volume);
	static UE_API bool GetAudioMuted();
	static UE_API void OnAudioMutedChanged(bool bMuted); 

	static UE_API void OnEnableActorSnap();
	static UE_API bool OnIsActorSnapEnabled();
	static UE_API FText GetActorSnapTooltip();
	static UE_API float GetActorSnapSetting();
	static UE_API void SetActorSnapSetting(float Distance);
	static UE_API void OnEnableVertexSnap();
	static UE_API bool OnIsVertexSnapEnabled();

	static UE_API void OnToggleShowViewportToolbar();
	static UE_API bool IsViewportToolbarVisible();

	static UE_API void OnToggleShowViewportUI();
	static UE_API bool IsViewportUIVisible();

	static UE_API bool IsEditorModeActive( FEditorModeID EditorMode );

	static UE_API void MakeBuilderBrush( UClass* BrushBuilderClass );

	static UE_API void OnAddVolume( UClass* VolumeClass );

	static UE_API void SelectActorsInLayers();

	static UE_API void SetWidgetMode( UE::Widget::EWidgetMode WidgetMode );
	static UE_API bool IsWidgetModeActive( UE::Widget::EWidgetMode WidgetMode );
	static UE_API bool CanSetWidgetMode( UE::Widget::EWidgetMode WidgetMode );
	static UE_API bool IsTranslateRotateModeVisible();
	static UE_API void SetCoordinateSystem( ECoordSystem CoordSystem );
	static UE_API bool IsCoordinateSystemActive( ECoordSystem CoordSystem );

	/**
	 * Return a world
	 */
	static UE_API class UWorld* GetWorld();
public:
	/** 
	 * Moves the selected elements to the grid.
	 */
	static UE_API void MoveElementsToGrid_Clicked( bool InAlign, bool InPerElement );

	/**
	* Snaps a selected actor to the camera view.
	*/
	static UE_API void SnapObjectToView_Clicked();

	/**
	* Copy the file path where the actor is saved.
	*/
	static UE_API void CopyActorFilePathtoClipboard_Clicked();

	/**
	* Opens the reference viewer for the selected actor.
	*/
	static UE_API void ViewActorReferences_Clicked();

	/**
	 * Checks whether OpenActorInReferenceViewer can be executed on the selected actor.
	 * @return 	True if OpenActorInReferenceViewer can be executed on the selected actor.
	 */
	static UE_API bool ViewActorReferences_CanExecute();

	/**
	* Save the actor.
	*/
	static UE_API void SaveActor_Clicked();

	/**
	 * Checks whether SaveActor can be executed on the selected actors.
	 * @return 	True if SaveActor can be executed on the selected actors.
	 */
	static UE_API bool SaveActor_CanExecute();

	/**
	* Shows the history of the file containing the actor.
	*/
	static UE_API void ShowActorHistory_Clicked();

	/**
	 * Checks whether ShowActorHistory can be executed on the selected actors.
	 * @return 	True if ShowActorHistory can be executed on the selected actors.
	 */
	static UE_API bool ShowActorHistory_CanExecute();

	/** 
	 * Moves the selected elements to the last selected element.
	 */
	static UE_API void MoveElementsToElement_Clicked( bool InAlign );

	/** 
	 * Snaps an actor to the currently selected 2D snap layer
	 */
	static UE_API void SnapTo2DLayer_Clicked();

	/**
	 * Checks to see if at least a single actor is selected and the 2D editor mode is enabled
	 *	@return true if it can execute.
	 */
	static UE_API bool CanSnapTo2DLayer();

	/** 
	 * Snaps an actor to the currently selected 2D snap layer
	 */
	static UE_API void MoveSelectionToDifferent2DLayer_Clicked(bool bGoingUp, bool bForceToTopOrBottom);

	/**
	 * Checks to see if at least a single actor is selected and the 2D editor mode is enabled and there is a layer above/below the current setting
	 *	@return true if it can execute.
	 */
	static UE_API bool CanMoveSelectionToDifferent2DLayer(bool bGoingUp);

	/**
	 * Changes the active 2D snap layer to one a delta above or below the current layer
	 */
	static UE_API void Select2DLayerDeltaAway_Clicked(int32 Delta);

	/** 
	 * Snaps the selected elements to the floor.  Optionally will align with the trace normal.
	 * @param InAlign			Whether or not to rotate the actor to align with the trace normal.
	 * @param InUseLineTrace	Whether or not to only trace with a line through the world.
	 * @param InUseBounds		Whether or not to base the line trace off of the bounds.
	 * @param InUsePivot		Whether or not to use the pivot position.
	 */
	static UE_API void SnapToFloor_Clicked( bool InAlign, bool InUseLineTrace, bool InUseBounds, bool InUsePivot );

	/**
	 * Snaps the selected elements to another element.  Optionally will align with the trace normal.
	 * @param InAlign			Whether or not to rotate the actor to align with the trace normal.
	 * @param InUseLineTrace	Whether or not to only trace with a line through the world.
	 * @param InUseBounds		Whether or not to base the line trace off of the bounds.
	 * @param InUsePivot		Whether or not to use the pivot position.
	 */
	static UE_API void SnapElementsToElement_Clicked( bool InAlign, bool InUseLineTrace, bool InUseBounds, bool InUsePivot );

	/**
	 * Aligns brush vertices to the nearest grid point.
	 */
	static UE_API void AlignBrushVerticesToGrid_Execute();

	/**
	 * Checks to see if at least one actor is selected
	 *	@return true if it can execute.
	 */
	static UE_API bool ActorSelected_CanExecute();

	enum EActorTypeFlags : uint8
	{
		IncludePawns			= 1 << 0,
		IncludeStaticMeshes		= 1 << 1,
		IncludeSkeletalMeshes	= 1 << 2,
		IncludeEmitters			= 1 << 3,
	};

	/**
	 * Checks to see if at least one actor (of the given types) is selected
	 *
	 * @param TypeFlags		actor types to look for - one or more of EActorTypeFlags or'ed together
	 * @param bSingleOnly	if true, then requires selection to be exactly one actor
	 * @return				true if it can execute.
	 */
	static UE_API bool ActorTypesSelected_CanExecute(EActorTypeFlags TypeFlags, bool bSingleOnly);

	/**
	 * Checks to see if multiple actors are selected
	 *	@return true if it can execute.
	 */
	static UE_API bool ActorsSelected_CanExecute();

	/**
	 * Checks to see if at least one element is selected
	 *	@return true if it can execute.
	 */
	UE_DEPRECATED(5.4, "Use ElementSelected_CanExecuteMove. If you only needed to verify if there is at least one item in the selection set, use the selection set directly from the level editor.")
	static UE_API bool ElementSelected_CanExecute();

	/**
	 * Checks to see if multiple elements are selected
	 *	@return true if it can execute.
	 */
	UE_DEPRECATED(5.4, "Use ElementsSelected_CanExecuteMove. If you only needed to verify if there is more than one item in the selection set, use the selection set directly from the level editor.")
	static UE_API bool ElementsSelected_CanExecute();

	/**
	 * Checks to see if at least one element is selected that can be translated
	 *	@return true if it can execute.
	 */
	static UE_API bool ElementSelected_CanExecuteMove();

	/**
	 * Checks to see if multiple elements are selected that can be translated
	 *	@return true if it can execute.
	 */
	static UE_API bool ElementsSelected_CanExecuteMove();

	/**
	 * Checks to see if at least one element is selected that can be scaled
	 *	@return true if it can execute.
	 */
	static UE_API bool ElementSelected_CanExecuteScale();

	/**
	 * Checks to see if multiple elements are selected that can be scaled
	 *	@return true if it can execute.
	 */
	static UE_API bool ElementsSelected_CanExecuteScale();

	/** Called when 'Open Merge Actor' is clicked */
	static UE_API void OpenMergeActor_Clicked();

private:
	/** 
	 * Moves the selected elements.
	 * @param InDestination		The destination element we want to move this element to, or invalid to move to the grid
	 */
	static UE_API void MoveTo_Clicked( const UTypedElementSelectionSet* InSelectionSet, const bool InAlign, bool InPerElement, const TTypedElement<ITypedElementWorldInterface>& InDestination = TTypedElement<ITypedElementWorldInterface>() );

	/** 
	 * Snaps the selected elements. Optionally will align with the trace normal.
	 * @param InAlign			Whether or not to rotate the actor to align with the trace normal.
	 * @param InUseLineTrace	Whether or not to only trace with a line through the world.
	 * @param InUseBounds		Whether or not to base the line trace off of the bounds.
	 * @param InUsePivot		Whether or not to use the pivot position.
	 * @param InDestination		The destination element we want to move this actor to, or invalid to go towards the floor
	 */
	static UE_API void SnapTo_Clicked(const UTypedElementSelectionSet* InSelectionSet, const bool InAlign, const bool InUseLineTrace, const bool InUseBounds, const bool InUsePivot, const TTypedElement<ITypedElementWorldInterface>& InDestination = TTypedElement<ITypedElementWorldInterface>() );

	/** 
	 * Create and apply animation to the SkeletalMeshComponent if Simulating
	 * 
	 * @param EditorActor	Editor Counterpart Actor
	 * @param SimActor		Simulating Actor in PIE or SIE
	 */
	static UE_API bool SaveAnimationFromSkeletalMeshComponent(AActor * EditorActor, AActor * SimActor, TArray<class USkeletalMeshComponent*> & OutEditorComponents);

public:
	static UE_API void FixupGroupActor_Clicked();
};

#undef UE_API
