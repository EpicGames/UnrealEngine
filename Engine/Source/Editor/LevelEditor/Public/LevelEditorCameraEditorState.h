// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorState/WorldEditorState.h"
#include "LevelEditorCameraEditorState.generated.h"

class ILevelEditor;
class SLevelViewport;

UCLASS(MinimalAPI)
class ULevelEditorCameraEditorState : public UWorldDependantEditorState
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UEditorState Interface
	virtual FText GetCategoryText() const override;

	LEVELEDITOR_API const FVector& GetCameraLocation() const;
	LEVELEDITOR_API const FRotator& GetCameraRotation() const;
	LEVELEDITOR_API const float GetCameraFOVAngle() const;

private:
	virtual FOperationResult CaptureState() override;
	virtual FOperationResult RestoreState() const override;
	//~ End UEditorState Interface

	void RestoreCameraState(TSharedPtr<ILevelEditor> LevelEditor) const;

private:
	// Camera position
	UPROPERTY(EditAnywhere, Category = Camera)
	FVector CameraLocation;

	// Camera rotation
	UPROPERTY(EditAnywhere, Category = Camera)
	FRotator CameraRotation;

	// Camera FOV Angle
	UPROPERTY(EditAnywhere, Category = Camera)
	float CameraFOVAngle;

	mutable FDelegateHandle OnLevelEditorCreatedDelegateHandle;
};
