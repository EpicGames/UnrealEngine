// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor/UnrealEdTypes.h"
#include "LevelEditorMenuContext.h"
#include "ToolMenuMisc.h" // FToolMenuInsert

#define UE_API LEVELEDITOR_API

class FExtender;
class UToolMenu;
class UTypedElementSelectionSet;
struct FToolMenuContext;
class SLevelEditor;
class SWidget;
struct FToolMenuSection;

/**
 * Context menu construction class 
 */
class FLevelEditorContextMenu
{

public:

	/**
	 * Summons the level viewport context menu
	 * @param	LevelEditor		The level editor using this menu.
	 * @param	ContextType		The context we should use to specialize this menu
	 * @param	HitProxyActor	The hitproxy actor in the case the ContextType is Viewport
	 */
	static UE_API void SummonMenu( const TSharedRef< class SLevelEditor >& LevelEditor, ELevelEditorMenuContext ContextType, const FTypedElementHandle& HitProxyElement = FTypedElementHandle());

	/**
	 * Summons the viewport view option menu
	 * @param LevelEditor		The level editor using this menu.
	 */
	static UE_API void SummonViewOptionMenu( const TSharedRef< class SLevelEditor >& LevelEditor, const ELevelViewportType ViewOption );

	/**
	 * Creates a widget for the context menu that can be inserted into a pop-up window
	 *
	 * @param	LevelEditor		The level editor using this menu.
	 * @param	ContextType		The context we should use to specialize this menu
	 * @param	Extender		Allows extension of this menu based on context.
	 * @param	HitProxyActor	The hitproxy actor in the case the ContextType is Viewport
	 * @return	Widget for this context menu
	 */
	static UE_API TSharedRef< SWidget > BuildMenuWidget(TWeakPtr< ILevelEditor > LevelEditor, ELevelEditorMenuContext ContextType, TSharedPtr<FExtender> Extender = TSharedPtr<FExtender>(), const FTypedElementHandle& HitProxyElement = FTypedElementHandle());

	/**
	 * Populates the specified menu builder for the context menu that can be inserted into a pop-up window
	 *
	 * @param	Menu			The menu to fill
	 * @param	LevelEditor		The level editor using this menu.
	 * @param	ContextType		The context we should use to specialize this menu
	 * @param	Extender		Allows extension of this menu based on context.
	 * @param	HitProxyActor	The hitproxy actor in the case the ContextType is Viewport
	 */
	static UE_API UToolMenu* GenerateMenu(TWeakPtr< ILevelEditor > LevelEditor, ELevelEditorMenuContext ContextType, TSharedPtr<FExtender> Extender = TSharedPtr<FExtender>(), const FTypedElementHandle& HitProxyElement = FTypedElementHandle());

	/* Adds required information to Context for build menu based on current selection */
	static UE_API FName InitMenuContext(FToolMenuContext& Context, TWeakPtr<ILevelEditor> LevelEditor, ELevelEditorMenuContext ContextType, const FTypedElementHandle& HitProxyElement = FTypedElementHandle());

	/* Returns name of menu to display based on current selection */
	static UE_API FName GetContextMenuName(ELevelEditorMenuContext ContextType, const UTypedElementSelectionSet* InSelectionSet);

	/* Returns a user-readable title for the menu to display; the title can be displayed in UI like the menu bar */
	static UE_API FText GetContextMenuTitle(ELevelEditorMenuContext ContextType, const UTypedElementSelectionSet* InSelectionSet);

	/* Returns a user-readable tooltip describing the menu to display */
	static UE_API FText GetContextMenuToolTip(ELevelEditorMenuContext ContextType, const UTypedElementSelectionSet* InSelectionSet);

	static UE_API void RegisterComponentContextMenu();
	static UE_API void RegisterActorContextMenu();
	static UE_API void RegisterElementContextMenu();
	static UE_API void RegisterSceneOutlinerContextMenu();
	static UE_API void RegisterMenuBarEmptyContextMenu();
	static UE_API void RegisterEmptySelectionContextMenu();

	/**
	 * Registers a menu with name MenuNameToUse that is derived from BaseMenuName, but has
	 *  the level editor context menu items added at InsertLocation.
	 * This is used to add the level editor context menu to a scene outliner context menu.
	 */
	UE_EXPERIMENTAL(5.8, "This helper might be removed when a single outliner implementation is used.")
	static UE_API void RegisterAsDerivedMenu(FName MenuNameToUse, FName BaseMenuName,
		const FToolMenuInsert& InsertLocation = FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));

	static UE_API void AddPlayFromHereSubMenu(FToolMenuSection& Section);

	/**
	 * Builds the actor group menu
	 *
	 * @param Menu		The menu to add items to.
	 * @param SelectedActorInfo	Information about the selected actors.
	 */
	static UE_API void BuildGroupMenu(FToolMenuSection& Section, const struct FSelectedActorInfo& SelectedActorInfo);
};

#undef UE_API
