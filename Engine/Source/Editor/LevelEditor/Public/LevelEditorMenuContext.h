// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "LevelEditorMenuContext.generated.h"

#define UE_API LEVELEDITOR_API

class AActor;
class FLevelEditorViewportClient;
class ILevelEditor;
class SLevelEditor;
class SLevelViewport;
class SLevelViewportToolBar;
class UActorComponent;
class UTypedElementSelectionSet;
struct FFrame;
class UToolMenu;
struct FToolMenuSection;
struct FToolMenuContext;

UCLASS(MinimalAPI)
class ULevelEditorMenuContext : public UObject
{
	GENERATED_BODY()
public:
	UE_API TWeakPtr<ILevelEditor> GetLevelEditor() const;

	TWeakPtr<SLevelEditor> LevelEditor;
};


/** Enum to describe what a level editor context menu should be built for */
UENUM()
enum class ELevelEditorMenuContext : uint8
{
	/** This context menu is applicable to a viewport (limited subset of entries) */
	Viewport,
	/** This context menu is applicable to the Scene Outliner (disables click-position-based menu items) */
	SceneOutliner,
	/** This is the replica of the context menu that appears in the main menu bar (lists all entries) */
	MainMenu,
};

UCLASS(MinimalAPI)
class ULevelEditorContextMenuContext : public UObject
{
	GENERATED_BODY()
public:
	TWeakPtr<ILevelEditor> LevelEditor;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="LevelEditor|Menu")
	ELevelEditorMenuContext ContextType;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="LevelEditor|Menu")
	TObjectPtr<const UTypedElementSelectionSet> CurrentSelection;
	
	/** If the ContextType is Viewport this property can be set to the HitProxy element that triggered the ContextMenu. */
	FTypedElementHandle HitProxyElement;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="LevelEditor|Menu")
	FVector CursorWorldLocation;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="LevelEditor|Menu")
	TArray<TObjectPtr<UActorComponent>> SelectedComponents;

	/** If the ContextType is Viewport this property can be set to the HitProxy actor that triggered the ContextMenu. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="LevelEditor|Menu")
	TWeakObjectPtr<AActor> HitProxyActor = nullptr;

	UFUNCTION(BlueprintCallable, Category = "LevelEditor | Menu", DisplayName="Get Hit Proxy Element", meta=(ScriptName="GetHitProxyElement"))
	UE_API FScriptTypedElementHandle GetScriptHitProxyElement();
};

UCLASS(MinimalAPI)
class ULevelViewportContext : public UObject
{
	GENERATED_BODY()

public:
	static UE_API TSharedPtr<SLevelViewport> GetLevelViewport(const UToolMenu* Menu);
	static UE_API TSharedPtr<SLevelViewport> GetLevelViewport(const FToolMenuSection& Section);
	static UE_API TSharedPtr<SLevelViewport> GetLevelViewport(const FToolMenuContext& Context);

	TWeakPtr<SLevelViewport> LevelViewport;
	
	UE_API FLevelEditorViewportClient* GetLevelViewportClient() const;
};


UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "Use ULevelViewportContext instead") ULevelViewportToolBarContext : public UObject
{
	GENERATED_BODY()
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TWeakPtr<SLevelViewportToolBar> LevelViewportToolBarWidget;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_API virtual FLevelEditorViewportClient* GetLevelViewportClient() const { return nullptr; }
};


UCLASS(MinimalAPI)
class UQuickActionMenuContext : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="LevelEditor|Menu")
	TObjectPtr<const UTypedElementSelectionSet> CurrentSelection;
};

#undef UE_API
