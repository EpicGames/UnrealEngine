// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"

/**
 * Unreal level editor actions
 */
class LEVELEDITOR_API FLevelEditorModesCommands : public TCommands<FLevelEditorModesCommands>
{

public:
	FLevelEditorModesCommands() : TCommands<FLevelEditorModesCommands>
	(
		"LevelEditorModes", // Context name for fast lookup
		NSLOCTEXT("Contexts", "LevelEditorModes", "Level Editor Modes"), // Localized context name for displaying
		"MainFrame", // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
	{
	}
	
	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

public:
	
	/** Editor mode commands */
	TArray< TSharedPtr< FUICommandInfo > > EditorModeCommands;
};


