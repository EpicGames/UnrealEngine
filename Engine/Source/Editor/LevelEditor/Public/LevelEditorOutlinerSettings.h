// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/GenericFilter.h"

#define UE_API LEVELEDITOR_API

struct ISceneOutlinerTreeItem;

namespace SceneOutliner
{
	/** The type of item that the Outliner's Filter Bar operates on */
	typedef const ISceneOutlinerTreeItem& FilterBarType;
}

struct FSceneOutlinerFilterBarOptions;
class FCustomClassFilterData;

/** Structure of built-in filter categories. Defined as functions to enable external use without linkage */
struct FLevelEditorOutlinerBuiltInCategories
{
	static FName Common()			 { static FName Name("Essential");			return Name; }
	static FName Basic()			 { static FName Name("Basic");				return Name; }
	static FName Animation()		 { static FName Name("Animation");			return Name; }
	static FName Audio()			 { static FName Name("Audio");				return Name; }
	static FName Geometry()			 { static FName Name("Geometry");			return Name; }
	static FName Lights()			 { static FName Name("Lights");				return Name; }
	static FName Environment()		 { static FName Name("Environment");		return Name; }
	static FName Visual()			 { static FName Name("Visual");				return Name; }
	static FName Volumes()			 { static FName Name("Volumes");			return Name; }
	static FName VirtualProduction() { static FName Name("VirtualProduction");	return Name; }
	static FName SourceControl()	 { static FName Name("RevisionControl");	return Name; }
};

/** Helper class to manage initalization options specific to the Level Editor Outliners
 *  Use AddCustomFilter/AddCustomClass Filter to register new filters in the Outliner.
 *  Make sure that the filters you attach have a category, otherwise they will not show up
 */
class FLevelEditorOutlinerSettings :  public TSharedFromThis<class FLevelEditorOutlinerSettings>
{
public:
	FLevelEditorOutlinerSettings(){}
	UE_API ~FLevelEditorOutlinerSettings();
	
	UE_API void Initialize();
	
	/** Delegate to create a Filter for the Outliner */
	DECLARE_DELEGATE_RetVal(TSharedPtr<FFilterBase<const ISceneOutlinerTreeItem&>>, FOutlinerFilterFactory);

	/**  Add a custom filter to the outliner filter bar. These are all AND'd together
	 *   @see FGenericFilter on how to create generic filters
	 *   Note: Any filter added here will be shared between all 4 Outliners, so you cannot rely on the filter to know
	 *   which outliner it is active in. Use the override that takes a delegate instead
	 */

	UE_DEPRECATED(5.2, "Use the AddCustomFilter override that takes in a factory instead so each Outliner can have a unique instance of the filter")
	UE_API void AddCustomFilter(TSharedRef<FFilterBase<const ISceneOutlinerTreeItem&>> InCustomFilter);

	/**  Add a custom filter to the outliner filter bar. These are all AND'd together
	 *   This function takes in a delegate that will be called to create an instance of the custom filter for each Outliner that
	 *   the level editor creates.
	 *   @see FGenericFilter on how to create generic filters
	 */
	UE_API void AddCustomFilter(FOutlinerFilterFactory InCreateCustomFilter);

	/**  Add a custom class filter to the outliner filter bar. These represent asset/actor type filters and are OR'd
	 *   Can be created using IAssetTypeActions or UClass (@see constructor)
	 */
	UE_API void AddCustomClassFilter(TSharedRef<FCustomClassFilterData> InCustomClassFilterData);

	// Creates the default filters that the level editor outliner has
	UE_API void CreateDefaultFilters();

	// Setup the built in filter categories
	UE_API void SetupBuiltInCategories();

	// Append the init options stored in this class to the given Outliner init options
	UE_API void GetOutlinerFilters(FSceneOutlinerFilterBarOptions& OutFilterBarOptions);

	// Get the FFilterCategory attached to the given category name. Use this to add filters to the built in categories
	UE_API TSharedPtr<FFilterCategory> GetFilterCategory(const FName& CategoryName);

private:

	UE_API void CreateSCCFilters();

	UE_API bool DoesActorPassUnsavedFilter(const ISceneOutlinerTreeItem& InItem);
	UE_API bool DoesActorPassUncontrolledFilter(const ISceneOutlinerTreeItem& InItem);

	UE_API void OnUnsavedAssetAdded(const FString& InAsset);
	UE_API void OnUnsavedAssetRemoved(const FString& InAsset);

	UE_API void OnUncontrolledChangelistModuleChanged();
	
	// Refresh any Outliners that have the given filter active
	UE_API void RefreshOutlinersWithActiveFilter(bool bFullRefresh, const FString& InFilterName);
private:

	/** These are the custom filters that the Scene Outliner will have. All active filters will be AND'd together to test
	 *  against.
	 */
	TArray<TSharedRef<FFilterBase<const ISceneOutlinerTreeItem&>>> CustomFilters;

	/** These are delegates that will be used to create custom filters for each outliner that calls GetOutlinerFilters */ 
	TArray<FOutlinerFilterFactory> CustomFilterDelegates;

	/** These are the asset type filters that the Scene Outliner will have. All active filters will be OR'd together to
	*  test against.
	 */
	TArray<TSharedRef<FCustomClassFilterData>> CustomClassFilters;

	/** The built in custom filters created by this instance */
	TArray<TSharedRef<FFilterBase<const ISceneOutlinerTreeItem&>>> BuiltInCustomFilters;

	/* A map of the categories the Outliner filter bar will have */
	TMap<FName, TSharedPtr<FFilterCategory>> FilterBarCategories;

	/* A map to convert placement mode built in categories to filter bar categories */
	TMap<FName, FName> PlacementToFilterCategoryMap;

	// Source Control Cache

	// List of currently unsaved packages. Using a set here because it can grow a lot and impact performances when looking up in the list.
	TSet<FName> UnsavedPackages;

	// List of currently uncontrolled packages. Using a set here because it can grow a lot and impact performances when looking up in the list.
	TSet<FName> UncontrolledPackages;

	static UE_API const FString UnsavedAssetsFilterName;
	static UE_API const FString UncontrolledAssetsFilterName;
};

#undef UE_API
