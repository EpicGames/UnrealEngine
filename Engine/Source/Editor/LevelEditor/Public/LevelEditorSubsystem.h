// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "Editor/UnrealEdTypes.h"
#include "Engine/EngineTypes.h"
#include "IActorEditorContextClient.h"
#include "UObject/ObjectSaveContext.h"

#include "LevelEditorSubsystem.generated.h"

#define UE_API LEVELEDITOR_API

class UTypedElementSelectionSet;
struct FToolMenuContext;
class FEditorModeTools;

/** Delegate type for pre save world events ( uint32 SaveFlags, UWorld* World ) */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnLevelEditorPreSaveWorld, int32, SaveFlags, class UWorld*, World);
/** Delegate type for post save world events ( uint32 SaveFlags, UWorld* World, bool bSuccess ) */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnLevelEditorPostSaveWorld, int32, SaveFlags, class UWorld*, World, bool, bSuccess);

/** Delegate type for editor camera movement */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams(FOnLevelEditorEditorCameraMoved, const FVector&, Location, const FRotator&, Rotation, ELevelViewportType, ViewportType, int32, ViewIndex);

/** Delegate type for map change events ( Params: uint32 MapChangeFlags (MapChangeEventFlags) ) */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnLevelEditorMapChanged, int32, MapChangeEventFlags);
/** Delegate type for triggering when a map is opened */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnLevelEditorMapOpened, const FString&, Filename, bool, bAsTemplate);

/**
* ULevelEditorSubsystem
* Subsystem for exposing Level Editor related functionality to scripts
*/
UCLASS(MinimalAPI)
class ULevelEditorSubsystem : public UEditorSubsystem, public IActorEditorContextClient
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	UE_API virtual void Deinitialize() override;
	UE_API void ExtendQuickActionMenu();

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta=(DevelopmentOnly))
	UE_API void PilotLevelActor(AActor* ActorToPilot, FName ViewportConfigKey = NAME_None);
	UE_API void PilotLevelActor(const FToolMenuContext& InContext);
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void EjectPilotLevelActor(FName ViewportConfigKey = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta=(DevelopmentOnly))
	UE_API AActor* GetPilotLevelActor(FName ViewportConfigKey = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API bool GetExactCameraView(FName ViewportConfigKey = NAME_None);
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void SetExactCameraView(bool bExactCameraView, FName ViewportConfigKey = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void EditorPlaySimulate();

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void EditorInvalidateViewports();

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void EditorSetViewportRealtime(bool bInRealtime, FName ViewportConfigKey = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void EditorSetGameView(bool bGameView, FName ViewportConfigKey = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API bool EditorGetGameView(FName ViewportConfigKey = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void EditorRequestBeginPlay();

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void EditorRequestEndPlay();

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API bool IsInPlayInEditor() const;

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API TArray<FName> GetViewportConfigKeys();

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API FName GetActiveViewportConfigKey();

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void SetAllowsCinematicControl(bool bAllow, FName ViewportConfigKey = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API bool GetAllowsCinematicControl(FName ViewportConfigKey = NAME_None);

	/**
	 * Gets information about the camera position for the given level editor viewport.
	 *
	 * @param	CameraLocation	(out) Current location of the level editing viewport camera, or zero if none found
	 * @param	CameraRotation	(out) Current rotation of the level editing viewport camera, or zero if none found
	 * @param	ViewportConfigKey the viewport you wish to get from
	 * @return	Whether we were able to get the camera position for a level editing viewport
	 */
	UFUNCTION(BlueprintPure, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API bool GetLevelViewportCameraInfo(FVector& CameraLocation, FRotator& CameraRotation, FName ViewportConfigKey);

	
	/**
	* Sets information about the camera position for the given level editor viewport.
	*
	* @param	CameraLocation	Location the camera will be moved to
	* @param	CameraRotation	Rotation the camera will be set to
	* @param	ViewportConfigKey	the viewport you wish to target
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DevelopmentOnly))
	UE_API void SetLevelViewportCameraInfo(FVector CameraLocation, FRotator CameraRotation, FName ViewportConfigKey);
	
	/**
	 * Gets the horizontal field of view for the given level editor viewport. 
	 *
	 * @param	FOV				Field of View of the Level editing viewport camera
	 * @param	ViewportConfigKey	the viewport you wish to get from
	 * @return	Whether we were able to get the FOV for a level editing viewport
	 */
	UFUNCTION(BlueprintPure, Category = "Editor Scripting | Level Utility", meta=(DisplayName="Get Level Viewport Field Of View", DevelopmentOnly))
	UE_API bool GetLevelViewportFOV(float& FOV, FName ViewportConfigKey);

	/**
	* Sets the horizontal field of view for the given level editor viewport.
	*
	* @param	FOV					Field of View to set the camera to. Value will be clamped between 5 and 170 deg
	* @param	ViewportConfigKey	the viewport you wish to target
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta=(DisplayName="Set Level Viewport Field Of View", DevelopmentOnly))
	UE_API  void SetLevelViewportFOV(float FOV, FName ViewportConfigKey);
	
	/**
	 * Close the current Persistent Level (without saving it). Create a new blank Level and save it. Load the new created level.
	 * @param	AssetPath		Asset Path of where the level will be saved.
	 *		ie. /Game/MyFolder/MyAsset
	 * @param   bIsPartitionedWorld	If true, new map is partitioned.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API bool NewLevel(const FString& AssetPath, bool bIsPartitionedWorld = false);

	/**
	 * Close the current Persistent Level (without saving it). Create a new Level base on another level and save it. Load the new created level.
	 * @param	AssetPath				Asset Path of where the level will be saved.
	 *		ie. /Game/MyFolder/MyAsset
	 * @param	TemplateAssetPath		Level to be used as Template.
	 *		ie. /Game/MyFolder/MyAsset
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API bool NewLevelFromTemplate(const FString& AssetPath, const FString& TemplateAssetPath);

	/**
	 * Close the current Persistent Level (without saving it). Loads the specified level.
	 * @param	AssetPath				Asset Path of the level to be loaded.
	 *		ie. /Game/MyFolder/MyAsset
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API bool LoadLevel(const FString& AssetPath);

	/**
	 * Saves the specified Level. Must already be saved at lease once to have a valid path.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API bool SaveCurrentLevel();

	/**
	 * Saves all Level currently loaded by the World Editor.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API bool SaveAllDirtyLevels();

	/**
	 * Set the current level used by the world editor.
	 * If more than one level shares the same name, the first one encounter of that level name will be used.
	 * @param	LevelName	The name of the Level the actor belongs to (same name as in the ContentBrowser).
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API bool SetCurrentLevelByName(FName LevelName);

	/**
	* Get the current level used by the world editor.
	* @return	The current level
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API ULevel* GetCurrentLevel();

	/**
	 * Deletes a folder from the World Outliner.
	 * Does not move or delete actors; only removes the folder entry itself.
	 * @param	FolderPath	The path of the folder to delete (e.g. 'Lighting/Spotlights').
	 * @return	True if the folder existed and was deleted.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API bool DeleteActorFolder(FName FolderPath);

	/**
	 * Get the selection set for the current world, you can use this to track
	 * and create changes to the level editor's selection
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API UTypedElementSelectionSet* GetSelectionSet();

	/**
	 * Build Light Maps and optionally the reflection captures.
	 * @param	Quality	One of the enum LightingBuildQuality value. Default is Quality_Production.
	 * @param	bWithReflectionCaptures	Build the related reflection captures after building the light maps.
	 * @return	True if build was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UE_API bool BuildLightMaps(ELightingBuildQuality Quality = ELightingBuildQuality::Quality_Production, bool bWithReflectionCaptures = false);

	/**
	 * Gets the global level editor mode manager, if we have one.
	 * The mode manager is not created in commandlet environments, because modes inherently imply user interactions.
	 */
	UE_API FEditorModeTools* GetLevelEditorModeManager();

	//~ Begin IActorEditorContextClient interface
	virtual void OnExecuteActorEditorContextAction(UWorld* InWorld, const EActorEditorContextAction& InType, class AActor* InActor = nullptr) override {}
	virtual void CaptureActorEditorContextState(UWorld* InWorld, UActorEditorContextStateCollection* InStateCollection) const override {}
	virtual void RestoreActorEditorContextState(UWorld* InWorld, const UActorEditorContextStateCollection* InStateCollection) override {}
	UE_API virtual bool GetActorEditorContextDisplayInfo(UWorld* InWorld, FActorEditorContextClientDisplayInfo& OutDiplayInfo) const override;
	virtual bool CanResetContext(UWorld* InWorld) const override { return false; };
	UE_API virtual TSharedRef<SWidget> GetActorEditorContextWidget(UWorld* InWorld) const override;
	virtual FOnActorEditorContextClientChanged& GetOnActorEditorContextClientChanged() override { return ActorEditorContextClientChanged; }
	//~ End IActorEditorContextClient interface

	/** Expose PreSaveWorld to blueprints */
	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnLevelEditorPreSaveWorld OnPreSaveWorld;
	/** Expose PostSaveWorld to blueprints */
	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnLevelEditorPostSaveWorld OnPostSaveWorld;

	/** Expose EditorCameraMoved to blueprints */
	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnLevelEditorEditorCameraMoved OnEditorCameraMoved;

	/** Expose MapChanged to blueprints. Note: This executes too early for some editor scripting, consider using OnMapOpened if this doesn't work for you. */
	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnLevelEditorMapChanged OnMapChanged;
	
	/** Expose MapOpened to blueprints */
	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnLevelEditorMapOpened OnMapOpened;

private:

	/** Called when a Level is added to a world or remove from a world */
	UE_API void OnLevelAddedOrRemoved(ULevel* InLevel, UWorld* InWorld);

	/** Called when the current level changes on a world */
	UE_API void OnCurrentLevelChanged(ULevel* InNewLevel, ULevel* InOldLevel, UWorld* InWorld);

	/** Delegate used to notify changes to ActorEditorContextSubsystem */
	FOnActorEditorContextClientChanged ActorEditorContextClientChanged;

	/** Called Pre SaveWorld */
	UE_API void HandleOnPreSaveWorldWithContext(class UWorld* World, FObjectPreSaveContext ObjectSaveContext);
	/** Called Post SaveWorld */
	UE_API void HandleOnPostSaveWorldWithContext(class UWorld* World, FObjectPostSaveContext ObjectSaveContext);

	/** Called on Camera Movement*/
	UE_API void HandleOnEditorCameraMoved(const FVector& Location, const FRotator& Rotation, ELevelViewportType ViewportType, int32 ViewIndex);

	/** Called on Map Change */
	UE_API void HandleOnMapChanged(uint32 MapChangeFlags);
	/** Called on Map Open */
	UE_API void HandleOnMapOpened(const FString& Filename, bool bAsTemplate);
};

#undef UE_API
