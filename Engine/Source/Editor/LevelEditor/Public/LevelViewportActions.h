// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"
#include "BufferVisualizationMenuCommands.h"
#include "NaniteVisualizationMenuCommands.h"
#include "LumenVisualizationMenuCommands.h"
#include "SubstrateVisualizationMenuCommands.h"
#include "GroomVisualizationMenuCommands.h"
#include "VirtualShadowMapVisualizationMenuCommands.h"

/**
 * Public identifiers for the viewport layouts available in LevelViewportLayoutX.h files
 * These are names rather than enums as they're also used to persist states in config
 * When editing this list you also need to edit FLevelViewportTabContent::ConstructViewportLayoutByTypeName()
 */
namespace LevelViewportConfigurationNames
{
	static FName TwoPanesHoriz("TwoPanesHoriz");
	static FName TwoPanesVert("TwoPanesVert");
	static FName ThreePanesLeft("ThreePanesLeft");
	static FName ThreePanesRight("ThreePanesRight");
	static FName ThreePanesTop("ThreePanesTop");
	static FName ThreePanesBottom("ThreePanesBottom");
	static FName FourPanesLeft("FourPanesLeft");
	static FName FourPanesRight("FourPanesRight");
	static FName FourPanesTop("FourPanesTop");
	static FName FourPanesBottom("FourPanesBottom");
	static FName FourPanes2x2("FourPanes2x2");
	static FName OnePane("OnePane");
}

UE_DECLARE_TCOMMANDS(class FLevelViewportCommands, LEVELEDITOR_API)

/**
 * Class containing commands for level viewport actions
 */
class FLevelViewportCommands : public TCommands<FLevelViewportCommands>
{
public:

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FLevelViewportCommands() 
		: TCommands<FLevelViewportCommands>
		(
			TEXT("LevelViewport"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "LevelViewports", "Level Viewports"), // Localized context name for displaying
			TEXT("EditorViewport"), // Parent context name.  
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		),
		ClearAllBookMarks(nullptr)
	{
	}

	FLevelViewportCommands(const FLevelViewportCommands&) = default;
	FLevelViewportCommands(FLevelViewportCommands&&) = default;
	FLevelViewportCommands& operator=(const FLevelViewportCommands&) = default;
	FLevelViewportCommands& operator=(FLevelViewportCommands&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual ~FLevelViewportCommands();

	struct FShowMenuCommand
	{
		TSharedPtr<FUICommandInfo> ShowMenuItem;
		FText LabelOverride;

		FShowMenuCommand( TSharedPtr<FUICommandInfo> InShowMenuItem, const FText& InLabelOverride )
			: ShowMenuItem( InShowMenuItem )
			, LabelOverride( InLabelOverride )
		{
		}

		FShowMenuCommand( TSharedPtr<FUICommandInfo> InShowMenuItem )
			: ShowMenuItem( InShowMenuItem )
		{
		}
	};

	/** Opens the advanced viewport settings */
	TSharedPtr< FUICommandInfo > AdvancedSettings;
	
	/** Opens the play settings for PIE */
	TSharedPtr< FUICommandInfo > PlaySettings;

	/** Toggles game preview in the viewport */
	TSharedPtr< FUICommandInfo > ToggleGameView;

	/** Toggles immersive mode in the viewport */
	TSharedPtr< FUICommandInfo > ToggleImmersive;
	 
	/** Toggles moving all tabs except the viewport to a sidebar or removing them from a sidebar */
	TSharedPtr< FUICommandInfo > ToggleSidebarAllTabs;

	/** Toggles maximize mode in the viewport */
	TSharedPtr< FUICommandInfo > ToggleMaximize;

	/** Creates a subclass of ACameraActor at current perspective viewport position */
	TArray< TSharedPtr< FUICommandInfo > > CreateCameras;

	/** Opens the control panel for high resolution screenshots */
	TSharedPtr< FUICommandInfo > HighResScreenshot;

	/** Reset all show flags to default */
	TSharedPtr< FUICommandInfo > UseDefaultShowFlags;

	/** Allows this viewport to be used for previewing cinematic animations */ 
	TSharedPtr< FUICommandInfo > ToggleCinematicPreview;

	/** Allows this viewport to have a Constrained Aspect Ratio in Preview */
	TSharedPtr< FUICommandInfo > ToggleAllowConstrainedAspectRatioInPreview;

	/** Allows this viewport to flip the Constrained Aspect Ratio in Preview */
	TSharedPtr< FUICommandInfo > ToggleFlipConstrainedAspectRatioInPreview;

	/** Finds instances of selected object in level script. */ 
	TSharedPtr< FUICommandInfo > FindInLevelScriptBlueprint;

	/** Shows all volume classes */
	TSharedPtr< FUICommandInfo > ShowAllVolumes;

	/** Hides all volume classes */
	TSharedPtr< FUICommandInfo > HideAllVolumes;

	/** List of commands for showing volume classes and their localized names.  One for each command */
	TArray< FShowMenuCommand > ShowVolumeCommands;

	/** Shows all layers */
	TSharedPtr< FUICommandInfo > ShowAllLayers;

	/** Hides all layers */
	TSharedPtr< FUICommandInfo > HideAllLayers;

	/** Shows all DataLayers */
	TSharedPtr< FUICommandInfo > ShowAllDataLayers;

	/** Hides all DataLayers */
	TSharedPtr< FUICommandInfo > HideAllDataLayers;

	/** Shows all sprite categories */
	TSharedPtr< FUICommandInfo > ShowAllSprites;

	/** Hides all sprite categories  */
	TSharedPtr< FUICommandInfo > HideAllSprites;

	/** List of commands for showing sprite categories and their localized names.  One for each command */
	TArray< FShowMenuCommand > ShowSpriteCommands;

	/** Hides all Stats categories  */
	TSharedPtr< FUICommandInfo > HideAllStats;

	/** A map of stat categories and the commands that belong in them */
	TMap< FString, TArray< FShowMenuCommand > > ShowStatCatCommands;

	/** Applys a material to an actor */
	TSharedPtr< FUICommandInfo > ApplyMaterialToActor;
	
	TSharedPtr< FUICommandInfo > FocusViewportToSelectedActors;

	/** Enable editor shortcuts when PIE is active */
	TSharedPtr< FUICommandInfo > EnableEditorShortcutsInPIE;

	/**
	 * Bookmarks                   
	 */
	TArray< TSharedPtr< FUICommandInfo > > JumpToBookmarkCommands;
	TArray< TSharedPtr< FUICommandInfo > > SetBookmarkCommands;
	TArray< TSharedPtr< FUICommandInfo > > ClearBookmarkCommands;
	TSharedPtr< FUICommandInfo > CompactBookmarks;

	UE_DEPRECATED(4.21, "Please use the version of the member with corrected spelling (ClearAllBookmarks).")
	TSharedPtr< FUICommandInfo > ClearAllBookMarks;

	TSharedPtr< FUICommandInfo > ClearAllBookmarks;

	/** Actor pilot commands */
	TSharedPtr< FUICommandInfo > SelectPilotedActor;
	TSharedPtr< FUICommandInfo > EjectActorPilot;
	TSharedPtr< FUICommandInfo > PilotSelectedActor;

	/** Toggles showing the exact camera view when locking a viewport to a camera */
	TSharedPtr< FUICommandInfo > ToggleActorPilotCameraView;

	/** Viewport pane configurations */
	TSharedPtr< FUICommandInfo > ViewportConfig_OnePane;
	TSharedPtr< FUICommandInfo > ViewportConfig_TwoPanesH;
	TSharedPtr< FUICommandInfo > ViewportConfig_TwoPanesV;
	TSharedPtr< FUICommandInfo > ViewportConfig_ThreePanesLeft;
	TSharedPtr< FUICommandInfo > ViewportConfig_ThreePanesRight;
	TSharedPtr< FUICommandInfo > ViewportConfig_ThreePanesTop;
	TSharedPtr< FUICommandInfo > ViewportConfig_ThreePanesBottom;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanesLeft;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanesRight;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanesTop;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanesBottom;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanes2x2;

	TSharedPtr< FUICommandInfo > SetDefaultViewportType;

	TSharedPtr< FUICommandInfo > ToggleViewportToolbar;

	TSharedPtr< FUICommandInfo > EnablePreviewMesh;
	TSharedPtr< FUICommandInfo > CyclePreviewMesh;

	/** Delegate we fire every time a new stat has been had a command added */
	DECLARE_EVENT_TwoParams(FLevelViewportCommands, FOnNewStatCommandAdded, const TSharedPtr<FUICommandInfo>, const FString&);
	static FOnNewStatCommandAdded NewStatCommandDelegate;

public:
	/** Registers our commands with the binding system */
	virtual void RegisterCommands() override;

	/** Registers our commands with the binding system for showing volumes. */
	void RegisterShowVolumeCommands();

	/** Registers our commands with the binding system for showing sprites. */
	void RegisterShowSpriteCommands();

private:
#if STATS
	/** Registers additional commands as they are loaded */
	void HandleNewStatGroup(const TArray<FStatNameAndInfo>& NameAndInfos);
	void HandleNewStat(const FName& InStatName, const FName& InStatCategory, const FText& InStatDescription);
	int32 FindStatIndex(const TArray< FShowMenuCommand >* ShowStatCommands, const FString& InCommandName) const;
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Dummy function that's never used.
	// Used to hide deprecation warnings due to auto generated methods.
	void Dummy()
	{
		FLevelViewportCommands Commands;
		FLevelViewportCommands CommandsCopy(Commands);
		FLevelViewportCommands CommandsCopy2(MoveTemp(Commands));
		Commands = CommandsCopy2;
		CommandsCopy = MoveTemp(Commands);
		check(false);
	}
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS