// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Stats/Stats.h"
#include "Misc/Attribute.h"
#include "Animation/CurveSequence.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Editor/UnrealEdTypes.h"
#include "Application/ThrottleManager.h"
#include "AssetEditorViewportLayout.h"
#include "TickableEditorObject.h"

#define UE_API LEVELEDITOR_API

class FLevelEditorViewportClient;
class FLevelViewportLayout;
class FLevelViewportTabContent;
class ILevelEditor;
class SLevelViewport;
class SViewportsOverlay;
class SWindow;

/** Interface that defines an entity within a viewport layout */
class ILevelViewportLayoutEntity : public IEditorViewportLayoutEntity
{
public:
	/** Virtual destruction */
	virtual ~ILevelViewportLayoutEntity() = default;

	/** Optionally return this entity as an SLevelViewport. Legacy function for interop with code that used to deal directly with SLevelViewports */
	virtual TSharedPtr<SLevelViewport> AsLevelViewport() const { return nullptr; }

	/** Get this viewport's level editor viewport client */
	virtual FLevelEditorViewportClient& GetLevelViewportClient() const = 0;

	/** Check if this entity has an active play in editor viewport */
	virtual bool IsPlayInEditorViewportActive() const = 0;

	/** Register this viewport layout entity as a game viewport, if it's currently PIE-ing */
	virtual void RegisterGameViewportIfPIE() = 0;
};

/**
 * Base class for level viewport layout configurations
 * Handles maximizing and restoring well as visibility of specific viewports.
 */
class FLevelViewportLayout : public FAssetEditorViewportLayout
{
public:
	/**
	 * Constructor
	 */
	UE_API FLevelViewportLayout();

	/**
	 * Destructor
	 */
	UE_API virtual ~FLevelViewportLayout();

	/**
	 * Builds a viewport layout and returns the widget containing the layout
	 * 
	 * @param InParentDockTab		The parent dock tab widget of this viewport configuration
	 * @param InParentTab			The parent tab object
	 * @param LayoutString			The layout string loaded from file to custom build the layout with
	 * @param InParentLevelEditor	Optional level editor parent to use for new viewports
	 */
	UE_API virtual TSharedRef<SWidget> BuildViewportLayout(TSharedPtr<SDockTab> InParentDockTab, TSharedPtr<FEditorViewportTabContent> InParentTab, const FString& LayoutString) override;

	UE_API virtual TSharedRef<SWidget> FactoryViewport(FName InTypeName, const FAssetEditorViewportConstructionArgs& ConstructionArgs) override;

	/**
	 * Makes a request to maximize a specific viewport and hide the others in this layout
	 * 
	 * @param	ViewportToMaximize	The viewport that should be maximized
	 * @param	bWantMaximize		True to maximize or false to "restore"
	 * @param	bWantImmersive		True to perform an "immersive" maximize, which transitions the viewport to fill the entire application window
	 * @param	bAllowAnimation		True if an animated transition should be used
	 */
	UE_API void RequestMaximizeViewport( FName ViewportToMaximize, const bool bWantMaximize, const bool bWantImmersive, const bool bAllowAnimation = true );

	/**
	 * @return true if this layout is visible.  It is not visible if its parent tab is not active
	 */
	UE_API virtual bool IsVisible() const override;

	/**
	 * Checks to see the specified level viewport is visible in this layout
	 * A viewport is visible in a layout if the layout is visible and the viewport is the maximized viewport or there is no maximized viewport
	 *
	 * @param InViewport	The viewport within this layout that should be checked
	 * @return true if the viewport is visible.  
	 */
	UE_API bool IsLevelViewportVisible( FName InViewport ) const;

	/**
	* Checks to see if the specified level viewport supports maximizing one pane
	*
	* @return true if the viewport supports maximizing
	*/
	bool IsMaximizeSupported() const { return bIsMaximizeSupported; }

	/**
	 * @return true if any viewport is maximized. 
	 */
	bool IsMaximized() const { return bIsMaximized;}

	/** 
	 * Checks to see if the specified level viewport is maximized
	 *
	 * @param InViewport	The viewport to check
	 * @return true if the viewport is maximized, false otherwise
	 */
	UE_API bool IsViewportMaximized( FName InViewport ) const;

	/** 
	 * Checks to see if the specified level viewport is in immersive mode
	 *
	 * @param InViewport	The viewport to check
	 * @return true if the viewport is immersive, false otherwise
	 */
	UE_API bool IsViewportImmersive( FName InViewport ) const;




	/** FTickableEditorObject interface */
	UE_API virtual void Tick( float DeltaTime ) override;
	UE_API virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override { return TStatId(); }


	/** Tells this layout whether it was the intial layout or replaced an existing one when the user switched layouts */
	void SetIsReplacement(bool bInIsReplacement) { bIsReplacement = bInIsReplacement; }

	/** Returns whether a viewport animation is currently taking place */
	bool IsTransitioning() const { return bIsTransitioning; }

	UE_API virtual void LoadConfig(const FString& LayoutString) override;
	UE_API virtual void SaveConfig(const FString& LayoutString) const override;

protected:

	/**
	 * Maximizes a specific viewport and hides the others in this layout
	 * 
	 * @param	ViewportToMaximize	The viewport that should be maximized
	 * @param	bWantMaximize		True to maximize or false to "restore"
	 * @param	bWantImmersive		True to perform an "immersive" maximize, which transitions the viewport to fill the entire application window
	 * @param	bAllowAnimation		True if an animated transition should be used
	 */
	UE_API void MaximizeViewport( FName ViewportToMaximize, const bool bWantMaximize, const bool bWantImmersive, const bool bAllowAnimation );

	/**
	 * Delegate called to get the visibility of the non-maximized viewports
	 * The non-maximized viewports are not visible if there is a maximized viewport on top of them
	 *
	 * @param EVisibility::Visible when visible, EVisibility::Collapsed otherwise
	 */
	UE_API virtual EVisibility OnGetNonMaximizedVisibility() const override;

	/**
	 * Returns the widget position for viewport transition animations
	 *
	 * @return	Viewport position on canvas
	 */
	UE_API FVector2D GetMaximizedViewportPositionOnCanvas() const;

	/**
	 * Returns the widget size for viewport transition animations
	 *
	 * @return	Viewport size on canvas
	 */
	UE_API FVector2D GetMaximizedViewportSizeOnCanvas() const;

	/** If a viewport animation is in progress, finishes that transition immediately */
	UE_API void FinishMaximizeTransition();

	/**
	 * Begins a draw throttle for responsiveness when animating a viewports size
	 */
	UE_API void BeginThrottleForAnimatedResize();

	/**
	 * Ends a draw throttle for responsiveness when animating a viewports size
	 */
	UE_API void EndThrottleForAnimatedResize();

protected:
	/** Called in after factory of a new layout functions for derived types of layout to init values common to all types */
	UE_API void InitCommonLayoutFromString(const FString& LayoutString, const FName PerspectiveViewportKey);

	/** Called in SaveConfig() functions for derived types of layout to save values common to all types */
	UE_API void SaveCommonLayoutString(const FString& SpecificLayoutString) const;

	/** True if we've started an animation and are waiting for it to finish */
	bool bIsTransitioning;

	/** Curve for animating from a "restored" state to a maximized state */
	FCurveSequence MaximizeAnimation;

	/** The optional parent level editor for this layout */
	TWeakPtr< ILevelEditor > ParentLevelEditor;

	/** Method for taking high res screen shots of viewports */
	UE_API void TakeHighResScreenShot();

	/** The current maximized viewport if any */
	FName MaximizedViewport;

	/** True if the user selected this layout, false if it's the initial layout loaded */
	bool bIsReplacement;

	/** Temporarily set to true while we are querying layout metrics and want all widgets to be visible */
	bool bIsQueryingLayoutMetrics;

	/** True if the layout supports maximizing one viewport, false if the feature is disabled  */
	bool bIsMaximizeSupported;

	/** True if we're currently maximized */
	bool bIsMaximized;

	/** True if we're currently in immersive mode */
	bool bIsImmersive;

	/** True when transitioning from a maximized state */
	bool bWasMaximized;

	/** True when transitioning from an immersive state */
	bool bWasImmersive;

	/** Window-space start position of the viewport that's currently being maximized */
	FVector2D MaximizedViewportStartPosition;

	/** Window-space start size of the viewport that's currently being maximized */
	FVector2D MaximizedViewportStartSize;

	/** When maximizing viewports (or making them immersive), this stores the widget we create to wrap the viewport */
	TSharedPtr< SWidget > ViewportsOverlayWidget;

	/** Dummy widget that we'll inline-replace viewport widgets with while a view is maximized (or made immersive) */
	TSharedPtr< SWidget > ViewportReplacementWidget;
	
	/** Caches the window that our widgets are contained within */
	TWeakPtr< SWindow > CachedOwnerWindow;

	/** Viewport resize draw throttle request */
	FThrottleRequest ViewportResizeThrottleRequest;

	/**
	 * Maximize/immersive commands can be queued up at startup to be executed on the first tick.
	 * This is necessary, because these commands can't be executed until the viewport has a parent window,
	 * which might not be there upon viewport initialization
	 */
	struct FMaximizeViewportCommand
	{
		FMaximizeViewportCommand(FName InViewport, bool bInMaximize, bool bInImmersive, bool bInToggle=true, bool bInAllowAnimation=true)
			: Viewport(InViewport)
			, bMaximize(bInMaximize)
			, bImmersive(bInImmersive) 
			, bToggle(bInToggle)
			, bAllowAnimation(bInAllowAnimation)
		{}

		FName Viewport;
		bool bMaximize;
		bool bImmersive;
		bool bToggle;
		bool bAllowAnimation;
	};
	TArray<FMaximizeViewportCommand> DeferredMaximizeCommands;
};

#undef UE_API
