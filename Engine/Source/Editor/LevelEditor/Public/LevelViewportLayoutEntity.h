// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "LevelViewportLayout.h"

#define UE_API LEVELEDITOR_API

class FLevelEditorViewportClient;
class SLevelViewport;

class FLevelViewportLayoutEntity : public ILevelViewportLayoutEntity
{
public:

	UE_API FLevelViewportLayoutEntity(TSharedPtr<SAssetEditorViewport> InLevelViewport);
	UE_API virtual TSharedRef<SWidget> AsWidget() const override;
	UE_API virtual TSharedPtr<SLevelViewport> AsLevelViewport() const override;

	UE_API bool IsPlayInEditorViewportActive() const;
	UE_API void RegisterGameViewportIfPIE();
	UE_API void SetKeyboardFocus();
	UE_API void OnLayoutDestroyed();
	UE_API void SaveConfig(const FString& ConfigSection);
	UE_API FLevelEditorViewportClient& GetLevelViewportClient() const;
	UE_API FName GetType() const;
	UE_API void TakeHighResScreenShot() const;

protected:

	/** This entity's level viewport */
	TSharedRef<SLevelViewport> LevelViewport;
};

#undef UE_API
