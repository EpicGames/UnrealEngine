// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelViewportLayout.h"
#include "EditorViewportTabContent.h"

#define UE_API LEVELEDITOR_API

class ILevelEditor;
class FEditorViewportLayout;

/**
 * Represents the content in a viewport tab in the level editor.
 * Each SDockTab holding viewports in the level editor contains and owns one of these.
 */
class FLevelViewportTabContent : public FEditorViewportTabContent
{
public:
	UE_API ~FLevelViewportTabContent();

	/** Starts the tab content object and creates the initial layout based on the layout string */
	UE_API virtual void Initialize(AssetEditorViewportFactoryFunction Func, TSharedPtr<SDockTab> InParentTab, const FString& InLayoutString) override;

	UE_API virtual void BindViewportLayoutCommands(FUICommandList& InOutCommandList, FName ViewportConfigKey) override;

protected:
	UE_API virtual TSharedPtr<FEditorViewportLayout> FactoryViewportLayout(bool bIsSwitchingLayouts) override;
	UE_API virtual FName GetLayoutTypeNameFromLayoutString() const override;

	UE_API void OnLayoutStartChange(bool bSwitchingLayouts);
	UE_API void OnLayoutChanged();

private:
	UE_API void OnUIActionSetViewportConfiguration(FName InConfigurationName);
	UE_API FName GetViewportTypeWithinLayout(FName InConfigKey) const;
	UE_API void OnUIActionSetViewportTypeWithinLayout(FName InConfigKey, FName InLayoutType);
	UE_API bool IsViewportTypeWithinLayoutEqual(FName InConfigName, FName InLayoutType) const;
	UE_API bool IsViewportConfigurationChecked(FName InLayoutType) const;
	UE_API bool IsLayoutMaximized() const;
};

#undef UE_API
