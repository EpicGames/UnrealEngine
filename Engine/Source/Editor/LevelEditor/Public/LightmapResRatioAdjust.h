// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	LightmapResRatioAdjust.h: Lightmap Resolution Ratio Adjustment helper
================================================================================*/

#pragma once

#include "CoreMinimal.h"

class ULevel;

/**
 *	LightmapResRatioAdjust settings
 */
class FLightmapResRatioAdjustSettings
{

public:

	/** LightmapResRatioAdjust level options */
	enum AdjustLevels
	{
		/** Current level only */
		Current,
		/** Selected levels only */
		Selected,
		/** All loaded levels */
		AllLoaded
	};

	/** Static: Returns global lightmap resolution ratio adjust settings */
	static FLightmapResRatioAdjustSettings& Get()
	{
		return LightmapResRatioAdjustSettings;
	}

	bool ApplyRatioAdjustment();

	/** 
	* Add levels in a world to an array based on the LevelOptions setting
	*
	* @param InLevelOptions	Level options used to decided whether to add levels to the array
	* @param InWorld		The world in which to get the levels from
	* @param OutLevels		The array to add any levels to (The array is NOT reset by this function)
	*/
	void AddRequiredLevels( AdjustLevels InLevelOptions, UWorld* World , TArray<ULevel*>& OutLevels );

protected:

	/** Static: Global lightmap resolution ratio adjust settings */
	static FLightmapResRatioAdjustSettings LightmapResRatioAdjustSettings;

public:

	/** Constructor */
	FLightmapResRatioAdjustSettings() :
		  Ratio(1.0f)
		, Min_StaticMeshes(32)
		, Max_StaticMeshes(256)
		, Min_BSPSurfaces(1)
		, Max_BSPSurfaces(512)
		, bStaticMeshes(false)
		, bBSPSurfaces(false)
		, LevelOptions(AdjustLevels::Current)
		, bSelectedObjectsOnly(false)
	{
	}

	/** Ratio to apply */
	float Ratio;

	/** Min/Max values */
	/** Static meshes */
	int32 Min_StaticMeshes;
	int32 Max_StaticMeshes;
	/** BSP Surfaces */
	int32 Min_BSPSurfaces;
	int32 Max_BSPSurfaces;

	/** If true, apply to static meshes */
	bool bStaticMeshes;
	/** If true, apply to BSP surfaces*/
	bool bBSPSurfaces;

	/** The primitives to apply the adjustment to */
	uint8 PrimitiveFlags;

	/** The level(s) to check */
	AdjustLevels LevelOptions;

	/** If true, then only do selected primitives in the level(s) */
	bool bSelectedObjectsOnly;
};
