// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Animation/CurveSequence.h"
#include "Styling/SlateColor.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Commands/UICommandInfo.h"
#include "SWorldPartitionViewportWidget.h"
#include "EditorViewportClient.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SWindow.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SAssetEditorViewport.h"
#include "EditorModeManager.h"
#include "IAssetViewport.h"
#include "LevelEditorViewport.h"
#include "SViewportToolBar.h"

#define UE_API LEVELEDITOR_API

class SLevelViewportToolBar;
class FLevelEditorViewportClient;
class FLevelViewportLayout;
class FSceneViewport;
class FUICommandList;
class ILevelEditor;
class SActionableMessageViewportWidget;
class SActorPreview;
class SCaptureRegionWidget;
class SDebugCanvas;
class UFoliageType;
enum class EMapChangeType : uint8;
enum ELabelAnchorMode : int;

// This implementation is only present so that ULevelViewportToolBarContext & UViewportToolbarContext continues to have a valid public pointer.
class UE_DEPRECATED(5.7, "Use the UToolMenu \"LevelEditor.ViewportToolbar\" menu instead.") SLevelViewportToolBar : public SViewportToolBar
{
	SLATE_BEGIN_ARGS(SLevelViewportToolBar) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs) {};
};

/**
 * Encapsulates an SViewport and an SLevelViewportToolBar
 */
class SLevelViewport : public SAssetEditorViewport, public IAssetViewport
{
public:
	SLATE_BEGIN_ARGS( SLevelViewport )
		{}

		SLATE_ARGUMENT( TWeakPtr<ILevelEditor>, ParentLevelEditor )
		SLATE_ARGUMENT( TSharedPtr<FLevelEditorViewportClient>, LevelEditorViewportClient )
	SLATE_END_ARGS()

	UE_API SLevelViewport();
	UE_API ~SLevelViewport();


	/**
	 * Constructs the viewport widget                   
	 */
	UE_API void Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InConstructionArgs);

	/**
	 * Constructs the widgets for the viewport overlay
	 */
	UE_API void ConstructViewportOverlayContent();

	/**
	 * Constructs the level editor viewport client
	 */
	UE_API void ConstructLevelEditorViewportClient(FLevelEditorViewportInstanceSettings& ViewportInstanceSettings);

	/**
	 * @return true if the viewport is visible. false otherwise                  
	 */
	UE_API virtual bool IsVisible() const override;

	/** @return true if this viewport is in a foregrounded tab */
	UE_API bool IsInForegroundTab() const;

	/**
	 * @return The editor client for this viewport
	 */
	const FLevelEditorViewportClient& GetLevelViewportClient() const 
	{		
		return *LevelViewportClient;
	}

	FLevelEditorViewportClient& GetLevelViewportClient() 
	{		
		return *LevelViewportClient;
	}

	virtual FEditorViewportClient& GetAssetViewportClient() override
	{
		return *LevelViewportClient;
	}

	/**
	 * Sets Slate keyboard focus to this viewport
	 */
	UE_API void SetKeyboardFocusToThisViewport();

	/**
	 * @return The list of commands on the viewport that are bound to delegates                    
	 */
	const TSharedPtr<FUICommandList>& GetCommandList() const { return CommandList; }

	/** Saves this viewport's config to ULevelEditorViewportSettings */
	UE_API void SaveConfig(const FString& ConfigName) const;

	/** IAssetViewport Interface */
	UE_API virtual void StartPlayInEditorSession( UGameViewportClient* PlayClient, const bool bInSimulateInEditor ) override;
	UE_API virtual void EndPlayInEditorSession() override;
	UE_API virtual void SwapViewportsForSimulateInEditor() override;
	UE_API virtual void SwapViewportsForPlayInEditor() override;
	UE_API virtual void OnSimulateSessionStarted() override;
	UE_API virtual void OnSimulateSessionFinished() override;
	UE_API virtual void RegisterGameViewportIfPIE() override;
	UE_API virtual bool HasPlayInEditorViewport() const override; 
	UE_API virtual FViewport* GetActiveViewport() override;
	TSharedPtr<FSceneViewport> GetSharedActiveViewport() const override {return ActiveViewport;};
	virtual TSharedRef< const SWidget> AsWidget() const override { return AsShared(); }
	virtual TSharedRef< SWidget> AsWidget() override { return AsShared(); }
	virtual TWeakPtr< SViewport > GetViewportWidget() override { return ViewportWidget; }
	UE_API virtual void AddOverlayWidget( TSharedRef<SWidget> OverlaidWidget, int32 ZOrder=INDEX_NONE ) override;
	UE_API virtual void RemoveOverlayWidget( TSharedRef<SWidget> OverlaidWidget ) override;



	/** SEditorViewport Interface */
	UE_API virtual void OnFocusViewportToSelection() override;
	UE_API virtual EVisibility GetTransformToolbarVisibility() const override;
	UE_API virtual UWorld* GetWorld() const override;
	UE_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	UE_API virtual void ToggleInViewportContextMenu() override;

	UE_API virtual void HideInViewportContextMenu() override;
	UE_API virtual bool CanToggleInViewportContextMenu() override;

	static UE_API void EnableInViewportMenu();
	UE_API FMargin GetContextMenuPadding() const;
	/**
	 * Called when the maximize command is executed                   
	 */
	UE_DEPRECATED(5.7, "Use OnTogleMaximizeMode() instead.")	
	UE_API FReply OnToggleMaximize();

	/**
	 * @return true if this viewport is maximized, false otherwise
	 */
	UE_API bool IsMaximized() const;

	/**
	 * @return true if this viewport can be maximized, false otherwise
	 */
	UE_API bool CanMaximize() const;

	/**
	 * Attempt to switch this viewport into a maximized or normal mode
	 * @param bWantMaximized Whether to witch to maximized or normal
	 * @param bAllowAnimation True to allow animation when transitioning, otherwise false
	 */
	UE_API void MakeMaximized(bool bWantMaximized, bool bAllowAnimation = true);

	/**
	 * Attempts to switch this viewport into immersive mode
	 *
	 * @param	bWantImmersive Whether to switch to immersive mode, or switch back to normal mode
	 * @param	bAllowAnimation	True to allow animation when transitioning, otherwise false
	 */
	UE_API void MakeImmersive( const bool bWantImmersive, const bool bAllowAnimation ) override;

	/**
	 * @return true if this viewport is in immersive mode, false otherwise
	 */
	UE_API bool IsImmersive () const override;
	
	UE_DEPRECATED(5.7, "Immersive mode is now handled differently")
	/**
	 * Called to get the visibility of the viewport's 'Restore from Immersive' button. Returns EVisibility::Collapsed when not in immersive mode
	 */
	UE_API EVisibility GetCloseImmersiveButtonVisibility() const;
		
	UE_DEPRECATED(5.7, "Maximuize & Immersive mode is now handled through the new viewport toolbar.")
	/**
	 * Called to get the visibility of the viewport's maximize/minimize toggle button. Returns EVisibility::Collapsed when in immersive mode
	 */
	UE_API EVisibility GetMaximizeToggleVisibility() const;

	/**
	 * @return true if the active viewport is currently being used for play in editor
	 */
	UE_API bool IsPlayInEditorViewportActive() const;

	/**
	 * @return The active play client, if any 
	 */
	UE_API UGameViewportClient* GetPlayClient() const;

	/**
	 * Called on all viewports, when actor selection changes.
	 * 
	 * @param NewSelection	List of objects that are now selected
	 */
	UE_API void OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh=false);

	/**
	 * Called on all viewports, when element selection changes.
	 * 
	 * @param SelectionSet  New selection
	 * @param bForceRefresh Force refresh
	 */
	UE_API void OnElementSelectionChanged(const UTypedElementSelectionSet* SelectionSet, bool bForceRefresh);

	/**
	 * Called when game view should be toggled
	 */
	UE_API void ToggleGameView() override;

	/**
	 * @return true if we can toggle game view
	 */
	UE_API bool CanToggleGameView() const;

	/**
	 * @return true if we are in game view                   
	 */
	UE_API bool IsInGameView() const override;

	/**
	 * Toggles layer visibility in this viewport
	 *
	 * @param LayerID					Index of the layer
	 */
	UE_API void ToggleShowLayer( FName LayerName );

	/**
	 * Checks if a layer is visible in this viewport
	 *
	 * @param LayerID					Index of the layer
	 */
	UE_API bool IsLayerVisible( FName LayerName ) const;

	/**
	 * Toggles foliage type visibility in this viewport
	 *
	 * @param FoliageType	Target foliage type
	 */
	UE_API void ToggleShowFoliageType(TWeakObjectPtr<class UFoliageType> FoliageType);

	/**
	 * Toggles all foliage types visibility
	 *
	 * @param Visible	true if foliage types should be visible, false otherwise
	 */
	UE_API void ToggleAllFoliageTypes(bool bVisible);

	/**
	 * Checks if a foliage type is visible in this viewport
	 *
	 * @param FoliageType	Target foliage type
	 */
	UE_API bool IsFoliageTypeVisible(TWeakObjectPtr<class UFoliageType> FoliageType) const;


	/** Called to lock/unlock the actor from the viewport's context menu */
	UE_API void OnActorLockToggleFromMenu(AActor* Actor);

	/** Called to unlock the actor from the viewport's context menu */
	UE_API void OnActorLockToggleFromMenu();

	/**
	 * @return true if the actor is locked to the viewport
	 */
	UE_API bool IsActorLocked(const TWeakObjectPtr<AActor> Actor) const;

	/**
	 * @return true if an actor is locked to the viewport
	 */
	UE_API bool IsAnyActorLocked() const;

	/**
	 * @return true if the viewport is locked to selected actor
	 */
	UE_API bool IsSelectedActorLocked() const;

	/**
	 * Toggles enabling the exact camera view when locking a viewport to a camera
	 */
	UE_API void ToggleActorPilotCameraView();

	/**
	 * Check whether locked camera view is enabled
	 */
	UE_API bool IsLockedCameraViewEnabled() const;

	/**
	 * Sets whether the viewport should allow cinematic control
	 * 
	 * @param Whether the viewport should allow cinematic control.
     */
	UE_API void SetAllowsCinematicControl(bool bAllow);

	/**
	 * @return Whether the viewport allows cinematic control.
     */
	UE_API bool GetAllowsCinematicControl() const;

	/**
	 * @return the fixed width that a column returned by CreateActorLockSceneOutlinerColumn expects to be
	 */
	static UE_API float GetActorLockSceneOutlinerColumnWidth();

	/**
	 * @return a new custom column for a scene outliner that indicates whether each actor is locked to this viewport
	 */
	UE_API TSharedRef< class ISceneOutlinerColumn > CreateActorLockSceneOutlinerColumn( class ISceneOutliner& SceneOutliner ) const;

	/** Called when Preview Selected Cameras preference is changed.*/
	UE_API void OnPreviewSelectedCamerasChange();

	/**
	 * Set the device profile name
	 *
	 * @param ProfileName The profile name to set
	 */
	UE_DEPRECATED(5.7, "Device profiles & preview platforms are handled in the new ToolMenus toolbar.")
	void SetDeviceProfileString( const FString& ProfileName ) {}

	/**
	 * @return true if the in profile name matches the set profile name
	 */
	UE_DEPRECATED(5.7, "Device profiles & preview platforms are handled in the new ToolMenus toolbar.")
	bool IsDeviceProfileStringSet( FString ProfileName ) const { return false; }

	/**
	 * @return the name of the selected device profile
	 */
	UE_DEPRECATED(5.7, "Device profiles & preview platforms are handled in the new ToolMenus toolbar.")
	FString GetDeviceProfileString( ) const { return FString(); }

	/** Get the parent level editor for this viewport */
	TWeakPtr<ILevelEditor> GetParentLevelEditor() const { return ParentLevelEditor; }

	/** @return whether the the actor editor context should be displayed for this viewport */
	UE_API virtual bool IsActorEditorContextVisible() const;

	/** Called to get the screen percentage preview text */
	UE_API FText GetCurrentScreenPercentageText() const;

	UE_DEPRECATED(5.1, "GetCurrentLevelTextVisibility not used anymore.")
	virtual EVisibility GetCurrentLevelTextVisibility() const { return EVisibility::Collapsed; }
	UE_DEPRECATED(5.1, "GetCurrentLevelButtonVisibility not used anymore.")
	virtual EVisibility GetCurrentLevelButtonVisibility() const { return EVisibility::Collapsed; }

	/** @return The visibility of the current level text display */
	UE_API virtual EVisibility GetSelectedActorsCurrentLevelTextVisibility() const;

	/** Called to get the text for the level the currently selected actor or actors are in. */
	UE_API FText GetSelectedActorsCurrentLevelText(bool bDrawOnlyLabel) const;

	/** @return The visibility of the current screen percentage text display */
	UE_API EVisibility GetCurrentScreenPercentageVisibility() const;

	UE_DEPRECATED(5.7, "This function is no longer used.")
	/** @return The visibility of the viewport controls popup */
	virtual EVisibility GetViewportControlsVisibility() const { return EVisibility::Collapsed; }

	/**
	 * Called to get the visibility of the level viewport toolbar
	 */
	UE_API virtual EVisibility GetToolBarVisibility() const;

	/**
	 * Sets the current layout on the parent tab that this viewport belongs to.
	 * 
	 * @param ConfigurationName		The name of the layout (for the names in namespace LevelViewportConfigurationNames)
	 */
	UE_DEPRECATED(5.1, "Moved to internal handling by FViewportTabContent. See FViewportTabContent::BindCommonViewportCommands")
	UE_API void OnSetViewportConfiguration(FName ConfigurationName);

	/**
	 * Returns whether the named layout is currently selected on the parent tab that this viewport belongs to.
	 *
	 * @param ConfigurationName		The name of the layout (for the names in namespace LevelViewportConfigurationNames)
	 * @return						True, if the named layout is currently active
	 */
	UE_DEPRECATED(5.1, "Moved to internal handling by FViewportTabContent. See FViewportTabContent::BindCommonViewportCommands")
	UE_API bool IsViewportConfigurationSet(FName ConfigurationName) const;

	/** Get this level viewport widget's type within its parent layout */
	UE_DEPRECATED(5.1, "Moved to internal handling by FViewportTabContent. See FViewportTabContent::BindCommonViewportCommands")
	UE_API FName GetViewportTypeWithinLayout() const;

	/** Set this level viewport widget's type within its parent layout */
	UE_DEPRECATED(5.1, "Moved to internal handling by FViewportTabContent. See FViewportTabContent::BindCommonViewportCommands")
	UE_API void SetViewportTypeWithinLayout(FName InLayoutType);

	/** Activates the specified viewport type in the layout, if it's not already, or reverts to default if it is. */
	UE_DEPRECATED(5.1, "Moved to internal handling by FViewportTabContent. See FViewportTabContent::BindCommonViewportCommands")
	UE_API void ToggleViewportTypeActivationWithinLayout(FName InLayoutType);

	/** Checks if the specified layout type matches our current viewport type. */
	UE_DEPRECATED(5.1, "Moved to internal handling by FViewportTabContent. See FViewportTabContent::BindCommonViewportCommands")
	UE_API bool IsViewportTypeWithinLayoutEqual(FName InLayoutType);

	/** For the specified actor, See if we're forcing a preview */
	UE_API bool IsActorAlwaysPreview(TWeakObjectPtr<AActor> Actor) const;

	/** For the specified actor, toggle Pinned/Unpinned of it's ActorPreview */
	UE_API void SetActorAlwaysPreview(TWeakObjectPtr<AActor> PreviewActor, bool bAlwaysPreview = true);

	/** For the specified actor, toggle Pinned/Unpinned of it's ActorPreview */
	UE_API void ToggleActorPreviewIsPinned(TWeakObjectPtr<AActor> PreviewActor);

	/** For the specified actor, toggle whether the panel is detached from it*/
	UE_API void ToggleActorPreviewIsPanelDetached(TWeakObjectPtr<AActor> PreviewActor);

	/** See if the specified actor's ActorPreview is pinned or not */
	UE_API bool IsActorPreviewPinned(TWeakObjectPtr<AActor> PreviewActor);

	/** See if the specified actor's ActorPreview is detached from actor */
	UE_API bool IsActorPreviewDetached(TWeakObjectPtr<AActor> PreviewActor);

	/** Actions to perform whenever the viewports floating buttons are pressed */
	UE_API void OnFloatingButtonClicked();

	UE_DEPRECATED(5.6, "Call GetOptionsMenuVisibility() instead.")
	EVisibility GetToolbarVisibility() const { return EVisibility::Visible;}

	UE_DEPRECATED(5.7, "Toolbar visibility is no longer controlled in partial steps.")
	/** Get the visibility for viewport toolbar's options menu. */
	EVisibility GetOptionsMenuVisibility() const { return EVisibility::Visible; }

	UE_DEPRECATED(5.7, "Toolbar visibility is no longer controlled in partial steps.")
	/** Get the visibility for items considered to be part of the 'full' viewport toolbar */
	EVisibility GetFullToolbarVisibility() const { return EVisibility::SelfHitTestInvisible; }

	/** Unpin and close all actor preview windows */
	UE_API void RemoveAllPreviews(const bool bRemoveFromDesktopViewport = true);

	/**
	 * Called to set a bookmark
	 *
	 * @param BookmarkIndex	The index of the bookmark to set
	 */
	UE_API void OnSetBookmark( int32 BookmarkIndex );

	/**
	 * Called to check if a bookmark is set
	 *
	 * @param BookmarkIndex	The index of the bookmark to check
	 */
	UE_API bool OnHasBookmarkSet(int32 BookmarkIndex);

	/**
	 * Called to jump to a bookmark
	 *
	 * @param BookmarkIndex	The index of the bookmark to jump to
	 */
	UE_API void OnJumpToBookmark( int32 BookmarkIndex );

	/**
	 * Called to clear a bookmark
	 *
	 * @param BookmarkIndex The index of the bookmark to clear
	 */
	UE_API void OnClearBookmark( int32 BookmarkIndex );

	/**
	 * Called to clear all bookmarks
	 */
	UE_API void OnClearAllBookmarks();

	/**
	 * Called to Compact Bookmarks.
	 */
	UE_API void OnCompactBookmarks();

	/** 
	 * Returns the config key associated with this viewport. 
	 * This is what is used when loading/saving per viewport settings. 
	 * If a plugin extends a LevelViewport menu, they'll be able to identify it and match their settings accordingly
	 */
	FName GetConfigKey() const { return ConfigKey; }

	/**
	 * Toggles the current viewport toolbar visibility. 
	 */
	UE_API void ToggleViewportToolbarVisibility();

	/**
	 * Whether the viewport toolbar is currently visible.
	 */
	UE_API bool IsViewportToolbarVisible() const; 

protected:
	/**
	 * Registers the viewport toolbar UToolMenu and returns its name.
	 */
	UE_API virtual FName RegisterViewportToolbar() const;

	/** SEditorViewport interface */
	UE_API virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	UE_API virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	UE_API TSharedRef<SWidget> BuildPIEViewportToolbar();

	UE_API virtual void OnIncrementPositionGridSize() override;
	UE_API virtual void OnDecrementPositionGridSize() override;
	UE_API virtual void OnIncrementRotationGridSize() override;
	UE_API virtual void OnDecrementRotationGridSize() override;
	UE_API virtual const FSlateBrush* OnGetViewportBorderBrush() const override;
	UE_API virtual FSlateColor OnGetViewportBorderColorAndOpacity() const override;
	UE_API virtual EVisibility OnGetViewportContentVisibility() const override;
	UE_API virtual EVisibility OnGetFocusedViewportIndicatorVisibility() const override;
	UE_API virtual void BindCommands() override;

	/** SWidget Interface */
	UE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	UE_API virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	/** End of SWidget interface */
	
private:
	/** Flag to know if we need to update the previews which is handled in the tick. */
	bool bNeedToUpdatePreviews;

	/** Loads this viewport's config from the ini file */
	UE_API FLevelEditorViewportInstanceSettings LoadLegacyConfigFromIni(const FString& ConfigKey, const FLevelEditorViewportInstanceSettings& InDefaultSettings);

	/** Called when a property is changed */
	UE_API void HandleViewportSettingChanged(FName PropertyName);

	/**
	 * Handles which viewport toolbar is active.
	 */
	UE_API int32 GetViewportToolbarIndex() const;

	/**
	 * Called when the advanced settings should be opened.
	 */
	UE_API void OnAdvancedSettings();

	/**
	 * Called when the play settings should be opened.
	 */
	UE_API void OnPlaySettings();

	/**
	 * Called when immersive mode is toggled by the user
	 */
	UE_API void OnToggleImmersive();

	/** Called when moving tabs in and out of a sidebar is activated by the user */
	UE_API void OnToggleSidebarTabs();

	/**
	* Called to determine whether the maximize mode of current viewport can be toggled
	*/
	UE_API bool CanToggleMaximizeMode() const;

	/**
	* Called to toggle maximize mode of current viewport
	*/
	UE_API void OnToggleMaximizeMode();

	/** Starts previewing any selected camera actors using live "PIP" sub-views */
	UE_API void PreviewSelectedCameraActors(const bool bPreviewInDesktopViewport = true);

	/**
	 * Called to create a cameraActor in the currently selected perspective viewport
	 */
	UE_API void OnCreateCameraActor(UClass *InClass);

	/**
	 * Called to bring up the screenshot UI
	 */
	UE_API void OnTakeHighResScreenshot();

	/**
	 * Returns whether this level viewport is the active level viewport 
	 */
	UE_API bool IsActiveLevelViewport() const;

	/**
	 * Called to check currently selected editor viewport is a perspective one
	 */
	UE_API bool IsPerspectiveViewport() const;

	/**
	 * Toggles all volume classes visibility
	 *
	 * @param Visible					true if volumes should be visible, false otherwise
	 */
	UE_API void OnToggleAllVolumeActors( bool bVisible );

	/**
	 * Toggles volume classes visibility
	 *
	 * @param VolumeID					Index of the volume class
	 */
	UE_API void ToggleShowVolumeClass( int32 VolumeID );

	/**
	 * Checks if volume class is visible in this viewport
	 *
	 * @param VolumeID					Index of the volume class
	 */
	UE_API bool IsVolumeVisible( int32 VolumeID ) const;

	/**
	 * Toggles all layers visibility
	 *
	 * @param Visible					true if layers should be visible, false otherwise
	 */
	UE_API void OnToggleAllLayers( bool bVisible );

	/**
	 * Toggles all sprite categories visibility
	 *
	 * @param Visible					true if sprites should be visible, false otherwise
	 */
	UE_API void OnToggleAllSpriteCategories( bool bVisible );

	/**
	 * Toggles sprite category visibility in this viewport
	 *
	 * @param CategoryID				Index of the category
	 */
	UE_API void ToggleSpriteCategory( int32 CategoryID );

	/**
	 * Checks if sprite category is visible in this viewport
	 *
	 * @param CategoryID				Index of the category
	 */
	UE_API bool IsSpriteCategoryVisible( int32 CategoryID ) const;

	/**
	 * Toggles all Stat commands visibility
	 *
	 * @param Visible					true if Stats should be visible, false otherwise
	 */
	UE_API void OnToggleAllStatCommands(bool bVisible);

	/**
	 * Called when show flags for this viewport should be reset to default, or the saved settings
	 */
	UE_API void OnUseDefaultShowFlags(bool bUseSavedDefaults = false);

	/**
	 * Called to toggle allowing sequencer to use this viewport to preview in
	 */
	UE_API void OnToggleAllowCinematicPreview();

	/**
	 * @return true if this viewport allows cinematics to be previewed in it                   
	 */
	UE_API bool AllowsCinematicPreview() const;

	/** Find currently selected actor in the level script.  */
	UE_API void FindSelectedInLevelScript();
	
	/** Can we find the currently selected actor in the level script. */
	UE_API bool CanFindSelectedInLevelScript() const;

	/** Called to select the currently locked actor */
	UE_API void OnSelectLockedActor();

	/**
	 * @return true if the currently locked actor is selectable
	 */
	UE_API bool CanExecuteSelectLockedActor() const;

	/** Called to clear the current actor lock */
	UE_API void OnActorUnlock();

	/**
	 * @return true if clearing the current actor lock is a valid input
	 */
	UE_API bool CanExecuteActorUnlock() const;

	/** Called to lock the viewport to the currently selected actor */
	UE_API void OnActorLockSelected();

	/**
	 * @return true if clearing the setting the actor lock to the selected actor is a valid input
	 */
	UE_API bool CanExecuteActorLockSelected() const;

	/**
	 * Called when the viewport should be redrawn
	 *
	 * @param bInvalidateHitProxies	Whether or not to invalidate hit proxies
	 */
	UE_API void RedrawViewport( bool bInvalidateHitProxies );

	/** An internal handler for dagging dropable objects into the viewport. */
	UE_API bool HandleDragObjects(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

	/** An internal handler for dropping objects into the viewport. 
	 *	@param DragDropEvent		The drag event.
	 *	@param bCreateDropPreview	If true, a drop preview actor will be spawned instead of a normal actor.
	 */
	UE_API bool HandlePlaceDraggedObjects(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool bCreateDropPreview);

	/**
	 * Tries to get assets from a drag and drop event.
	 * 
	 * @param DragDropEvent		Event to get assets from.
	 * @param AssetDataArray	Asets will be added here.
	 */
	UE_API void GetAssetsFromDrag(const FDragDropEvent& DragDropEvent, TArray<FAssetData>& AssetDataArray);

	/**
	 * Bound event Triggered via FLevelViewportCommands::ApplyMaterialToActor, attempts to apply a material selected in the content browser
	 * to an actor being hovered over in the Editor viewport.
	 */ 
	UE_API void OnApplyMaterialToViewportTarget();

	/**
	 * Binds commands for our toolbar options menu
	 *
	 * @param CommandList	The list to bind commands to
	 */
	UE_API void BindOptionCommands( FUICommandList& CommandList );

	/**
	 * Binds commands for our toolbar view menu
	 *
	 * @param CommandList	The list to bind commands to
	 */
	UE_API void BindViewCommands( FUICommandList& CommandList );

	/**
	 * Binds commands for our toolbar show menu
	 *
	 * @param CommandList	The list to bind commands to
	 */
	UE_API void BindShowCommands( FUICommandList& CommandList ) override;

	/**
	 * Binds commands for our drag-drop context menu
	 *
	 * @param CommandList	The list to bind commands to
	 */
	UE_API void BindDropCommands( FUICommandList& CommandList );

	/**
	* Binds commands for our stat menu, also used as a delegate listener
	*
	* @param InMenuItem		The menu item we need to bind
	* @param InCommandName	The command used by the functions
	*/
	UE_API void BindStatCommand(const TSharedPtr<FUICommandInfo> InMenuItem, const FString& InCommandName);

	/**
	 * Binds commands for PIE-specific behavior
	 * @param OutCommandList	The list to bind commands to
	 */
	UE_API void BindPIECommands( FUICommandList& OutCommandList );

	/**
	 * Resets the PIE show flags to default.
	 */
	UE_API void ResetPIEShowFlags();
	
	/**
	 * Toggles a PIE show flag on or off.
	 */
	UE_API void TogglePIEShowFlag(FEngineShowFlags::EShowFlag Flag);

	/**
	 * Returns whether a show flag is on in PIE.
	 */
	UE_API bool IsPIEShowFlagEnabled(FEngineShowFlags::EShowFlag Flag) const;

	/**
	 * Sets the view mode on the PIE viewport.
	 */
	UE_API void SetPIEViewMode(EViewModeIndex ViewMode);

	/**
	 * Gets whether the view mode is enabled in the PIE Viewport
	 */
	UE_API bool IsPIEViewModeEnabled(EViewModeIndex ViewMode) const;

	/**
	 * Refreshes the PIE viewport.
	 */
	UE_API void RefreshPIEViewport();

	/**
	 * A handler for processing input events before gameplay does.
	 */
	UE_API bool OnPIEViewportInputOverride(FInputKeyEventArgs& Input);
	
	/**
	 * Called to build the viewport context menu when the user is Drag Dropping from the Content Browser
	 */
	UE_API TSharedRef< SWidget > BuildViewportDragDropContextMenu();

	/**
	 * Called when a map is changed (loaded,saved,new map, etc)
	 */
	UE_API void OnMapChanged( UWorld* World, EMapChangeType MapChangeType );

	/** Called in response to an actor being deleted in the level */
	UE_API void OnLevelActorsRemoved(AActor* InActor);
	
	UE_API void OnEditorClose();

	/** Gets the locked icon tooltip text showing the meaning of the icon and the name of the locked actor */
	UE_API FText GetLockedIconToolTip() const;

	/**
	 * Starts previewing the specified actors.  If the supplied list of actors is empty, turns off the preview.
	 *
	 * @param	ActorsToPreview		List of actors to draw previews for
	 */
	UE_API void PreviewActors( const TArray< AActor* >& ActorsToPreview, const bool bPreviewInDesktopViewport = true);

	/** Called every frame to update any actor preview viewports we may have */
	UE_API void UpdateActorPreviewViewports();

	/**
	 * Removes a specified actor preview from the list
	 *
	 * @param PreviewIndex Array index of the preview to remove.
	 */
	UE_API void RemoveActorPreview( int32 PreviewIndex, AActor* Actor = nullptr, const bool bRemoveFromDesktopViewport = true);
	
	/** Returns true if this viewport is the active viewport and can process UI commands */
	UE_API bool CanProduceActionForCommand(const TSharedRef<const FUICommandInfo>& Command) const;

	/** Called when undo is executed */
	UE_API void OnUndo();

	/** Called when undo is executed */
	UE_API void OnRedo();

	/** @return Whether or not undo can be executed */
	UE_API bool CanExecuteUndo() const;

	/** @return Whether or not redo can be executed */
	UE_API bool CanExecuteRedo() const;

	/** @return Whether the mouse capture label is visible */
	UE_API EVisibility GetMouseCaptureLabelVisibility() const;

	/** @return The current color & opacity for the mouse capture label */
	UE_API FLinearColor GetMouseCaptureLabelColorAndOpacity() const;

	/** @return The current text for the mouse capture label */
	UE_API FText GetMouseCaptureLabelText() const;

	/** Show the mouse capture label with the specified vertical and horizontal alignment */
	UE_API void ShowMouseCaptureLabel(ELabelAnchorMode AnchorMode);
	
	/** Build the mouse capture label widget */
	UE_API TSharedRef<SWidget> BuildMouseCaptureWidget();

	/** Hide the mouse capture label */
	UE_API void HideMouseCaptureLabel();

	/** Resets view flags when a new level is created or opened */
	UE_API void ResetNewLevelViewFlags();

	/** Gets the active scene viewport for the game */
	UE_API FSceneViewport* GetGameSceneViewport() const;
	
	/** Handle any level viewport changes on entering PIE or simulate */
	UE_API void TransitionToPIE(bool bIsSimulating);

	/** Handle any level viewport changes on leaving PIE or simulate */
	UE_API void TransitionFromPIE(bool bIsSimulating);

	/** Get the stretch type of the viewport */
	UE_API EStretch::Type OnGetScaleBoxStretch() const;

	/** Get the SViewport size */
	UE_API FVector2D GetSViewportSize() const;

	/** Updates the real-time overrride applied to the viewport */
	UE_API void OnPerformanceSettingsChanged(UObject* Obj, struct FPropertyChangedEvent& ChangeEvent);

	/** Called when the user wants to enable or disable editor shortcuts when PIE is active. */
	UE_API void OnToggleEditorShortcutsInPIE();

	/** Called to determine when editor shortcuts in PIE is active. */
	UE_API bool IsEditorShortcutsEnabledInPIE();

private:
	/** Tab which this viewport is located in */
	TWeakPtr<class FLevelViewportLayout> ParentLayout;

	/** Pointer to the parent level editor for this viewport */
	TWeakPtr<ILevelEditor> ParentLevelEditor;

	/** Viewport horizontal box used internally for drawing actor previews on top of the level viewport */
	TSharedPtr<SHorizontalBox> ActorPreviewHorizontalBox;

	/** Active Slate viewport for rendering and I/O (Could be a pie viewport)*/
	TSharedPtr<class FSceneViewport> ActiveViewport;

	/**
	 * Inactive Slate viewport for rendering and I/O
	 * If this is valid there is a pie viewport and this is the previous level viewport scene viewport 
	 */
	TSharedPtr<class FSceneViewport> InactiveViewport;

	/** Debug canvas overlay for editor (non-PIE) mode; swapped out for SGameLayerManager during PIE */
	TSharedPtr<SDebugCanvas> EditorDebugCanvas;

	/**
	 * When in PIE this will contain the editor content for the viewport widget (toolbar). It was swapped
	 * out for GameUI content
	 */
	TSharedPtr<SWidget> InactiveViewportWidgetEditorContent;

	/** 
	 *  When PIE is active, the handle for the change feature level delegate
	 */
	FDelegateHandle PIEPreviewFeatureLevelChangedHandle;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<SLevelViewportToolBar> LevelViewportToolBar;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Level viewport client */
	TSharedPtr<FLevelEditorViewportClient> LevelViewportClient;
	
	/** The GameViewportClient */
	TWeakObjectPtr<UGameViewportClient> PlayClient;
	
	/** Commands specifically for PIE */
	TSharedPtr<FUICommandList> PIECommands;

	/** The brush to use if this viewport is in debug mode */
	const FSlateBrush* DebuggingBorder;
	/** The brush to use for a black background */
	const FSlateBrush* BlackBackground;
	/** The brush to use when transitioning into Play in Editor mode */
	const FSlateBrush* StartingPlayInEditorBorder;
	/** The brush to use when transitioning into Simulate mode */
	const FSlateBrush* StartingSimulateBorder;
	/** The brush to use when returning back to the editor from PIE or SIE mode */
	const FSlateBrush* ReturningToEditorBorder;
	/** The brush to use when the viewport is not maximized */
	const FSlateBrush* NonMaximizedBorder;
	/** Array of objects dropped during the OnDrop event */
	TArray<UObject*> DroppedObjects;

	/** Caching off of the DragDropEvent Local Mouse Position grabbed from OnDrop */
	FIntPoint CachedOnDropLocalMousePos;

	/** Weak pointer to the highres screenshot dialog if it's open. Will become invalid if UI is closed whilst the viewport is open */
	TWeakPtr<class SWindow> HighResScreenshotDialog;

	/** Pointer to the capture region widget in the viewport overlay. Enabled by the high res screenshot UI when capture region selection is required */
	TSharedPtr<class SCaptureRegionWidget> CaptureRegionWidget;

	/** Types of transition effects we support */
	struct EViewTransition
	{
		enum Type
		{
			/** No transition */
			None = 0,

			/** Fade in from black */
			FadingIn,

			/** Entering PIE */
			StartingPlayInEditor,

			/** Entering SIE */
			StartingSimulate,

			/** Leaving either PIE or SIE */
			ReturningToEditor
		};
	};

	/** Type of transition we're currently playing */
	EViewTransition::Type ViewTransitionType;

	/** Animation progress within current view transition */
	FCurveSequence ViewTransitionAnim;

	/** True if we want to kick off a transition animation but are waiting for the next tick to do so */
	bool bViewTransitionAnimPending;

	/** The current viewport config key */
	FName ConfigKey;

	/**
	 * Contains information about an actor being previewed within this viewport
	 */
	class FViewportActorPreview
	{

	public:

		FViewportActorPreview() 
			: bIsPinned(false)
			, bIsPanelDetached(false)
		{}

		void ToggleIsPinned()
		{
			bIsPinned = !bIsPinned;
		}

		void ToggleIsPanelDetached()
		{
			bIsPanelDetached = !bIsPanelDetached;
		}

		/** The Actor that is the center of attention. */
		TWeakObjectPtr< AActor > Actor;

		/** Level viewport client for our preview viewport */
		TSharedPtr< FLevelEditorViewportClient > LevelViewportClient;

		/** The scene viewport */
		TSharedPtr< FSceneViewport > SceneViewport;

		/** Slate widget that represents this preview in the viewport */
		TSharedPtr< SActorPreview > PreviewWidget;

		/** Whether or not this actor preview will remain on screen if the actor is deselected */
		bool bIsPinned;	

		/** Whether this actor preview is displayed in a detached panel */
		bool bIsPanelDetached;
	};

	/** List of actor preview objects */
	TArray< FViewportActorPreview > ActorPreviews;

	/** Storage for actors we always want to preview.  This comes from MU transactions .*/
	TSet<TWeakObjectPtr<AActor>> AlwaysPreviewActors;

	/** The border in the SOverlay for the PIE mouse control label */
	TSharedPtr<class SBorder> PIEOverlayBorder;

	/** Separate curve to control fading out the PIE mouse control label */
	FCurveSequence PIEOverlayAnim;

	/** Whether the PIE view has focus so we can track when to reshow the mouse control label */
	bool bPIEHasFocus;

	/** Whether the PIE view contains focus (even if not captured), if so we disable throttling. */
	bool bPIEContainsFocus;

	/** The users value for allowing throttling, we restore this value when we lose focus. */
	int32 UserAllowThrottlingValue;

	/** Whether to show the editor toolbar */
	bool bShowEditorToolbar = true;
	
	/** Whether to show the in-PIE toolbar */
	bool bShowPIEToolbar = true;
	
	/** Whether to show the in-PIE toolbar in immersive mode */
	bool bShowImmersivePIEToolbar = false;
	
	TSharedPtr<class SWidget> InViewportMenuWrapper;
	bool bIsInViewportMenuShowing;
	bool bIsInViewportMenuInitialized;
	TSharedPtr<class SInViewportDetails> InViewportMenu;
	static UE_API bool bInViewportMenuEnabled;

	TSharedPtr<SWorldPartitionViewportWidget> WorldPartitionViewportWidget;

	/** Viewport widget for warning messages */
	TSharedPtr<SActionableMessageViewportWidget> ActionableMessageViewportWidget;

	/**
	 * Used to store last perspective camera transform before piloting (Actor Lock)
	 * Once piloting ends, this transform is re-applied to the camera.
	 */
	FViewportCameraTransform CachedPerspectiveCameraTransform;

protected:
	UE_API void LockActorInternal(AActor* NewActorToLock);

	/** Can be overriden by derived classes to add new context objects to the new toolbar */
	virtual void ExtendToolbarContext(FToolMenuContext& InToolMenuContext) {};

public:
	static UE_API bool GetCameraInformationFromActor(AActor* Actor, FMinimalViewInfo& out_CameraInfo);

	static UE_API bool CanGetCameraInformationFromActor(AActor* Actor);
};

#undef UE_API
