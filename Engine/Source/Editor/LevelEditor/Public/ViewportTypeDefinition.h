// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FUICommandInfo;
class ILevelViewportLayoutEntity;
struct FAssetEditorViewportConstructionArgs;
class ILevelEditor;

/** Definition of a custom viewport */
struct FViewportTypeDefinition
{
	typedef TFunction<TSharedRef<ILevelViewportLayoutEntity>(const FAssetEditorViewportConstructionArgs&, TSharedPtr<ILevelEditor>)> FFactoryFunctionType;

	template<typename T>
	static FViewportTypeDefinition FromType(const TSharedPtr<FUICommandInfo>& ActivationCommand)
	{
		return FViewportTypeDefinition([](const FAssetEditorViewportConstructionArgs& Args, TSharedPtr<ILevelEditor> InLevelEditor) -> TSharedRef<ILevelViewportLayoutEntity> {
			return MakeShareable(new T(Args, InLevelEditor));
		}, ActivationCommand);
	}

	FViewportTypeDefinition(const FFactoryFunctionType& InFactoryFunction, const TSharedPtr<FUICommandInfo>& InActivationCommand)
		: ActivationCommand(InActivationCommand)
		, FactoryFunction(InFactoryFunction)
	{}

	/** A UI command for toggling activation this viewport */
	TSharedPtr<FUICommandInfo> ActivationCommand;

	/** Function used to create a new instance of the viewport */
	FFactoryFunctionType FactoryFunction;
};
